[CoreRedirects]
; 4.27 > 4.27 (internal build)
+ClassRedirects=(OldName="DisjunctiveNormalFormFilter", NewName="LevelSnapshotsFilterPreset")

; 4.27 > 5.0
+PropertyRedirects=(OldName="SnapshotFileVersionInfo.FileVersionLicenseeUE4",NewName="SnapshotFileVersionInfo.FileVersionLicensee")
+PropertyRedirects=(OldName="ClassDefaultObjectSnapshotData.bWasBlacklistedCDO",NewName="ClassDefaultObjectSnapshotData.bSerializationSkippedCDO")
+PropertyRedirects=(OldName="SubobjectSnapshotData.bWasBlacklisted",NewName="SubobjectSnapshotData.bWasSkippedClass")

+ClassRedirects=(OldName="LevelSnapshotsEditorProjectSettings", NewName="LevelSnapshotsSettings")
+ClassRedirects=(OldName="LevelSnapshotsEditorDataManagementSettings", NewName="LevelSnapshotsEditorSettings")


