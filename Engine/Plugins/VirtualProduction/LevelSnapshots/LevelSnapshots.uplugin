{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LevelSnapshots",
	"Description": "",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": false,
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LevelSnapshots",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LevelSnapshotFilters",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LevelSnapshotsEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "FoliageSupport",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostDefault"
		},
        {
            "Name": "nDisplaySupport",
            "Type": "UncookedOnly",
            "LoadingPhase": "PostDefault",
            "PlatformAllowList": [
              "Win64",
              "Linux"
            ]
        } 
	],
	"Plugins": [
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		}
	]
}