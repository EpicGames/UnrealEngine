// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FoliageSupport : ModuleRules
{
	public FoliageSupport(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"LevelSnapshots",
                "Foliage"
			}
			);
			
		if (Target.bBuildEditor)
        {
            // This is so we can update foliage UI after applying a snapshot
            PrivateDependencyModuleNames.Add("FoliageEdit");
        }
	}
}
