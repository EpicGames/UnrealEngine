// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageLevelSnapshotsConsoleVariables.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	TAutoConsoleVariable<bool> CVarAllowFoliageDataPre5dot1(
		TEXT("LevelSnapshots.AllowFoliageDataBefore5dot1"),
		false,
		TEXT("Foliage data saved before 5.1 was saved in a bad format which may cause a crash. Data from 5.1 is unusable. It is safe to enable this if either:\n"
			"A. you added static mesh actors into the foliage tool (e.g. drop-drop static mesh from content browser)\n"
			"B. you added foliage types into the foliage tool (e.g. drag-dop foliage type from content browser)")
		);
}