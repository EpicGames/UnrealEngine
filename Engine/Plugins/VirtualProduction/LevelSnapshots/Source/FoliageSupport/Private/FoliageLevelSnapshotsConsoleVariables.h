// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	/**
	 * Foliage data saved before 5.2 was saved in a bad format which may cause a crash. It is safe to enable this if either:
	 *	1. All foliage were assets (possible / conceivable)
	 *	2. All foliage were subobjects (possible / conceivable)
	 *	3. First assets were added, then subobjects subobjects (unlikely)
	 *	Subobject = add by drag-droping static mesh, asset = drag-drop UFoliageType asset
	 */
	extern TAutoConsoleVariable<bool> CVarAllowFoliageDataPre5dot1;
}
