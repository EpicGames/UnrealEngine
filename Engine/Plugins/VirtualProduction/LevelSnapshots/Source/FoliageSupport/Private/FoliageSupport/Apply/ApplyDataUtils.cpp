// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplyDataUtils.h"

#include "LevelSnapshotsLog.h"
#include "Filtering/PropertySelectionMap.h"

#include "FoliageType.h"
#include "InstancedFoliageActor.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	FFoliageInfo* FindOrAddFoliageInfo(UFoliageType* FoliageType, AInstancedFoliageActor* FoliageActor)
	{
		FFoliageInfo* FoliageInfo = FoliageActor->FindInfo(FoliageType);
		if (!FoliageInfo)
		{
			FoliageInfo = &FoliageActor->AddFoliageInfo(FoliageType).Get();
		}

		UE_CLOGF(FoliageInfo == nullptr, LogLevelSnapshots, Warning, "Failed to create foliage info for foliage type %ls", *FoliageType->GetPathName());
		return FoliageInfo; 
	}

	void HandleExistingFoliageUsingRequiredComponent(AInstancedFoliageActor* FoliageActor, FName RequiredComponentName, const TMap<FName, UFoliageType*>& PreexistingComponentToFoliageType, UFoliageType* AllowedFoliageType)
	{
		// Handles the following case:
		// 1. Add (static mesh) foliage type FoliageA. Let's suppose the component that is added is called Component01.
		// 2. Take snapshot
		// 3. Reload map without saving
		// 4. Add foliage type FoliageB. This creates a new component that is also called Component01.
		// 5. Restore snapshot
		// 
		// Without the below check, both FoliageA and FoliageB would use Component01 to add instances.
		// We remove the foliage type under the "interpretation" that Component01 is "restored" to reuse the previous foliage type.
		if (UFoliageType* const* FoliageType = PreexistingComponentToFoliageType.Find(RequiredComponentName); FoliageType && *FoliageType != AllowedFoliageType)
		{
			TUniqueObj<FFoliageInfo> FoliageInfo;
			FoliageActor->RemoveFoliageInfoAndCopyValue(*FoliageType, FoliageInfo);
		}
	}

	void RemoveComponentAutoRecreatedByLevelSnapshots(AInstancedFoliageActor* FoliageActor, TOptional<FName> RecreatedComponentName)
	{
		if (ensure(RecreatedComponentName))
		{
			for (UHierarchicalInstancedStaticMeshComponent* Component : TInlineComponentArray<UHierarchicalInstancedStaticMeshComponent*>(FoliageActor))
			{
				if (Component->GetFName() == *RecreatedComponentName)
				{
					Component->DestroyComponent();
					return;
				}
			}
		}
	}

	TMap<FName, UFoliageType*> BuildComponentToFoliageType(AInstancedFoliageActor* FoliageActor)
	{
		TMap<FName, UFoliageType*> Result;
		for (auto It = FoliageActor->GetFoliageInfos().CreateConstIterator(); It; ++It)
		{
			if (UHierarchicalInstancedStaticMeshComponent* Component = It->Value->GetComponent())
			{
				Result.Add(Component->GetFName(), It->Key);
			}
		}
		return Result;
	}
}