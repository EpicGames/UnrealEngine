// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"

class AInstancedFoliageActor;
class UFoliageType;
struct FFoliageInfo;

namespace UE::LevelSnapshots::Foliage::Private
{
	/** Adds the foliage type to the instanced actor or returns the associated FFoliageInfo if already present. */
	FFoliageInfo* FindOrAddFoliageInfo(UFoliageType* FoliageType, AInstancedFoliageActor* FoliageActor);

	/**
	 * Handles name clashes with pre-existing instanced components.
	 *
	 * This handles in particular this case
	 * 1. Add (static mesh) foliage type FoliageA. Let's suppose the component that is added is called Component01.
	 * 2. Take snapshot
	 * 3. Reload map without saving
	 * 4. Add foliage type FoliageB. This creates a new component that is also called Component01.
	 * 5. Restore snapshot
	 *
	 * Both FoliageA and FoliageB would use Component01 to add instances.
	 * We remove the foliage type under the "interpretation" that Component01 is "restored" to reuse the previous foliage type
	 */
	void HandleExistingFoliageUsingRequiredComponent(AInstancedFoliageActor* FoliageActor, FName RequiredComponentName, const TMap<FName, UFoliageType*>& PreexistingComponentToFoliageType, UFoliageType* AllowedFoliageType);

	/** Default LS restoration will recreate all components. However, if the user said to skip restoring some of the components, we'll remove the wrongly added components again. */
	void RemoveComponentAutoRecreatedByLevelSnapshots(AInstancedFoliageActor* FoliageActor, TOptional<FName> RecreatedComponentName);

	/** Returns an inverse map binding hierarchical mesh component names to the foliage type they represent. */
	TMap<FName, UFoliageType*> BuildComponentToFoliageType(AInstancedFoliageActor* FoliageActor);
}
