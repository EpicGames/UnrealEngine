// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/Data/InstancedFoliageActorData.h"

#include "ApplyDataUtils.h"
#include "Filtering/PropertySelectionMap.h"
#include "FoliageRestorationInfo.h"
#include "LevelSnapshotsLog.h"
#include "SnapshotCustomVersion.h"

#include "FoliageType.h"
#include "InstancedFoliageActor.h"
#include "Serialization/Archive.h"
#include "Misc/ScopeExit.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	static void ApplyAssets_5dot2(
		FArchive& Archive,
		AInstancedFoliageActor* FoliageActor,
		const FFoliageRestorationInfo& RestorationInfo,
		const TMap<TSoftObjectPtr<UFoliageType>, FFoliageInfoData>& FoliageAssets,
		const TMap<FName, UFoliageType*> PreexistingComponentToFoliageType)
	{
		for (auto AssetIt = FoliageAssets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FFoliageInfoData& FoliageInfoData = AssetIt->Value;
			UFoliageType* FoliageType = AssetIt->Key.LoadSynchronous();
			if (!FoliageType)
			{
				UE_LOGF(LogLevelSnapshots, Warning, "Foliage type %ls is missing. Component %ls will not be restored.", *AssetIt->Key.ToString(), *FoliageInfoData.GetComponentName().Get(NAME_None).ToString());
				continue;
			}

			if (RestorationInfo.ShouldSkipFoliageType(FoliageInfoData) || !RestorationInfo.ShouldSerializeFoliageType(FoliageInfoData))
			{
				continue;
			}

			FFoliageInfo* FoliageInfo = FindOrAddFoliageInfo(FoliageType, FoliageActor);
			if (FoliageInfo)
			{
				// If there were no instances, no component existed, hence no component name could be saved
				const bool bHadAtLeastOneInstanceWhenSaved = FoliageInfoData.GetComponentName().IsSet();
				if (bHadAtLeastOneInstanceWhenSaved)
				{
					HandleExistingFoliageUsingRequiredComponent(FoliageActor, *FoliageInfoData.GetComponentName(), PreexistingComponentToFoliageType, FoliageType);
				}
				
				FoliageInfoData.ApplyTo(Archive, *FoliageInfo);
			}
			else
			{
				// Standard component restoration has already recreated the component. Remove it again.
				RemoveComponentAutoRecreatedByLevelSnapshots(FoliageActor, FoliageInfoData.GetComponentName());
			}
		}
	}

	static void ApplySubobjects_5dot2(
		FArchive& Archive,
		AInstancedFoliageActor* FoliageActor,
		const FFoliageRestorationInfo& RestorationInfo,
		const TArray<FSubobjectFoliageInfoData>& SubobjectData, 
		const TMap<FName, UFoliageType*> PreexistingComponentToFoliageType)
	{
		for (const FSubobjectFoliageInfoData& InstanceData : SubobjectData)
		{
			if (RestorationInfo.ShouldSkipFoliageType(InstanceData) || !RestorationInfo.ShouldSerializeFoliageType(InstanceData))
			{
				continue;
			}

			UFoliageType* FoliageType = InstanceData.FindOrRecreateSubobject(FoliageActor);
			if (!FoliageType)
			{
				UE_LOGF(LogLevelSnapshots, Error, "Failed to recreate foliage type. Skipping...");
				continue;
			}

			bool bWasRestored = false;
			ON_SCOPE_EXIT
			{
				if (!bWasRestored)
				{
					// Standard component restoration has already recreated the component. Remove it again.
					RemoveComponentAutoRecreatedByLevelSnapshots(FoliageActor, InstanceData.GetComponentName());
				}
			};

			if (!InstanceData.ApplyToFoliageType(Archive, FoliageType))
			{
				UE_LOGF(LogLevelSnapshots, Warning, "Skipping restoration of foliage item due to errors.");
				continue;
			}
				
			FFoliageInfo* FoliageInfo = FindOrAddFoliageInfo(FoliageType, FoliageActor);
			if (FoliageInfo
				&& ensureAlwaysMsgf(InstanceData.GetComponentName(), TEXT("ComponentName is supposed to have been saved. Investigate. Maybe you used an unsupported foliage type (only EFoliageImplType::StaticMesh is supported)?")))
			{
				HandleExistingFoliageUsingRequiredComponent(FoliageActor, *InstanceData.GetComponentName(), PreexistingComponentToFoliageType, FoliageType);
				InstanceData.ApplyToFoliageInfo(Archive, *FoliageInfo);
				bWasRestored = true;
			}
		}
	}
	
	void FInstancedFoliageActorData::ApplyTo_5dot2(FArchive& Archive, AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated) const
	{
		check(Archive.CustomVer(FSnapshotCustomVersion::GUID) >= FSnapshotCustomVersion::FoliageTypesUnreadable);

		const FFoliageRestorationInfo RestorationInfo = FFoliageRestorationInfo::From(FoliageActor, SelectedProperties, bWasRecreated);
		const TMap<FName, UFoliageType*> PreexistingComponentToFoliageType = BuildComponentToFoliageType(FoliageActor);
		ApplyAssets_5dot2(Archive, FoliageActor, RestorationInfo, FoliageAssets, PreexistingComponentToFoliageType);
		ApplySubobjects_5dot2(Archive, FoliageActor, RestorationInfo, SubobjectData, PreexistingComponentToFoliageType);
	}
}