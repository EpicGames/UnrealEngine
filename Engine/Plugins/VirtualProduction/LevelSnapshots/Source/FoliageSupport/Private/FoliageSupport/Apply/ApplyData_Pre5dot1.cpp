// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/Data/InstancedFoliageActorData.h"

#include "ApplyDataUtils.h"
#include "Filtering/PropertySelectionMap.h"
#include "FoliageLevelSnapshotsConsoleVariables.h"
#include "FoliageRestorationInfo.h"
#include "LevelSnapshotsLog.h"
#include "SnapshotCustomVersion.h"

#include "FoliageType.h"
#include "InstancedFoliageActor.h"
#include "Misc/ScopeExit.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	static void ApplyAssets_Pre5dot2(
		FArchive& Archive,
		AInstancedFoliageActor* FoliageActor,
		const FFoliageRestorationInfo& RestorationInfo,
		const TMap<TSoftObjectPtr<UFoliageType>, FFoliageInfoData>& FoliageAssets,
		const TMap<FName, UFoliageType*> PreexistingComponentToFoliageType)
	{
		// Some objects may not want to be restored (either due to errors or because user deselected them) ...
		int64 DataToSkip = 0;
		
		for (auto AssetIt = FoliageAssets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FFoliageInfoData& FoliageInfoData = AssetIt->Value;
			
			UFoliageType* FoliageType = AssetIt->Key.LoadSynchronous();
			if (!FoliageType)
			{
				UE_LOGF(LogLevelSnapshots, Warning, "Foliage type %ls is missing. Component %ls will not be restored.", *AssetIt->Key.ToString(), *FoliageInfoData.GetComponentName().Get(NAME_None).ToString());
				continue;
			}

			if (RestorationInfo.ShouldSkipFoliageType(FoliageInfoData) || !RestorationInfo.ShouldSerializeFoliageType(FoliageInfoData))
			{
				continue;
			}

			FFoliageInfo* FoliageInfo = FindOrAddFoliageInfo(FoliageType, FoliageActor);
			if (FoliageInfo)
			{
				// If there were no instances, no component existed, hence no component name could be saved
				const bool bHadAtLeastOneInstanceWhenSaved = FoliageInfoData.GetComponentName().IsSet();
				if (bHadAtLeastOneInstanceWhenSaved)
				{
					HandleExistingFoliageUsingRequiredComponent(FoliageActor, *FoliageInfoData.GetComponentName(), PreexistingComponentToFoliageType, FoliageType);
				}

				// ... since Archive data is sequential we must account for possibly having skipped some data
				Archive.Seek(Archive.Tell() + DataToSkip);
				DataToSkip = 0;
				
				FoliageInfoData.ApplyTo(Archive, *FoliageInfo);
			}
			else
			{
				// Standard component restoration has already recreated the component. Remove it again.
				RemoveComponentAutoRecreatedByLevelSnapshots(FoliageActor, FoliageInfoData.GetComponentName());
				DataToSkip += FoliageInfoData.GetFoliageInfoArchiveSize();
			}
		}
	}

	static void ApplySubobjects_Pre5dot2(
		FArchive& Archive,
		AInstancedFoliageActor* FoliageActor,
		const FFoliageRestorationInfo& RestorationInfo,
		const TArray<FSubobjectFoliageInfoData>& SubobjectData, 
		const TMap<FName, UFoliageType*> PreexistingComponentToFoliageType)
	{
		// Some objects may not want to be restored (either due to errors or because user deselected them) ...
		int64 DataToSkip = 0;
		
		for (const FSubobjectFoliageInfoData& InstanceData : SubobjectData)
		{
			if (RestorationInfo.ShouldSkipFoliageType(InstanceData) || !RestorationInfo.ShouldSerializeFoliageType(InstanceData))
			{
				continue;
			}

			UFoliageType* FoliageType = InstanceData.FindOrRecreateSubobject(FoliageActor);
			if (!FoliageType)
			{
				UE_LOGF(LogLevelSnapshots, Error, "Failed to recreate foliage type. Skipping...");
				continue;
			}

			// ... since Archive data is sequential we must account for possibly having skipped some data
			Archive.Seek(Archive.Tell() + DataToSkip);
			DataToSkip = 0;
			
			bool bWasRestored = false;
			bool bWasPartiallyRestored = false;
			ON_SCOPE_EXIT
			{
				if (!bWasRestored)
				{
					// Standard component restoration has already recreated the component. Remove it again.
					RemoveComponentAutoRecreatedByLevelSnapshots(FoliageActor, InstanceData.GetComponentName());
					DataToSkip += static_cast<uint64>(bWasPartiallyRestored) * InstanceData.GetFoliageTypeArchiveSize() + InstanceData.GetFoliageInfoArchiveSize();
					// Technically we have to remove the foliage info from FoliageActor ... it was already serialized into FoliageInfos
				}
			};

			bWasPartiallyRestored = true;
			if (!InstanceData.ApplyToFoliageType(Archive, FoliageType))
			{
				UE_LOGF(LogLevelSnapshots, Warning, "Skipping restoration of foliage item due to errors.");
				continue;
			}
				
			FFoliageInfo* FoliageInfo = FindOrAddFoliageInfo(FoliageType, FoliageActor);
			if (FoliageInfo
				&& ensureAlwaysMsgf(InstanceData.GetComponentName(), TEXT("ComponentName is supposed to have been saved. Investigate. Maybe you used an unsupported foliage type (only EFoliageImplType::StaticMesh is supported)?")))
			{
				HandleExistingFoliageUsingRequiredComponent(FoliageActor, *InstanceData.GetComponentName(), PreexistingComponentToFoliageType, FoliageType);
				InstanceData.ApplyToFoliageInfo(Archive, *FoliageInfo);
				bWasRestored = true;
			}
		}
	}
	
	void FInstancedFoliageActorData::ApplyData_Pre5dot2(FArchive& Archive, AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated) const
	{
		check(Archive.CustomVer(FSnapshotCustomVersion::GUID) < FSnapshotCustomVersion::FoliageTypesUnreadable);
		
		/*
		 * Remember: Subobject = add by drag-droping static mesh, asset = drag-drop UFoliageType asset
		 *
		 * Simplified version of the previous saving code:
		 * FoliageActor->ForEachFoliageInfo([this, &Archive, FoliageActor](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
		 * {
		 *		if (bIsSubobject) Internal::SaveSubobject(Archive, FoliageType, FoliageInfo, SubobjectData);
		 *		else Internal::SaveAsset(Archive, FoliageType, FoliageInfo, FoliageAssets);
		 * });
		 * 
		 * ApplyAssets and ApplySubobjects just iterate the map from beginning to end.
		 * If the foliage infos are interleaved, we'll get a crash. There is NO way to know in advance.
		 * 
		 * Example interleaved, dangerous saved state (with hypothetical byte sizes):
		 * [0] Subobject1	(42 bytes)
		 * [1] Subobject2	(58 bytes)
		 * [2] Asset1		(100 bytes)		// Ok so far
		 * [3] Subobject3	(50 bytes)		// This will cause a crash
		 *
		 * The above would crash in ApplyAssets because it would do Archive << x, reading data it expects to be Asset1.
		 * This code only works in the following cases:
		 * 1. Everything assets (possible / conceivable)
		 * 2. Everything subobjects (possible / conceivable)
		 * 3. First assets, then subobjects (unlikely)
		 */
		const bool bIsSafeData = Archive.CustomVer(FSnapshotCustomVersion::GUID) < FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor;
		if (bIsSafeData || ensureMsgf(CVarAllowFoliageDataPre5dot1.GetValueOnGameThread(), TEXT("This check was already supposed to have happened in FFoliageSupport's IActorSnapshotFilter implementation")))
		{
			UE_CLOGF(!bIsSafeData, LogLevelSnapshots, Warning, "Data saved before 5.1 may crash the editor (unlikely).");
			
			const FFoliageRestorationInfo RestorationInfo = FFoliageRestorationInfo::From(FoliageActor, SelectedProperties, bWasRecreated);
			const TMap<FName, UFoliageType*> PreexistingComponentToFoliageType = BuildComponentToFoliageType(FoliageActor);
			ApplyAssets_Pre5dot2(Archive, FoliageActor, RestorationInfo, FoliageAssets, PreexistingComponentToFoliageType);
			ApplySubobjects_Pre5dot2(Archive, FoliageActor, RestorationInfo, SubobjectData, PreexistingComponentToFoliageType);
		}
		else
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Use \"LevelSnapshots.DangerousAllowFoliageDataPre5dot1 true\" to enable restoring foliage data from before 5.2 (may crash).");
		}
	}
}
