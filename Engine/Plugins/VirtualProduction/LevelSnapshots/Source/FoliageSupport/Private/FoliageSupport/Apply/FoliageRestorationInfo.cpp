// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/Apply/FoliageRestorationInfo.h"

#include "Filtering/AddedAndRemovedComponentInfo.h"
#include "Filtering/RestorableObjectSelection.h"
#include "Filtering/PropertySelectionMap.h"
#include "FoliageSupport/Data/FoliageInfoData.h"
#include "LevelSnapshotsLog.h"

#include "InstancedFoliageActor.h"

UE::LevelSnapshots::Foliage::Private::FFoliageRestorationInfo UE::LevelSnapshots::Foliage::Private::FFoliageRestorationInfo::From(AInstancedFoliageActor* Object, const FPropertySelectionMap& SelectionMap, bool bWasRecreated)
{
	FFoliageRestorationInfo Result;
	Result.bWasRecreated = bWasRecreated;
	
	const FRestorableObjectSelection ObjectSelection = SelectionMap.GetObjectSelection(Object);
	if (const FAddedAndRemovedComponentInfo* ComponentInfo = ObjectSelection.GetComponentSelection())
	{
		Result.EditorWorldComponentsToRemove = ComponentInfo->EditorWorldComponentsToRemove.Array();
		Result.SnapshotComponentsToAdd = ComponentInfo->SnapshotComponentsToAdd.Array();
	}

	for (UHierarchicalInstancedStaticMeshComponent* FoliageComp : TInlineComponentArray<UHierarchicalInstancedStaticMeshComponent*>(Object))
	{
		if (SelectionMap.HasChanges(FoliageComp))
		{
			Result.ModifiedComponents.Add(FoliageComp);
		}
	}

	return Result;
}

bool UE::LevelSnapshots::Foliage::Private::FFoliageRestorationInfo::ShouldSkipFoliageType(const FFoliageInfoData& SavedData) const
{
	return SavedData.GetImplType() != EFoliageImplType::StaticMesh;
}

bool UE::LevelSnapshots::Foliage::Private::FFoliageRestorationInfo::ShouldSerializeFoliageType(const FFoliageInfoData& SavedData) const
{
	if (bWasRecreated)
	{
		return true;
	}
	
	if (TOptional<FName> CompName = SavedData.GetComponentName(); CompName && SavedData.GetImplType() == EFoliageImplType::StaticMesh)
	{
		const bool bIsAddedComponent = SnapshotComponentsToAdd.ContainsByPredicate(
			[CompName](const TWeakObjectPtr<UActorComponent>& Component)
			{
				return Component.IsValid() && Component->GetFName().IsEqual(*CompName);
			});
		const bool bIsModifiedComponent = ModifiedComponents.ContainsByPredicate(
			[CompName](const UActorComponent* Component)
			{
				return Component->GetFName().IsEqual(*CompName);
			});

		return bIsAddedComponent || bIsModifiedComponent;
	}

	// Actor foliage is currently not supported: just restore everything
	UE_CLOGF(SavedData.GetImplType() != EFoliageImplType::StaticMesh, LogLevelSnapshots, Warning, "Only static mesh foliage is supported");
	return false;
}
