// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AInstancedFoliageActor;
class UActorComponent;
struct FPropertySelectionMap;

namespace UE::LevelSnapshots::Foliage::Private
{
	class FFoliageInfoData;
	
	/** Tells us whether foliage types should be added / removed / serialized into AInstancedFoliageActor::FoliageInfos */
	class FFoliageRestorationInfo
	{
		TArray<UActorComponent*> ModifiedComponents;
		TArray<TWeakObjectPtr<UActorComponent>> EditorWorldComponentsToRemove;
		TArray<TWeakObjectPtr<UActorComponent>> SnapshotComponentsToAdd;

		bool bWasRecreated = false;
	public:
	
		static FFoliageRestorationInfo From(AInstancedFoliageActor* Object, const FPropertySelectionMap& SelectionMap, bool bWasRecreated);

		bool ShouldSkipFoliageType(const FFoliageInfoData& SavedData) const;
		bool ShouldSerializeFoliageType(const FFoliageInfoData& SavedData) const;
	};
}