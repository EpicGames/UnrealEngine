// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/Data/FoliageInfoData.h"

#include "LevelSnapshotsLog.h"
#include "SnapshotCustomVersion.h"

#include "InstancedFoliageActor.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	static FString AsName(EFoliageImplType ImplType)
	{
		switch(ImplType)
		{
		case EFoliageImplType::StaticMesh:
			return FString("StaticMesh");
		case EFoliageImplType::Actor:
			return FString("Actor");
		case EFoliageImplType::ISMActor:
			return FString("ISMActor");
			
		case EFoliageImplType::Unknown: 
			return FString("Unknown");
		default:
			ensureMsgf(false, TEXT("Update this switch case."));
			return FString("NewType (Update FoliageInfoData.cpp)");
		}
	}
}

FArchive& UE::LevelSnapshots::Foliage::Private::FFoliageInfoData::SerializeInternal(FArchive& Ar)
{
	Ar << ImplType;
	Ar << ComponentName;
	if (Ar.CustomVer(FSnapshotCustomVersion::GUID) < FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
	{
		Ar << SerializedData_DEPRECATED;
	}
	else
	{
		check(Ar.CustomVer(FSnapshotCustomVersion::GUID) >= FSnapshotCustomVersion::FoliageTypesUnreadable);
		Ar << FoliageInfoArchiveSize;
		Ar << ArchiveTellBeforeSerialization;
	}
	return Ar;
}

void UE::LevelSnapshots::Foliage::Private::FFoliageInfoData::FillDataMembersAndWriteFoliageInfoIntoArchive(FArchive& Archive, FFoliageInfo& DataToReadFrom)
{
	ArchiveTellBeforeSerialization = Archive.Tell();
	ImplType = DataToReadFrom.Type;
	
	const bool bIsComponentValid = DataToReadFrom.GetComponent() != nullptr; 
	ComponentName = bIsComponentValid ? DataToReadFrom.GetComponent()->GetFName() : FName(NAME_None);
	UE_CLOGF(!bIsComponentValid, LogLevelSnapshots, Warning, "Could not save component for ImplType %ls", *AsName(ImplType));

	int64 ArchivePos = Archive.Tell();
	Archive << DataToReadFrom;
	FoliageInfoArchiveSize = Archive.Tell() - ArchivePos;
}

void UE::LevelSnapshots::Foliage::Private::FFoliageInfoData::ApplyTo(FArchive& Archive, FFoliageInfo& DataToWriteInto) const
{
	// Avoid foliage internal checks
	DataToWriteInto.Implementation.Reset();
	
	if (Archive.CustomVer(FSnapshotCustomVersion::GUID) >= FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
	{
		Archive.Seek(ArchiveTellBeforeSerialization);
		Archive << DataToWriteInto;
	}
	else
	{
		FMemoryReader MemoryReader(SerializedData_DEPRECATED, true);
		FObjectAndNameAsStringProxyArchive RootArchive(MemoryReader, false);
		RootArchive.SetCustomVersions(Archive.GetCustomVersions());
		RootArchive << DataToWriteInto;
	}
	
	DataToWriteInto.RecomputeHash();
}