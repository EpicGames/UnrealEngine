// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InstancedFoliage.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	/** Data we'll send to FFoliageImpl::Serialize */
	class FFoliageInfoData
	{
		EFoliageImplType ImplType = EFoliageImplType::Unknown;
		FName ComponentName;
		
		/** Used before FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor */
		TArray<uint8> SerializedData_DEPRECATED;
		/** Added after FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor */
		int64 FoliageInfoArchiveSize = 0;
		
		/**
		 * What Ar.Tell returned at the beginning of CaptureDataAndWriteFoliageInfoIntoArchive (since 5.2, FoliageTypesUnreadable).
		 * Used to jump across data to selectively apply data to foliage (since the user can choose to restore partially).
		 */
		int64 ArchiveTellBeforeSerialization;

		FArchive& SerializeInternal(FArchive& Ar);
	
	public:

		/** Saves FFoliageInfo into the archive and updates our data members. The our data members are NOT saved into the archive at this point. */
		void FillDataMembersAndWriteFoliageInfoIntoArchive(FArchive& Archive, FFoliageInfo& DataToReadFrom);
		/**
		 * Given an archive containing data written by FillDataMembersAndWriteFoliageInfoIntoArchive previously and applies that data to the FFoliageInfo.
		 * 5.0 and before still uses SerializedData_DEPRECATED for this while 5.2 and beyond seeks to ArchiveTellBeforeSerialization and then reads.
		 */
		void ApplyTo(FArchive& Archive, FFoliageInfo& DataToWriteInto) const;

		EFoliageImplType GetImplType() const { return ImplType; }
		TOptional<FName> GetComponentName() const { return ImplType == EFoliageImplType::StaticMesh && !ComponentName.IsNone() ? TOptional<FName>(ComponentName) : TOptional<FName>(); }
		
		int64 GetFoliageInfoArchiveSize() const { return FoliageInfoArchiveSize; }
		int64 GetArchiveTellBeforeSerialization() const { return  ArchiveTellBeforeSerialization; }
	
		friend FArchive& operator<<(FArchive& Ar, FFoliageInfoData& MeshInfo)
		{
			return MeshInfo.SerializeInternal(Ar);
		}
	};
}
