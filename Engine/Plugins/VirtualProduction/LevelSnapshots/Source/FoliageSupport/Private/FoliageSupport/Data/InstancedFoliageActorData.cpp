// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/Data/InstancedFoliageActorData.h"

#include "Filtering/PropertySelectionMap.h"
#include "SnapshotCustomVersion.h"

#include "FoliageType.h"
#include "InstancedFoliageActor.h"
#include "Serialization/Archive.h"

namespace UE::LevelSnapshots::Foliage::Private
{
	static void SaveAsset(FArchive& Archive, UFoliageType* FoliageType, FFoliageInfo& FoliageInfo, TMap<TSoftObjectPtr<UFoliageType>, FFoliageInfoData>& FoliageAssets)
	{
		FFoliageInfoData& InfoData = FoliageAssets.Add(FoliageType);
		InfoData.FillDataMembersAndWriteFoliageInfoIntoArchive(Archive, FoliageInfo);
	}
	
	static void SaveSubobject(FArchive& Archive, UFoliageType* FoliageType, FFoliageInfo& FoliageInfo,TArray<FSubobjectFoliageInfoData>& SubobjectData)
	{
		FSubobjectFoliageInfoData Data;
		Data.FillDataMembersAndSerializeFoliageTypeAndInfoIntoArchive(Archive, FoliageType, FoliageInfo);
		SubobjectData.Emplace(Data);
	}

	void FInstancedFoliageActorData::CaptureData(FArchive& Archive, AInstancedFoliageActor* FoliageActor)
	{
		FInstancedFoliageActorData Data;
		
		// Reserve an int64 so later we can write how big the saved data was... See LoadAndApplyTo for details
		const int64 LocBeforeWrite = Archive.Tell();
		int64 Offset = 0;
		Archive << Offset;

		// Block 1 (see LoadAndApplyTo)
		FoliageActor->ForEachFoliageInfo([&Archive, FoliageActor, &Data](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
		{
			FFoliageImpl* Impl = FoliageInfo.Implementation.Get();
			if (!FoliageType || !Impl)
			{
				return true;
			}
			
			const bool bIsSubobject = FoliageType->IsIn(FoliageActor);
			if (bIsSubobject)
			{
				SaveSubobject(Archive, FoliageType, FoliageInfo, Data.SubobjectData);
			}
			else
			{
				SaveAsset(Archive, FoliageType, FoliageInfo, Data.FoliageAssets);
			}
			
			return true;
		});

		// ... Now go to the beginning and write how big the saved data was ...
		Offset = Archive.Tell() - LocBeforeWrite;
		Archive.Seek(LocBeforeWrite);
		Archive << Offset;
		// ... And make sure to go back to the end so anyone after us won't overwrite our data
		Archive.Seek(LocBeforeWrite + Offset);

		// Block 2 (see LoadAndApplyTo)
		Archive << Data;
	}

	void FInstancedFoliageActorData::LoadAndApplyTo(FArchive& Archive, AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated)
	{
		FInstancedFoliageActorData CurrentFoliageData;
		if (Archive.CustomVer(FSnapshotCustomVersion::GUID) < FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
		{
			// In this version, all data was saved inline in FFoliageInfoData::SerializedData_DEPRECATED, so Archive << Data is enough
			FInstancedFoliageActorData Data;
			Archive << Data;
			Data.ApplyData_Pre5dot2(Archive, FoliageActor, SelectedProperties, bWasRecreated);
		}
		else
		{
			// In this version, we have two blocks...
			checkf(Archive.CustomVer(FSnapshotCustomVersion::GUID) >= FSnapshotCustomVersion::FoliageTypesUnreadable,
				TEXT("Data from CustomSubobjectSoftObjectPathRefactor <= x < FoliageTypesUnreadable should have been discarded by the IActorSnapshotFilter implementation!")
				);
			
			// ... block 1's size is saved at beginning. Contains "random" sub-blocks of of 1.1 FFoliageInfo or 1.2 FFoliageInfo followed by UFoliageType data
			int64 Offset = 0;
			Archive << Offset;
			
			// ... block 2: Archive << Data. FFoliageInfoData::ArchiveTellBeforeSerialization contains the sub-block locations to be used for FArchive::Seek() in ApplyTo_Post5dot2.
			Archive.Seek(Offset);
			FInstancedFoliageActorData Data;
			Archive << Data;

			Data.ApplyTo_5dot2(Archive, FoliageActor, SelectedProperties, bWasRecreated);
		}
	}

	FArchive& operator<<(FArchive& Archive, FInstancedFoliageActorData& Data)
	{
		Archive << Data.FoliageAssets;
		Archive << Data.SubobjectData;
		return Archive;
	}
}
