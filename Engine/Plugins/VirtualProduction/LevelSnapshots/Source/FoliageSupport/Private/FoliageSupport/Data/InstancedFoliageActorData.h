// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FoliageSupport/Data/SubobjectFoliageInfoData.h"
#include "FoliageSupport/Data/FoliageInfoData.h"
#include "UObject/SoftObjectPtr.h"

class FArchive;
class UFoliageType;
struct FPropertySelectionMap;

namespace UE::LevelSnapshots::Foliage::Private
{
	/** Handles saving of AInstancedFoliageActor::FoliageInfos */
	class FInstancedFoliageActorData
	{
		/**
		 * Foliage type asset references.
		 * 
		 * Consists of foliage type assets added via the green Add button or drag-drop a foliage type or if you drag-drop
		 * a foliage type from the content browser.
		 */
		TMap<TSoftObjectPtr<UFoliageType>, FFoliageInfoData> FoliageAssets;

		/**
		 * Saves foliage type subobjects.
		 * Consists of foliage types created when you drag-drop a static mesh or add Blueprint derived foliage types.
		 */
		TArray<FSubobjectFoliageInfoData> SubobjectData;
		
		/** Serialization version pre 5.1. Data saved in 5.1 is discarded. Looks up saved foliage types and restores the passed in foliage actor. */
		void ApplyData_Pre5dot2(FArchive& Archive, AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated) const;

		/** Serialization version starting 5.2. Data saved in 5.1 is discarded. */
		void ApplyTo_5dot2(FArchive& Archive, AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated) const;

		friend FArchive& operator<<(FArchive& Archive, FInstancedFoliageActorData& Data);
		
	public:

		/** Saves foliage actor into this object */
		static void CaptureData(FArchive& Archive, AInstancedFoliageActor* FoliageActor);

		/** Loads the saved foliage data and applies it to FoliageActor respecting the selection in SelectedProperties. */
		static void LoadAndApplyTo(FArchive& Archive, AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated);
	};
}
