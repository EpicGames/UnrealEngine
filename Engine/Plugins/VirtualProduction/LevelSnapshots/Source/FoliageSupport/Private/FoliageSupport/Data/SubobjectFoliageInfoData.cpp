// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/Data/SubobjectFoliageInfoData.h"

#include "FoliageInfoData.h"
#include "LevelSnapshotsLog.h"
#include "SnapshotCustomVersion.h"

#include "FoliageType_InstancedStaticMesh.h"
#include "InstancedFoliageActor.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

FArchive& UE::LevelSnapshots::Foliage::Private::FSubobjectFoliageInfoData::SerializeInternal(FArchive& Ar)
{
	Ar << static_cast<FFoliageInfoData&>(*this);
	Ar << Class;
	Ar << SubobjectName;
	if (Ar.CustomVer(FSnapshotCustomVersion::GUID) < FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
	{
		Ar << SerializedSubobjectData_DEPRECATED;
	}
	else
	{
		Ar << FoliageTypeArchiveSize;
	}
	return Ar;
}

void UE::LevelSnapshots::Foliage::Private::FSubobjectFoliageInfoData::FillDataMembersAndSerializeFoliageTypeAndInfoIntoArchive(FArchive& Archive, UFoliageType* FoliageSubobject, FFoliageInfo& FoliageInfo)
{
	FFoliageInfoData::FillDataMembersAndWriteFoliageInfoIntoArchive(Archive, FoliageInfo);

	Class = FoliageSubobject->GetClass();
	SubobjectName = FoliageSubobject->GetFName();

	const int64 ArchivePos = Archive.Tell();
	FoliageSubobject->Serialize(Archive);
	FoliageTypeArchiveSize += Archive.Tell() - ArchivePos;
}

UFoliageType* UE::LevelSnapshots::Foliage::Private::FSubobjectFoliageInfoData::FindOrRecreateSubobject(AInstancedFoliageActor* Outer) const
{
	if (UObject* FoundObject = StaticFindObjectFast(nullptr, Outer, SubobjectName))
	{
		UE_CLOGF(FoundObject->GetClass() != Class, LogLevelSnapshots, Warning, "Name collision for foliage type %ls", *FoundObject->GetPathName());
		return Cast<UFoliageType>(FoundObject);
	}

	return NewObject<UFoliageType>(Outer, Class, SubobjectName, RF_Transactional);
}

bool UE::LevelSnapshots::Foliage::Private::FSubobjectFoliageInfoData::ApplyToFoliageType(FArchive& Archive, UFoliageType* FoliageSubobject) const
{
	if (Archive.CustomVer(FSnapshotCustomVersion::GUID) >= FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
	{
		Archive.Seek(GetArchiveTellBeforeSerialization() + GetFoliageInfoArchiveSize());
		FoliageSubobject->Serialize(Archive);
	}
	else
	{
		FMemoryReader MemoryReader(SerializedSubobjectData_DEPRECATED, true);
		constexpr bool bLoadIfFindFails = true;
		FObjectAndNameAsStringProxyArchive RootArchive(MemoryReader, bLoadIfFindFails);
		FoliageSubobject->Serialize(RootArchive);
	}

	if (UFoliageType_InstancedStaticMesh* MeshFoliageType = Cast<UFoliageType_InstancedStaticMesh>(FoliageSubobject)
		; ensureMsgf(MeshFoliageType, TEXT("Only static mesh foliage is supported right now")))
	{
		// There used to be a bug where the asset registry was not aware
		UE_CLOGF(!MeshFoliageType->Mesh, LogLevelSnapshots, Error, "Reference to foliage mesh was corrupted.");
		return MeshFoliageType->Mesh != nullptr;
	}

	return true;
}

void UE::LevelSnapshots::Foliage::Private::FSubobjectFoliageInfoData::ApplyToFoliageInfo(FArchive& Archive, FFoliageInfo& DataToWriteInto) const
{
	FFoliageInfoData::ApplyTo(Archive, DataToWriteInto);
}
