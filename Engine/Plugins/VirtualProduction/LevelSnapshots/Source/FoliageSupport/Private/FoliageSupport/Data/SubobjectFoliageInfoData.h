// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FoliageSupport/Data/FoliageInfoData.h"
#include "Containers/Array.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"

class FArchive;
class UFoliageType;

namespace UE::LevelSnapshots::Foliage::Private
{
	class FSubobjectFoliageInfoData;
	FArchive& operator<<(FArchive& Ar, FSubobjectFoliageInfoData& Data);
	
	class FSubobjectFoliageInfoData : public FFoliageInfoData
	{
		TSubclassOf<UFoliageType> Class;
		FName SubobjectName;
		
		/** Used before FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor*/
		TArray<uint8> SerializedSubobjectData_DEPRECATED;
		/** Added after FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor */
		int64 FoliageTypeArchiveSize = 0;

		FArchive& SerializeInternal(FArchive& Ar);
	
	public:

		void FillDataMembersAndSerializeFoliageTypeAndInfoIntoArchive(FArchive& Archive, UFoliageType* FoliageSubobject, FFoliageInfo& FoliageInfo);
	
		UFoliageType* FindOrRecreateSubobject(AInstancedFoliageActor* Outer) const;
		bool ApplyToFoliageType(FArchive& Archive, UFoliageType* FoliageSubobject) const;
		void ApplyToFoliageInfo(FArchive& Archive, FFoliageInfo& DataToWriteInto) const;

		int64 GetFoliageTypeArchiveSize() const { return FoliageTypeArchiveSize; }
		
		friend FArchive& operator<<(FArchive& Ar, FSubobjectFoliageInfoData& Data)
		{
			return Data.SerializeInternal(Ar);
		}
	};
}
