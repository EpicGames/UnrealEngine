// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupport/FoliageSupport.h"

#include "Filtering/PropertySelection.h"
#include "Filtering/PropertySelectionMap.h"
#include "FoliageSupport/Data/InstancedFoliageActorData.h"
#include "Params/ObjectSnapshotSerializationData.h"
#include "SnapshotCustomVersion.h"
#include "WorldSnapshotData.h"

#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/Level.h"
#include "FoliageHelper.h"
#include "FoliageLevelSnapshotsConsoleVariables.h"
#include "ILevelSnapshotsModule.h"
#include "InstancedFoliageActor.h"
#if WITH_EDITOR
#include "FoliageEditModule.h"
#endif

#define LOCTEXT_NAMESPACE "LevelSnapshots.FoliageSupport"

namespace UE::LevelSnapshots::Foliage::Private
{
	static void EnableRequiredFoliageProperties(ILevelSnapshotsModule& Module)
	{
		FProperty* InstanceReorderTable = UInstancedStaticMeshComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, InstanceReorderTable));

		UClass* HierarchicalClass				= UHierarchicalInstancedStaticMeshComponent::StaticClass(); 
		FProperty* SortedInstances				= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, SortedInstances));
		FProperty* NumBuiltInstances			= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, NumBuiltInstances));
		FProperty* NumBuiltRenderInstances		= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, NumBuiltRenderInstances));
		FProperty* BuiltInstanceBounds			= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, BuiltInstanceBounds));
		FProperty* UnbuiltInstanceBounds		= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, UnbuiltInstanceBounds));
		FProperty* UnbuiltInstanceBoundsList	= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, UnbuiltInstanceBoundsList));
		FProperty* bEnableDensityScaling		= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, bEnableDensityScaling));
		FProperty* OcclusionLayerNumNodes		= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, OcclusionLayerNumNodes));
		FProperty* CacheMeshExtendedBounds		= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, CacheMeshExtendedBounds));
		FProperty* bDisableCollision			= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, bDisableCollision));
		FProperty* InstanceCountToRender		= HierarchicalClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UHierarchicalInstancedStaticMeshComponent, InstanceCountToRender));
		
		if (ensure(InstanceReorderTable))
		{
			Module.AddExplicitilySupportedProperties({
				InstanceReorderTable,
				SortedInstances,
				NumBuiltInstances,
				NumBuiltRenderInstances,
				BuiltInstanceBounds,
				UnbuiltInstanceBounds,
				UnbuiltInstanceBoundsList,
				bEnableDensityScaling,
				OcclusionLayerNumNodes,
				CacheMeshExtendedBounds,
				bDisableCollision,
				InstanceCountToRender
			});
		}
	}

	static UFoliageType* FindFoliageInfoFor(UHierarchicalInstancedStaticMeshComponent* Component)
	{
		AInstancedFoliageActor* Foliage = Cast<AInstancedFoliageActor>(Component->GetOwner());
		if (!LIKELY(Foliage))
		{
			return nullptr;
		}

		for (auto FoliageIt = Foliage->GetFoliageInfos().CreateConstIterator(); FoliageIt; ++FoliageIt)
		{
			if (FoliageIt->Value->Implementation->IsOwnedComponent(Component))
			{
				return FoliageIt->Key;
			}
		}

		return nullptr;
	}

	static UFoliageType* FindFoliageInfoFor(UActorComponent* Component)
	{
		if (UHierarchicalInstancedStaticMeshComponent* Comp = Cast<UHierarchicalInstancedStaticMeshComponent>(Component))
		{
			return FindFoliageInfoFor(Comp);
		}
		return nullptr;
	}

	static void RebuildChangedFoliageComponents(AInstancedFoliageActor* FoliageActor, const FPropertySelectionMap& SelectionMap, bool bWasRecreated)
	{
		const FRestorableObjectSelection FoliageSelection = SelectionMap.GetObjectSelection(FoliageActor);
		const FAddedAndRemovedComponentInfo* RecreatedComponentInfo = FoliageSelection.GetComponentSelection();
		
		TInlineComponentArray<UHierarchicalInstancedStaticMeshComponent*> Components(FoliageActor);
		for (UHierarchicalInstancedStaticMeshComponent* Comp : Components)
		{
			const FRestorableObjectSelection ObjectSelection = SelectionMap.GetObjectSelection(Comp);

			const FPropertySelection* CompPropertySelection = ObjectSelection.GetPropertySelection();
			const bool bHasChangedProperties = CompPropertySelection && !CompPropertySelection->IsEmpty();
			const bool bComponentWasRecreated = RecreatedComponentInfo && Algo::FindByPredicate(RecreatedComponentInfo->SnapshotComponentsToAdd, [Comp](TWeakObjectPtr<UActorComponent> SnapshotComp)
			{
				return SnapshotComp->GetFName().IsEqual(Comp->GetFName());
			}) != nullptr;
			
			if (bWasRecreated || bHasChangedProperties || bComponentWasRecreated)
			{
				// Getting foliage to update its visual state is quite involved and this is the easiest way...
				// Feel free to adjust if you come up with an easier one
				TArray<FTransform> InstanceTransforms;
				InstanceTransforms.SetNumUninitialized(Comp->GetInstanceCount());
				for (int32 i = 0; i < Comp->GetInstanceCount(); ++i)
				{
					Comp->GetInstanceTransform(i, InstanceTransforms[i]);
				}
				Comp->ClearInstances();
				Comp->AddInstances(InstanceTransforms, false);
			}
		}
	}
	
	static void UpdateFoliageUI()
	{
#if WITH_EDITOR
		IFoliageEditModule& FoliageEditModule = FModuleManager::Get().GetModuleChecked<IFoliageEditModule>("FoliageEdit");
		FoliageEditModule.UpdateMeshList();
#endif
	}

	void FFoliageSupport::Register(ILevelSnapshotsModule& Module)
	{
		EnableRequiredFoliageProperties(Module);
	
		const TSharedRef<FFoliageSupport> FoliageSupport = MakeShared<FFoliageSupport>();
		Module.RegisterRestorabilityOverrider(FoliageSupport);
		Module.RegisterRestorationListener(FoliageSupport);
		Module.RegisterCustomObjectSerializer(AInstancedFoliageActor::StaticClass(), FoliageSupport);
		Module.RegisterGlobalActorFilter(FoliageSupport);
	}

	ISnapshotRestorabilityOverrider::ERestorabilityOverride FFoliageSupport::IsActorDesirableForCapture(const AActor* Actor)
	{
		// Foliage's not allowed by default because it is hidden from the scene outliner
		return Actor->GetClass() == AInstancedFoliageActor::StaticClass() || FFoliageHelper::IsOwnedByFoliage(Actor)
			? ERestorabilityOverride::Allow : ERestorabilityOverride::DoNotCare;
	}

	void FFoliageSupport::OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage)
	{
		AInstancedFoliageActor* FoliageActor = Cast<AInstancedFoliageActor>(EditorObject);
		check(FoliageActor);

		DataStorage.WriteObjectAnnotation(FObjectAnnotator::CreateLambda([FoliageActor](FArchive& Archive)
		{
			FInstancedFoliageActorData FoliageData;
			FoliageData.CaptureData(Archive, FoliageActor);
		}));
	}
	
	void FFoliageSupport::PostApplyToEditorObject(UObject* Object, const ICustomSnapshotSerializationData& DataStorage, const FPropertySelectionMap& SelectionMap)
	{
		AInstancedFoliageActor* FoliageActor = Cast<AInstancedFoliageActor>(Object);
		check(FoliageActor);
	
		// Track this actor for safety so we know for sure that the functions were called in the order we expected them to
		CurrentFoliageActor = FoliageActor;
		DataStorage.ReadObjectAnnotation(FObjectAnnotator::CreateLambda([this, &SelectionMap, FoliageActor](FArchive& Archive)
		{
	#if WITH_EDITOR
			// Actor might not have been modified because none of its uproperties were changed
			FoliageActor->Modify();
			// Prevents: 1. Paint 1 instance 2. Take snapshot 3. Paint many instances 4. Select all 5. Apply snapshot > Crash due to selection index out of bounds
			FoliageActor->ForEachFoliageInfo([](UFoliageType*, FFoliageInfo& Info)
			{
				Info.ClearSelection();
				return true;
			});
	#endif
			const bool bWasRecreated = SelectionMap.GetDeletedActorsToRespawn().Contains(FoliageActor);
			FInstancedFoliageActorData::LoadAndApplyTo(Archive, FoliageActor, SelectionMap, bWasRecreated);

			ActorsNeedingRefresh.Add(FoliageActor);
			RebuildChangedFoliageComponents(FoliageActor, SelectionMap, bWasRecreated);
			UpdateFoliageUI();
		}));

		// Rest is done in PostApplySnapshotToActor (need access to the property selection map)
	}

	void FFoliageSupport::PostApplySnapshot(const FPostApplySnapshotParams& Params)
	{
		// Without this step all foliage will teleport visually upon being selected by the user; its cache needs to be updated:
		// 1. Add 1 instance 2. Take snapshot 3. Paint many instances 4. Restore 5. Lasso select all > The instance jumps to a random place until you paint
		for (TWeakObjectPtr<AInstancedFoliageActor> FoliageActor : ActorsNeedingRefresh)
		{
			if (!FoliageActor.IsValid())
			{
				return;
			}
			
			FoliageActor->ForEachFoliageInfo([](UFoliageType*, FFoliageInfo& Info)
			{
				constexpr bool bAsync = true; 
				constexpr bool bForce = true; // Must always run
				Info.Refresh(bAsync, bForce);
				return true;
			});
		}
		ActorsNeedingRefresh.Empty();
	}

	void FFoliageSupport::PostApplySnapshotToActor(const FApplySnapshotToActorParams& Params)
	{
		// This is the order of operations up until now
		// 1. OnTakeSnapshot > Set initial data
		// 2. User requests apply to world
		// 3. FindOrRecreateSubobjectInEditorWorld allocates the UFoliageType subobjects
		// 4. PreRemoveComponent
		// 5. PostApplySnapshotProperties loads the data that OnTakeSnapshot saved
	
		if (AInstancedFoliageActor* FoliageActor = Cast<AInstancedFoliageActor>(Params.Actor))
		{
			checkf(FoliageActor == CurrentFoliageActor.Get(), TEXT("PostApplySnapshotToActor was not directly followed by PostApplySnapshotProperties. Investigate."));
			CurrentFoliageActor.Reset();
		}
	}

	void FFoliageSupport::PreApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params)
	{
		// RF_Transactional is temporarily required because Level Snapshots changes components differently than Foliage originally designed.
		// Needed for correct undo / redo 
		if (UHierarchicalInstancedStaticMeshComponent* Comp = Cast<UHierarchicalInstancedStaticMeshComponent>(Params.Object); Comp && CurrentFoliageActor.IsValid() && Comp->IsIn(CurrentFoliageActor.Get()))
		{
			Comp->SetFlags(RF_Transactional);
		}
	}

	void FFoliageSupport::PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params)
	{
		if (UHierarchicalInstancedStaticMeshComponent* Comp = Cast<UHierarchicalInstancedStaticMeshComponent>(Params.Object); Comp && CurrentFoliageActor.IsValid() && Comp->IsIn(CurrentFoliageActor.Get()))
		{
			Comp->ClearFlags(RF_Transactional);
		}
	}

	void FFoliageSupport::PreRecreateActor(UWorld* World, TSubclassOf<AActor> ActorClass, FActorSpawnParameters& InOutSpawnParameters)
	{
		if (ActorClass->IsChildOf(AInstancedFoliageActor::StaticClass()))
		{
			InOutSpawnParameters.bCreateActorPackage = true;

			// See AInstancedFoliageActor::GetDefault
			ULevel* CurrentLevel = World->GetCurrentLevel();
			if (CurrentLevel && !CurrentLevel->GetWorld()->GetSubsystem<UActorPartitionSubsystem>()->IsLevelPartition())
			{
				InOutSpawnParameters.ObjectFlags |= RF_Transient;
			}
		}
	}

	void FFoliageSupport::PostRecreateActor(AActor* RecreatedActor)
	{
		if (AInstancedFoliageActor* FoliageActor = Cast<AInstancedFoliageActor>(RecreatedActor))
		{
			RecreatedActor->GetLevel()->InstancedFoliageActor = FoliageActor;
		}
	}

	void FFoliageSupport::PreRemoveActor(const FPreRemoveActorParams& Params)
	{
		if (AInstancedFoliageActor* FoliageActor = Cast<AInstancedFoliageActor>(Params.ActorToRemove))
		{
			FoliageActor->GetLevel()->InstancedFoliageActor.Reset();
		}
	}

	void FFoliageSupport::PreRemoveComponent(UActorComponent* ComponentToRemove)
	{
		if (UFoliageType* FoliagType = FindFoliageInfoFor(ComponentToRemove))
		{
			AInstancedFoliageActor* FoliageActor = Cast<AInstancedFoliageActor>(ComponentToRemove->GetOwner());
			FoliageActor->RemoveFoliageType(&FoliagType, 1);
		}
	}

	static IActorSnapshotFilter::FFilterResultData EvaluateSavedData(const FWorldSnapshotData& WorldData)
	{
		const int32 SavedVersion = WorldData.SnapshotVersionInfo.GetSnapshotCustomVersion();
		const bool bHasDataFromBefore5dot2 = SavedVersion < FSnapshotCustomVersion::FoliageTypesUnreadable;
		const bool bHasBrokenData = bHasDataFromBefore5dot2 && FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor <= SavedVersion;
		
		if (bHasBrokenData)
		{
			return IActorSnapshotFilter::FFilterResultData{
				IActorSnapshotFilter::EFilterResult::Disallow,
				LOCTEXT("CannotModify.5dot1", "Snapshot was captured in 5.1. This data is unusable.")
			};
		}
		
		if (bHasDataFromBefore5dot2 && !CVarAllowFoliageDataPre5dot1.GetValueOnGameThread())
		{
			return IActorSnapshotFilter::FFilterResultData{
				IActorSnapshotFilter::EFilterResult::Disallow,
				LOCTEXT("CannotModify.UseConsoleVariable", "Use \"LevelSnapshots.AllowFoliageDataBefore5dot1 true\" to enable restoring foliage data from before 5.1 (may crash).")
			};
		}

		return IActorSnapshotFilter::FFilterResultData{ IActorSnapshotFilter::EFilterResult::Allow };
	}

	IActorSnapshotFilter::FFilterResultData FFoliageSupport::CanModifyMatchedActor(const FCanModifyMatchedActorParams& Params)
	{
		return Params.MatchedEditorWorldActor.IsA<AInstancedFoliageActor>()
			? EvaluateSavedData(Params.WorldData)
			: EFilterResult::DoNotCare;
	}

	IActorSnapshotFilter::FFilterResultData FFoliageSupport::CanRecreateMissingActor(const FCanRecreateActorParams& Params)
	{
		return Params.Class && Params.Class->IsChildOf(AInstancedFoliageActor::StaticClass())
			? EvaluateSavedData(Params.WorldData)
			: EFilterResult::DoNotCare;
	}
}
#undef LOCTEXT_NAMESPACE