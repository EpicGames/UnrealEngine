// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILevelSnapshotsModule.h"
#include "Interfaces/ISnapshotRestorabilityOverrider.h"
#include "Interfaces/IRestorationListener.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AInstancedFoliageActor;
class UObject;

namespace UE::LevelSnapshots::Foliage::Private
{
	class FFoliageSupport
		:
		public ISnapshotRestorabilityOverrider,
		public ICustomObjectSnapshotSerializer,
		public IRestorationListener,
		public IActorSnapshotFilter // For not allowing SkipSubobjectReferencesWhenSerializingArchetypesRefactor < version < FoliageTypesUnreadable
	{
		/** Just used to verify snapshots is in a valid state */
		TWeakObjectPtr<AInstancedFoliageActor> CurrentFoliageActor;

		/** Tracks modified foliage actors since they must be refreshed AFTER the snapshot has serialized all properties (in PostApplySnapshot). */
		TArray<TWeakObjectPtr<AInstancedFoliageActor>> ActorsNeedingRefresh;
		
	public:

		using ICustomObjectSnapshotSerializer::PreApplyToSnapshotObject; 
		
		static void Register(ILevelSnapshotsModule& Module);

		//~ Begin ISnapshotRestorabilityOverrider Interface
		virtual ERestorabilityOverride IsActorDesirableForCapture(const AActor* Actor) override;
		//~ End ISnapshotRestorabilityOverrider Interface
		
		//~ Begin ICustomObjectSnapshotSerializer Interface
		virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override;
		virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override { checkNoEntry(); return nullptr; }
		virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override { checkNoEntry(); return nullptr; }
		virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override { checkNoEntry(); return nullptr; }
		virtual void PostApplyToEditorObject(UObject* Object, const ICustomSnapshotSerializationData& DataStorage, const FPropertySelectionMap& SelectionMap) override;
		//~ End ICustomObjectSnapshotSerializer Interface
		
		//~ Begin IRestorationListener Interface
		virtual void PostApplySnapshot(const FPostApplySnapshotParams& Params) override;
		virtual void PostApplySnapshotToActor(const FApplySnapshotToActorParams& Params) override;
		virtual void PreApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) override;
		virtual void PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) override;
		virtual void PreRecreateActor(UWorld* World, TSubclassOf<AActor> ActorClass, FActorSpawnParameters& InOutSpawnParameters) override;
		virtual void PostRecreateActor(AActor* RecreatedActor) override;
		virtual void PreRemoveActor(const FPreRemoveActorParams& Params) override;
		virtual void PreRemoveComponent(UActorComponent* ComponentToRemove) override;
		//~ End IRestorationListener Interface

		//~ Begin IActorSnapshotFilter Interface
		virtual FFilterResultData CanModifyMatchedActor(const FCanModifyMatchedActorParams& Params) override;
		virtual FFilterResultData CanRecreateMissingActor(const FCanRecreateActorParams& Params) override;
		//~ End IActorSnapshotFilter Interface
	};
}

