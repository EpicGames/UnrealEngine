// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageSupportModule.h"

#include "FoliageSupport/FoliageSupport.h"

#define LOCTEXT_NAMESPACE "FFoliageSupportModule"

void UE::LevelSnapshots::Foliage::Private::FFoliageSupportModule::StartupModule()
{
	ILevelSnapshotsModule& Module = ILevelSnapshotsModule::Get();
	UE::LevelSnapshots::Foliage::Private::FFoliageSupport::Register(Module);
}

void UE::LevelSnapshots::Foliage::Private::FFoliageSupportModule::ShutdownModule()
{}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(UE::LevelSnapshots::Foliage::Private::FFoliageSupportModule, FoliageSupport)