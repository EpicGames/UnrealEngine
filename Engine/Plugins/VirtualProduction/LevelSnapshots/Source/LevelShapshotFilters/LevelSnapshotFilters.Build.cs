// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LevelSnapshotFilters : ModuleRules
{
	public LevelSnapshotFilters(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] 
			{
				Path.Combine(ModuleDirectory, "Public", "Builtin"),
			}
		);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine"
			}
			);
		
	}
}
