// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorChangedTransformFilter.h"

#include "GameFramework/Actor.h"

EFilterResult::Type UActorChangedTransformFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	const FTransform& WorldActorTransform = Params.LevelActor->GetTransform();
	const FTransform& SnapshotActorTransform = Params.SnapshotActor->GetTransform();
	
	const bool bIsLocationEqual = bIgnoreLocation || WorldActorTransform.GetLocation().Equals(SnapshotActorTransform.GetLocation());
	const bool bIsRotationEqual = bIgnoreRotation || WorldActorTransform.GetRotation().Equals(SnapshotActorTransform.GetRotation());
	const bool bIsScaleEqual = bIgnoreScale || WorldActorTransform.GetScale3D().Equals(SnapshotActorTransform.GetScale3D());

	const bool bAreTransformsEqual = bIsLocationEqual && bIsRotationEqual && bIsScaleEqual;
	const bool bWantsEqualTransforms = TransformCheckRule == ETransformReturnType::IsValidWhenTransformStayedSame;
	return (bAreTransformsEqual && bWantsEqualTransforms) || (!bAreTransformsEqual && !bWantsEqualTransforms) 
		? EFilterResult::Include : EFilterResult::Exclude; 
}
