// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/ActorDependentPropertyFilter.h"

EFilterResult::Type UActorDependentPropertyFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return ActorFilter ? ActorFilter->IsActorValid(Params) : Super::IsActorValid(Params);
}

EFilterResult::Type UActorDependentPropertyFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	const EFilterResult::Type ActorValidResult = IsActorValid(FIsActorValidParams{Params.SnapshotActor, Params.LevelActor});
	switch (ActorValidResult)
	{
	case EFilterResult::Include:
		return IncludePropertyFilter ? IncludePropertyFilter->IsPropertyValid(Params) : Super::IsPropertyValid(Params);
		
	case EFilterResult::Exclude: 
		return ExcludePropertyFilter ? ExcludePropertyFilter->IsPropertyValid(Params) : Super::IsPropertyValid(Params);

	case EFilterResult::DoNotCare:

		switch(DoNotCareHandling)
		{
		case EDoNotCareHandling::UseIncludeFilter:
			return IncludePropertyFilter ? IncludePropertyFilter->IsPropertyValid(Params) : Super::IsPropertyValid(Params);
		case EDoNotCareHandling::UseExcludeFilter: 
			return ExcludePropertyFilter ? ExcludePropertyFilter->IsPropertyValid(Params) : Super::IsPropertyValid(Params);
		case EDoNotCareHandling::UseDoNotCareFilter: 
			return DoNotCarePropertyFilter ? DoNotCarePropertyFilter->IsPropertyValid(Params) : Super::IsPropertyValid(Params);
		default: 
			checkNoEntry();
			return Super::IsPropertyValid(Params);
		}
		
	default:
		checkNoEntry();
		return Super::IsPropertyValid(Params);
	}
}
