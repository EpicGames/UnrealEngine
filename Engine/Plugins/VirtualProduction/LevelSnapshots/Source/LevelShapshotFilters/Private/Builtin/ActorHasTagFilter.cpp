// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorHasTagFilter.h"

#include "GameFramework/Actor.h"

EFilterResult::Type UActorHasTagFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	const TFunction<bool(const FName)> CheckActorTags = [this, &Params]() -> TFunction<bool(const FName)>
	{
		const auto WorldActorHasTag = [&Params](const FName TagToCheck)
		{
			return Params.LevelActor->ActorHasTag(TagToCheck);	
		};
		const auto SnapshotActorHasTag = [&Params](const FName TagToCheck)
		{
			return Params.SnapshotActor->ActorHasTag(TagToCheck);	
		};
		const auto BothActorsHaveTag = [&Params](const FName TagToCheck)
		{
			return Params.LevelActor->ActorHasTag(TagToCheck) && Params.SnapshotActor->ActorHasTag(TagToCheck);	
		};
		
		switch(ActorToCheck)
		{
		case EActorToCheck::WorldActor:
			return WorldActorHasTag;
		
		case EActorToCheck::SnapshotActor:
			return SnapshotActorHasTag;
		
		case EActorToCheck::Both:
			return BothActorsHaveTag;

		default:
			return [](const FName){ ensure(false); return false; };
		}
		
	}();
	
	const bool bCheckAllTags = TagCheckingBehavior == ETagCheckingBehavior::HasAllTags; 
	for (const FName TagToCheck : AllowedTags)
	{
		const bool bHasTag = CheckActorTags(TagToCheck);
		if ((bCheckAllTags && !bHasTag) || (!bCheckAllTags && bHasTag))
		{
			return !bCheckAllTags ? EFilterResult::Include : EFilterResult::Exclude;
		}
	}
	
	return bCheckAllTags ? EFilterResult::Include : EFilterResult::Exclude;
}


