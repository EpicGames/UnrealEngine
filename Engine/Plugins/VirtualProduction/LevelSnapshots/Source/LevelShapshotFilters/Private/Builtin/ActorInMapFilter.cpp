// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/ActorInMapFilter.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

namespace UE::LevelSnapshotsFilters::Private
{
	static bool IsActorInMap(const FSoftObjectPath& Actor, const FSoftObjectPath& MapNameToCheck)
	{
		return Actor.GetAssetPath() == MapNameToCheck.GetAssetPath();
	}

	static EFilterResult::Type IsActorAllowed(const FSoftObjectPath& Actor, const TArray<TSoftObjectPtr<UWorld>>& AllowedLevels)
	{
		for (const TSoftObjectPtr<UWorld>& AllowedLevel : AllowedLevels)
		{
			if (IsActorInMap(Actor, AllowedLevel.ToSoftObjectPath()))
			{
				return EFilterResult::Include;
			}
		}

		return EFilterResult::Exclude;
	}
};

EFilterResult::Type UActorInMapFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return UE::LevelSnapshotsFilters::Private::IsActorAllowed(Params.LevelActor, AllowedLevels);
}

EFilterResult::Type UActorInMapFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return UE::LevelSnapshotsFilters::Private::IsActorAllowed(Params.NewActor, AllowedLevels);
}

EFilterResult::Type UActorInMapFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return UE::LevelSnapshotsFilters::Private::IsActorAllowed(Params.SavedActorPath, AllowedLevels);
}


