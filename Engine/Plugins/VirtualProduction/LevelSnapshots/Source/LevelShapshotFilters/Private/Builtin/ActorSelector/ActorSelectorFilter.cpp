// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/ActorSelector/ActorSelectorFilter.h"

EFilterResult::Type UActorSelectorFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return DefaultResult;
}

EFilterResult::Type UActorSelectorFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return DefaultResult;
}

EFilterResult::Type UActorSelectorFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return DefaultResult;
}
