// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/BlueprintOnly/AndFilter.h"

EFilterResult::Type UAndFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return EvaluateAndChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsActorValid(Params);
	});
}

EFilterResult::Type UAndFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return EvaluateAndChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsPropertyValid(Params);
	});
}

EFilterResult::Type UAndFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return EvaluateAndChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsDeletedActorValid(Params);
	});
}

EFilterResult::Type UAndFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return EvaluateAndChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsAddedActorValid(Params);
	});
}

EFilterResult::Type UAndFilter::EvaluateAndChain(TFunction<EFilterResult::Type(ULevelSnapshotFilter*)> EvaluateFilterCallback) const
{
	bool bAtLeastOneFilterSaidExclude = false;
	bool bAtLeastOneFilterSaidInclude = false;
	
	ForEachChild([&bAtLeastOneFilterSaidExclude, &bAtLeastOneFilterSaidInclude, &EvaluateFilterCallback](ULevelSnapshotFilter* Child)
	{
		const EFilterResult::Type FilterResult = EvaluateFilterCallback(Child);
		
		bAtLeastOneFilterSaidExclude |= FilterResult == EFilterResult::Exclude;
		if (bAtLeastOneFilterSaidExclude)
		{
			return EShouldBreak::Break;
		}
		
		bAtLeastOneFilterSaidInclude |= FilterResult == EFilterResult::Include;
		return EShouldBreak::Continue;
	});
	
	return bAtLeastOneFilterSaidExclude ?
		EFilterResult::Exclude
		:
		bAtLeastOneFilterSaidInclude ? EFilterResult::Include : EFilterResult::DoNotCare;
}
