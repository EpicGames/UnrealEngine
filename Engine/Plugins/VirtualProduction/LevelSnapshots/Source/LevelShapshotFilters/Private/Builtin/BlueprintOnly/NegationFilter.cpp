// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/BlueprintOnly/NegationFilter.h"

ULevelSnapshotFilter* UNegationFilter::CreateChild(const TSubclassOf<ULevelSnapshotFilter>& ChildClass)
{
	InstancedChild = NewObject<ULevelSnapshotFilter>(this, ChildClass.Get());
	return InstancedChild;
}

void UNegationFilter::SetExternalChild(ULevelSnapshotFilter* NewChild)
{
	if (NewChild && NewChild->IsIn(this))
	{
		InstancedChild = NewChild;
	}
	else
	{
		Child = NewChild;
	}
}

ULevelSnapshotFilter* UNegationFilter::GetChild() const
{
	return Child ? Child : InstancedChild;
}

EFilterResult::Type UNegationFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	if (!GetChild())
	{
		return Super::IsActorValid(Params);
	}
	
	const EFilterResult::Type Result = GetChild()->IsActorValid(Params);
	return bShouldNegate ? EFilterResult::Negate(Result) : Result;
}

EFilterResult::Type UNegationFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	if (!GetChild())
	{
		return Super::IsPropertyValid(Params);
	}
	
	const EFilterResult::Type Result = GetChild()->IsPropertyValid(Params);
	return bShouldNegate ? EFilterResult::Negate(Result) : Result;
}

EFilterResult::Type UNegationFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	if (!GetChild())
	{
		return Super::IsDeletedActorValid(Params);
	}
	
	const EFilterResult::Type Result = GetChild()->IsDeletedActorValid(Params);
	return bShouldNegate ? EFilterResult::Negate(Result) : Result;
}

EFilterResult::Type UNegationFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	if (!GetChild())
	{
		return Super::IsAddedActorValid(Params);
	}
	
	const EFilterResult::Type Result = GetChild()->IsAddedActorValid(Params);
	return bShouldNegate ? EFilterResult::Negate(Result) : Result;
}
