// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/BlueprintOnly/OrFilter.h"

EFilterResult::Type UOrFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return EvaluateOrChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsActorValid(Params);	
	});
}

EFilterResult::Type UOrFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return EvaluateOrChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsPropertyValid(Params);	
	});
}

EFilterResult::Type UOrFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return EvaluateOrChain([&Params](ULevelSnapshotFilter* Child)
		{
			return Child->IsDeletedActorValid(Params);	
		});
}

EFilterResult::Type UOrFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return EvaluateOrChain([&Params](ULevelSnapshotFilter* Child)
	{
		return Child->IsAddedActorValid(Params);	
	});
}

EFilterResult::Type UOrFilter::EvaluateOrChain(TFunction<EFilterResult::Type(ULevelSnapshotFilter*)> EvaluateFilterCallback) const
{
	bool bAtLeastOneFilterSaidExclude = false;
	bool bAtLeastOneFilterSaidInclude = false;
	
	ForEachChild([&bAtLeastOneFilterSaidExclude, &bAtLeastOneFilterSaidInclude, &EvaluateFilterCallback](ULevelSnapshotFilter* Child)
	{
		const EFilterResult::Type FilterResult = EvaluateFilterCallback(Child);
		
		bAtLeastOneFilterSaidInclude |= FilterResult == EFilterResult::Include;
		if (bAtLeastOneFilterSaidInclude)
		{
			return EShouldBreak::Break;
		}
		
		bAtLeastOneFilterSaidExclude |= FilterResult == EFilterResult::Exclude;
		return EShouldBreak::Continue;
	});
	
	return bAtLeastOneFilterSaidInclude ?
		EFilterResult::Include
		:
		bAtLeastOneFilterSaidExclude ? EFilterResult::Exclude : EFilterResult::DoNotCare;
}
