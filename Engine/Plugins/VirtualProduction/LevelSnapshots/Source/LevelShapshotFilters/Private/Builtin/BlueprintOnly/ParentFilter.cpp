// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/BlueprintOnly/ParentFilter.h"

void UParentFilter::AddChild(ULevelSnapshotFilter* Filter)
{
	if (!Filter)
	{
		return;
	}
	
	if (Filter->IsIn(this))
	{
		InstancedChildren.AddUnique(Filter);
	}
	else
	{
		Children.Add(Filter);
	}
}

bool UParentFilter::RemovedChild(ULevelSnapshotFilter* Filter)
{
	const int32 ChildrenIndex = Children.Find(Filter);
	if (ChildrenIndex != INDEX_NONE)
	{
		Children.RemoveAt(ChildrenIndex);
		return true;
	}
	
	const int32 InstancedChildrenIndex = InstancedChildren.Find(Filter);
	if (ChildrenIndex != INDEX_NONE)
	{
		InstancedChildren.RemoveAt(InstancedChildrenIndex);
		return true;
	}

	return false;
}

ULevelSnapshotFilter* UParentFilter::CreateChild(const TSubclassOf<ULevelSnapshotFilter>& Class)
{
	ULevelSnapshotFilter* InstancedChild = NewObject<ULevelSnapshotFilter>(this, Class.Get());
	InstancedChildren.Add(InstancedChild);
	return InstancedChild;
}

TArray<ULevelSnapshotFilter*> UParentFilter::GetChildren() const
{
	TArray<ULevelSnapshotFilter*> Result;
	Result.Append(Children);
	Result.Append(InstancedChildren);
	return Result;
}

void UParentFilter::ForEachChild(TFunction<EShouldBreak(ULevelSnapshotFilter* Child)> Callback) const
{
	for (ULevelSnapshotFilter* Child : Children)
	{
		if (!Child)
		{
			continue;
		}
		
		if (Callback(Child) == EShouldBreak::Break)
		{
			return;
		}
	}

	for (ULevelSnapshotFilter* Child : InstancedChildren)
	{
		if (!Child)
		{
			continue;
		}

		if (Callback(Child) == EShouldBreak::Break)
		{
			return;
		}
	}
}
