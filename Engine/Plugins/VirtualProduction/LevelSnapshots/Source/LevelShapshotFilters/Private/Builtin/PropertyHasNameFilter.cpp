// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyHasNameFilter.h"

EFilterResult::Type UPropertyHasNameFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	if (GetPropertyMatchResult(Params.Property->GetName()) == EFilterResult::Include)
	{
		return EFilterResult::Include;
	}
	return bIncludeStructSubproperties && DoesAnyOwningPropertyMatch(Params)
		? EFilterResult::Include : EFilterResult::Exclude;
}

bool UPropertyHasNameFilter::DoesAnyOwningPropertyMatch(const FIsPropertyValidParams& Params) const
{
	FIsPropertyValidParams ModifiableParams = Params;
	for (int32 NameIndex = 0; NameIndex < Params.PropertyPath.Num(); ++NameIndex)
	{
		if (GetPropertyMatchResult(Params.PropertyPath[NameIndex]) == EFilterResult::Include)
		{
			return true;
		}
	}
	return false;
}

EFilterResult::Type UPropertyHasNameFilter::GetPropertyMatchResult(const FString& PropertyNName) const
{
	class FNameMatcher
	{
	public:

		FNameMatcher(ENameMatchingRule::Type MatchingRule, const TFieldPath<FProperty>& Property)
			: MatchingRule(MatchingRule)
			, PropertyName(Property->GetName())
		{}
		FNameMatcher(ENameMatchingRule::Type MatchingRule, FString PropertyName)
			: MatchingRule(MatchingRule)
			, PropertyName(MoveTemp(PropertyName))
		{}

		bool MatchesName(const FString& AllowedName) const
		{
			switch(MatchingRule)
			{
			case ENameMatchingRule::MatchesExactly:
				return MatchesExactly(AllowedName);
			case ENameMatchingRule::MatchesIgnoreCase:
				return MatchesIgnoreCase(AllowedName);
			case ENameMatchingRule::ContainsExactly:
				return ContainsExactly(AllowedName);
			case ENameMatchingRule::ContainsIgnoreCase:
				return ContainsIgnoreCase(AllowedName);
			default:
				ensure(false);
				return false;
			}
		}
	
	private:
		
		ENameMatchingRule::Type MatchingRule;
		const FString PropertyName;

		bool MatchesExactly(const FString& AllowedName) const
		{
			return PropertyName.Equals(AllowedName, ESearchCase::CaseSensitive);
		}

		bool MatchesIgnoreCase(const FString& AllowedName) const
		{
			return PropertyName.Equals(AllowedName, ESearchCase::IgnoreCase);
		}

		bool ContainsExactly(const FString& AllowedName) const
		{
			return PropertyName.Find(AllowedName, ESearchCase::CaseSensitive) != INDEX_NONE;
		}

		bool ContainsIgnoreCase(const FString& AllowedName) const
		{
			return PropertyName.Find(AllowedName, ESearchCase::IgnoreCase) != INDEX_NONE;
		}
	};
	
	const FNameMatcher NameMatcher(NameMatchingRule, PropertyNName);
	for (const FString& AllowedPropertyName : AllowedNames)
	{
		if (NameMatcher.MatchesName(AllowedPropertyName))
		{
			return EFilterResult::Include;
		}
	}
	return EFilterResult::Exclude;
}
