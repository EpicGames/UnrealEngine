// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/PropertySelector/PropertySelectorFilter.h"

EFilterResult::Type UPropertySelectorFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return DefaultResult;
}

EFilterResult::Type UPropertySelectorFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return DefaultResult;
}

EFilterResult::Type UPropertySelectorFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return DefaultResult;
}
