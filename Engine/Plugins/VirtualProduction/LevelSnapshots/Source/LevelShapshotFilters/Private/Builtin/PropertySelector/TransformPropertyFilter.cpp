// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builtin/PropertySelector/TransformPropertyFilter.h"

#include "Components/SceneComponent.h"

namespace
{
	FName Name_RelativeLocation = USceneComponent::GetRelativeLocationPropertyName();
	FName Name_RelativeRotation = USceneComponent::GetRelativeRotationPropertyName();
	FName Name_RelativeScale3D = USceneComponent::GetRelativeScale3DPropertyName();
}

EFilterResult::Type UTransformPropertyFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	const FName MemberPropertyName = Params.Property->GetFName();

	const bool bIsLocation = MemberPropertyName == Name_RelativeLocation;
	if (bIsLocation)
	{
		return Location;
	}
	
	const bool bIsRotation = MemberPropertyName == Name_RelativeRotation;
	if (bIsRotation)
	{
		return Rotation;
	}
	
	const bool bIsScale = MemberPropertyName == Name_RelativeScale3D;
	if (bIsScale)
	{
		return Scale;
	}
	
	return Super::IsPropertyValid(Params);
}
