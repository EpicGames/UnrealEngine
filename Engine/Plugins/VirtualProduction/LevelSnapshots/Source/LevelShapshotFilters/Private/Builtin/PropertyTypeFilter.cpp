// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyTypeFilter.h"

#include "UObject/TextProperty.h"

EFilterResult::Type UPropertyTypeFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	const FProperty* Property = Params.Property.Get();
	for (const EBlueprintPropertyType::Type AllowedType : AllowedTypes)
	{
		switch(AllowedType)
		{
		case EBlueprintPropertyType::Byte:
			if (CastField<FByteProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Int: 
			if (CastField<FIntProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Int64:
			if (CastField<FInt64Property>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Bool:
			if (CastField<FBoolProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Float:
			if (CastField<FFloatProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::ObjectReference:
			if (CastField<FObjectProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Name:
			if (CastField<FNameProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Interface:
			if (CastField<FInterfaceProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Struct:
			if (CastField<FStructProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::String:
			if (CastField<FStrProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Text:
			if (CastField<FTextProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::WeakObjectReference:
			if (CastField<FWeakObjectProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::SoftObjectReference:
			if (CastField<FSoftObjectProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Double:
			if (CastField<FDoubleProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Array:
			if (CastField<FArrayProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Map:
			if (CastField<FMapProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		case EBlueprintPropertyType::Set:
			if (CastField<FSetProperty>(Property))
			{
				return EFilterResult::Include;
			}
			break;
		default:
			checkNoEntry();
		}
	}

	return EFilterResult::Exclude;
}
