// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterBlueprintFunctionLibrary.h"

#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

ULevelSnapshotFilter* UFilterBlueprintFunctionLibrary::CreateFilterByClass(const TSubclassOf<ULevelSnapshotFilter>& Class, FName Name, UObject* Outer)
{
	if (!IsValid(Outer))
	{
		Outer = GetTransientPackage();
	}
	
	return NewObject<ULevelSnapshotFilter>(Outer, Class, Name);
}
