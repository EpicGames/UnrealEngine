// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotFilters.h"

EFilterResult::Type ULevelSnapshotFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotFilter::IsDeletedComponentValid(const FIsDeletedComponentValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotFilter::IsAddedComponentValid(const FIsAddedComponentValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotBlueprintFilter::IsActorValid_Implementation(const FIsActorValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotBlueprintFilter::IsPropertyValid_Implementation(const FIsPropertyValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotBlueprintFilter::IsDeletedActorValid_Implementation(const FIsDeletedActorValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotBlueprintFilter::IsAddedActorValid_Implementation(const FIsAddedActorValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotBlueprintFilter::IsDeletedComponentValid_Implementation(const FIsDeletedComponentValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}

EFilterResult::Type ULevelSnapshotBlueprintFilter::IsAddedComponentValid_Implementation(const FIsAddedComponentValidParams& Params) const
{
	return EFilterResult::DoNotCare;
}