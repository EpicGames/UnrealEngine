// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBlueprintFunctionLibrary.h"

#include "LevelSnapshotFilterParams.h"

FString UPropertyBlueprintFunctionLibrary::GetPropertyOriginPath(const TFieldPath<FProperty>& Property)
{
	return Property.ToString();
}

FString UPropertyBlueprintFunctionLibrary::GetPropertyName(const TFieldPath<FProperty>& Property)
{
	return Property->GetName();
}

AActor* UPropertyBlueprintFunctionLibrary::LoadSnapshotActor(const FIsDeletedActorValidParams& Params)
{
	return ensure(Params.HelperForDeserialization) ? Params.HelperForDeserialization(Params.SavedActorPath) : nullptr;
}

UClass* UPropertyBlueprintFunctionLibrary::GetActorClassFromDeletedParams(const FIsDeletedActorValidParams& Params)
{
	return ensure(Params.ClassGetter) ? Params.ClassGetter(Params.SavedActorPath) : nullptr;
}
