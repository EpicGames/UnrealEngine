// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorSelector/ActorSelectorFilter.h"
#include "ActorChangedTransformFilter.generated.h"

UENUM()
namespace ETransformReturnType
{
	enum Type : int
	{
		/* Return true if the snapshot and world actor have different transforms */
		IsValidWhenTransformChanged,
		/* Returns true of the snapshot and world actor have the same transform */
		IsValidWhenTransformStayedSame
	};
}

/**
 * Allows an actor depending on whether the actors' transforms have changed.
 * Use case: You want detect whether an actor has changed its transform.
 */
UCLASS(meta = (CommonSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UActorChangedTransformFilter : public UActorSelectorFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

private:

	/* Whether we allow actors that changed transform or that stayed at the same place. */
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<ETransformReturnType::Type> TransformCheckRule;

	/* If true, we do not compare the actors' locations. */
	UPROPERTY(EditAnywhere, Category = "Config")
	bool bIgnoreLocation;

	/* If true, we do not compare the actors' rotations. */
	UPROPERTY(EditAnywhere, Category = "Config")
	bool bIgnoreRotation;

	/* If true, we do not compare the actors' scales. */
	UPROPERTY(EditAnywhere, Category = "Config")
	bool bIgnoreScale;
	
};
