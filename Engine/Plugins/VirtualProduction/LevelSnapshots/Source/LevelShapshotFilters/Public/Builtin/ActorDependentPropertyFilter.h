// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "ActorDependentPropertyFilter.generated.h"

UENUM()
enum class EDoNotCareHandling
{
	/* When IsActorValid returns Include, use RunOnIncludedActorFilter. */
	UseIncludeFilter,
	/* When IsActorValid returns Exclude, use RunOnExcludedActorFilter. */
	UseExcludeFilter,
	/* When IsActorValid returns DoNotCare, use RunOnDoNotCareActorFilter. */
	UseDoNotCareFilter
};

/**
 * Implements IsActorValid and IsPropertyValid as follows:
 *	- IsActorValid returns ActorFilter->IsActorValid
 *	- IsPropertyValid runs ActorFilter->IsActorValid. Depending on its results it runs
 *		- IncludePropertyFilter
 *		- ExcludePropertyFilter
 *		- DoNotCarePropertyFilter
 *
 * Use case: You want to allow certain properties when another filters would include the actor and allow different properties when excluded.
 */
UCLASS()
class LEVELSNAPSHOTFILTERS_API UActorDependentPropertyFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface
	
	
    /* We run IsActorValid on this filter. IsPropertyValid uses one of the below filters depending on this filter. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Config")
	TObjectPtr<ULevelSnapshotFilter> ActorFilter;

    /* Used by IsPropertyValid when ActorFilter->IsActorValid returns Include */
	UPROPERTY(EditAnywhere, Instanced, Category = "Config")
	TObjectPtr<ULevelSnapshotFilter> IncludePropertyFilter;

	/* Used by IsPropertyValid when ActorFilter->IsActorValid returns Exclude */
	UPROPERTY(EditAnywhere, Instanced, Category = "Config")
	TObjectPtr<ULevelSnapshotFilter> ExcludePropertyFilter;

	/* Determines what filter IsPropertyValid is supposed to use when IsActorValid returns DoNotCare. */
	UPROPERTY(EditAnywhere, Category = "Config")
	EDoNotCareHandling DoNotCareHandling;
	
	/* Used by IsPropertyValid when ActorFilter->IsActorValid returns DoNotCare and DoNotCareHandling == UseDoNotCareFilter. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Config")
	TObjectPtr<ULevelSnapshotFilter> DoNotCarePropertyFilter;
};
