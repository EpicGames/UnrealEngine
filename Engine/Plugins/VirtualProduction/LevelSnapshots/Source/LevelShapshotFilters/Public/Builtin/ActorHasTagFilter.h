// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorSelector/ActorSelectorFilter.h"
#include "ActorHasTagFilter.generated.h"

UENUM()
namespace EActorToCheck
{
	enum Type : int
	{
		/* Checks only the tags of the world actor */
		WorldActor,
		/* Checks only the tags of the snapshot actor */
		SnapshotActor,
		/* Checks the tags of both actors. */
		Both
	};
}

UENUM()
namespace ETagCheckingBehavior
{
	enum Type : int
	{
		/* Actor must have all tags to pass */
		HasAllTags,
		/* Actor must have at least one of the tags */
		HasAnyTag
	};
}

/* Allows an actor if it has all or any of the specified tags. */
UCLASS(meta = (CommonSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UActorHasTagFilter : public UActorSelectorFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

private:
	
	/* How to match AllowedTags in each actor. */
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<ETagCheckingBehavior::Type> TagCheckingBehavior;

	/* The tags to check the actor for.  */
	UPROPERTY(EditAnywhere, Category = "Config")
	TSet<FName> AllowedTags;

	/* Which of the actors we should check the tags on. */
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<EActorToCheck::Type> ActorToCheck;
	
};
