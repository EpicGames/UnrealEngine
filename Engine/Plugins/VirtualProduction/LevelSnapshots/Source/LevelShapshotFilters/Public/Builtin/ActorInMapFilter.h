// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorSelector/ActorSelectorFilter.h"

#include "ActorInMapFilter.generated.h"

/* Allows an actor if it belongs to the specified map. */
UCLASS(meta = (CommonSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UActorInMapFilter : public UActorSelectorFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

private:
	
	/**
	 * Only actors belonging to this Level are considered.
	 * If since the snapshot was taken:
	 *  1. an actor was modified, it is only allowed if it was in one of these levels
	 *  2. an actor was added, it is only added back if it was in one of these levels
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	TArray<TSoftObjectPtr<UWorld>> AllowedLevels;
};
