// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "ActorSelectorFilter.generated.h"

/**
 * Base class for filters that only implement IsActorValid
 */
UCLASS(Abstract)
class LEVELSNAPSHOTFILTERS_API UActorSelectorFilter : public ULevelSnapshotBlueprintFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface
	
	/**
	 * What to return for IsPropertyValid, IsDeletedActorValid, and IsAddedActorValid
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<EFilterResult::Type> DefaultResult = EFilterResult::DoNotCare;
};
