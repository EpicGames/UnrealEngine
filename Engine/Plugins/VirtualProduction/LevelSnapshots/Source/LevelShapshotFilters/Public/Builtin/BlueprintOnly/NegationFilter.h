// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "Templates/SubclassOf.h"
#include "NegationFilter.generated.h"

/* Returns the results of a child filter optionally negated.
 *
 * Negation rules:
 *	- Include negated becomes Exclude
 *	- Exclude negated becomes Include
 *	- DoNotCare negated becomes DoNotCare
 */
UCLASS(meta = (InternalSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UNegationFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	/**
	 * Creates an instanced child.
	 * If you intend to save this filter, you should use this function instead of SetExternalChild;
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	ULevelSnapshotFilter* CreateChild(const TSubclassOf<ULevelSnapshotFilter>& ChildClass);

	/**
	* Creates an instanced child.
	* If you intend to save this filter, you should use CreateChild;
	*/
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	void SetExternalChild(ULevelSnapshotFilter* NewChild);

	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	ULevelSnapshotFilter* GetChild() const;
	
	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Level Snapshots")
	bool bShouldNegate = false;
	
private:
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<ULevelSnapshotFilter> Child;

	UPROPERTY(EditAnywhere, Instanced, Category = "Level Snapshots")
	TObjectPtr<ULevelSnapshotFilter> InstancedChild;
};
