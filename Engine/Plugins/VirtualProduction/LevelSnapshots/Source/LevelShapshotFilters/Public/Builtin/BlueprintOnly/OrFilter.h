// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Builtin/BlueprintOnly/ParentFilter.h"
#include "OrFilter.generated.h"

UCLASS(meta = (InternalSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UOrFilter : public UParentFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

private:

	EFilterResult::Type EvaluateOrChain(TFunction<EFilterResult::Type(ULevelSnapshotFilter*)> EvaluateFilterCallback) const;	
};
