// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "Templates/SubclassOf.h"
#include "ParentFilter.generated.h"

UCLASS(meta = (InternalSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UParentFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	/**
	 * Adds a child you already created to this filter
	 *
	 * If you intend to save your filter, add children using CreateChild.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	void AddChild(ULevelSnapshotFilter* Filter);

	/* Removes a child from this filter */
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	bool RemovedChild(ULevelSnapshotFilter* Filter);

	/**
	 * Creates a child and adds it to this filter.
	 * If you intend to save your filter, add children using this function.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	ULevelSnapshotFilter* CreateChild(const TSubclassOf<ULevelSnapshotFilter>& Class);

	/* Gets the children in this filter */
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	TArray<ULevelSnapshotFilter*> GetChildren() const;

protected:
	
	enum class EShouldBreak
	{
		Continue,
		Break
	};
	
	void ForEachChild(TFunction<EShouldBreak(ULevelSnapshotFilter* Child)> Callback) const;
	
private:

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<TObjectPtr<ULevelSnapshotFilter>> Children;

	UPROPERTY(EditAnywhere, Instanced, Category = "Level Snapshots")
	TArray<TObjectPtr<ULevelSnapshotFilter>> InstancedChildren;
};
