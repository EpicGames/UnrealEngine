// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "ConstantFilter.generated.h"

/* Filter which treats all actors the same. */
UCLASS()
class LEVELSNAPSHOTFILTERS_API UConstantFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override { return IsActorValidResult; }
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override { return IsPropertyValidResult; };
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override { return IsDeletedActorValidResult; };
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override { return IsAddedActorValidResult; } ;
	//~ End ULevelSnapshotFilter Interface

	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<EFilterResult::Type> IsActorValidResult = EFilterResult::Include;

	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<EFilterResult::Type> IsPropertyValidResult = EFilterResult::Include;
	
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<EFilterResult::Type> IsDeletedActorValidResult = EFilterResult::Include;
	
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<EFilterResult::Type> IsAddedActorValidResult = EFilterResult::Include;
};
