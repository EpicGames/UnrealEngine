// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotFilterParams.h"
#include "Templates/Function.h"
#include "UObject/Package.h"
#include "LambdaFilter.generated.h"

/**
 * Utility filter for native C++ filters.
 */
UCLASS(meta = (InternalSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API ULambdaFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	using FIsActorValid = TFunction<EFilterResult::Type(const FIsActorValidParams&)>;
	using FIsPropertyValid = TFunction<EFilterResult::Type(const FIsPropertyValidParams&)>;
	using FIsDeletedActorValid = TFunction<EFilterResult::Type(const FIsDeletedActorValidParams&)>;
	using FIsAddedActorValid = TFunction<EFilterResult::Type(const FIsAddedActorValidParams&)>;

	static ULambdaFilter* Create(FIsActorValid IsActorValid, FIsPropertyValid IsPropertyValid, FIsDeletedActorValid IsDeletedActorValid, FIsAddedActorValid IsAddedActorValid, UObject* Outer = GetTransientPackage(), FName Name = FName(), EObjectFlags ObjectFlags = RF_NoFlags)
	{
		ULambdaFilter* Result = NewObject<ULambdaFilter>(Outer, Name, ObjectFlags);
		Result->IsActorValidCallback = MoveTemp(IsActorValid);
		Result->IsPropertyValidCallback = MoveTemp(IsPropertyValid);
		Result->IsDeletedActorValidCallback = MoveTemp(IsDeletedActorValid);
		Result->IsAddedActorValidCallback = MoveTemp(IsAddedActorValid);
		return Result;
	}
	
	////~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override { return IsActorValidCallback ? IsActorValidCallback(Params) : Super::IsActorValid(Params); }
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override { return IsPropertyValidCallback ? IsPropertyValidCallback(Params) : Super::IsPropertyValid(Params);  }
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override { return IsDeletedActorValidCallback ? IsDeletedActorValidCallback(Params) : Super::IsDeletedActorValid(Params);  }
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override { return IsAddedActorValidCallback ? IsAddedActorValidCallback(Params) : Super::IsAddedActorValid(Params);  } 
	//~ End ULevelSnapshotFilter Interface

	FIsActorValid IsActorValidCallback;
	FIsPropertyValid IsPropertyValidCallback;
	FIsDeletedActorValid IsDeletedActorValidCallback;
	FIsAddedActorValid IsAddedActorValidCallback;
};
