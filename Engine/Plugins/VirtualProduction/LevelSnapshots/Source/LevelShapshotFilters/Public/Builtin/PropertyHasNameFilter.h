// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertySelector/PropertySelectorFilter.h"
#include "PropertyHasNameFilter.generated.h"

UENUM()
namespace ENameMatchingRule
{
	enum Type : int
	{
		/* The property name must match the given name exactly. */
		MatchesExactly,
		/* The name must match the given name but ignores the case */
		MatchesIgnoreCase,
		/* The name must contains the following substring (case sensitive) */
		ContainsExactly,
		/* The name must contains the following substring (case insensitive) */
		ContainsIgnoreCase
	};
}

/**
 * Allows a property when is has a certain name
 * Use case: You only want to allow properties named "MyPropertyName"
 */
UCLASS(meta = (CommonSnapshotFilter))
class LEVELSNAPSHOTFILTERS_API UPropertyHasNameFilter : public UPropertySelectorFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

private:

	bool DoesAnyOwningPropertyMatch(const FIsPropertyValidParams& Params) const;
	EFilterResult::Type GetPropertyMatchResult(const FString& PropertyNName) const;
	
	/* How to compare the property name to AllowedNames */
	UPROPERTY(EditAnywhere, Category = "Config")
	TEnumAsByte<ENameMatchingRule::Type> NameMatchingRule = ENameMatchingRule::ContainsIgnoreCase;

	/**
	 * Whether to implicitly include sub-properties of structs.
	 *
	 * Example: AllowedNames contains RelativeLocation.
	 * If bIncludeStructSubproperties == true, then the properties X, Y, and Z are included. Otherwise, you must explicitly add them to AllowedNames.
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	bool bIncludeStructSubproperties = true;

	/* The names to match the property name against. */
	UPROPERTY(EditAnywhere, Category = "Config")
	TSet<FString> AllowedNames;
	
};
