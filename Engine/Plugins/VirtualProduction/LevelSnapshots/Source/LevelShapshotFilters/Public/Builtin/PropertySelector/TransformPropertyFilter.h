// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "PropertySelectorFilter.h"

#include "TransformPropertyFilter.generated.h"

/**
 * Allows you to filter location, rotation, and scale properties on scene components.
 * Use case: You want to restore the location but not rotation of an actor.
 */
UCLASS()
class LEVELSNAPSHOTFILTERS_API UTransformPropertyFilter : public UPropertySelectorFilter
{
	GENERATED_BODY()
public:

	//~ Begin UTransformPropertyFilter Interface
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	//~ End UTransformPropertyFilter Interface

	/* Should the location property be restored? */
	UPROPERTY(EditAnywhere, Category = "Transform")
	TEnumAsByte<EFilterResult::Type> Location;

	/* Should the rotation property be restored? */
	UPROPERTY(EditAnywhere, Category = "Transform")
	TEnumAsByte<EFilterResult::Type> Rotation;

	/* Should the scale property be restored? */
	UPROPERTY(EditAnywhere, Category = "Transform")
	TEnumAsByte<EFilterResult::Type> Scale;
};
