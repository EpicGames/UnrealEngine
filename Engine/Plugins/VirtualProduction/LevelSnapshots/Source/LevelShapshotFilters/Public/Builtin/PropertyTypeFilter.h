// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertySelector/PropertySelectorFilter.h"
#include "PropertyTypeFilter.generated.h"

UENUM()
namespace EBlueprintPropertyType
{
	// Copied from EPropertyType
	enum Type : int
	{
		// Commented out unsupported types types
		// Leave comments here so it is easier to see what was removed
		
		// CPT_NONE
	    Byte,
	    //CPT_UInt16 = 2, Unsupported by Blueprints
	    //CPT_UInt32 = 3, Unsupported by Blueprints
	    //CPT_UInt64 = 4, Unsupported by Blueprints
	    //CPT_Int8 = 5, Unsupported by Blueprints
	    //CPT_Int16 = 6, Unsupported by Blueprints
	    Int,
	    Int64,
	    Bool,
	    //CPT_Bool8 = 10, Unsupported by Blueprints
	    //CPT_Bool16 = 11, Unsupported by Blueprints
	    //CPT_Bool32 = 12, Unsupported by Blueprints
	    //CPT_Bool64 = 13, Unsupported by Blueprints
	    Float,
	    ObjectReference,
	    Name,
	    //CPT_Delegate = 17, Not useful for level snapshots
	    Interface,
	    // CPT_Unused_Index_19 = 19,
	    Struct,
	    //CPT_Unused_Index_21 = 21,
	    //CPT_Unused_Index_22 = 22,
	    String,
	    Text,
	    // CPT_MulticastDelegate = 25, Not useful for level snapshots
	    WeakObjectReference,
	    // CPT_LazyObjectReference = 27, No idea what this is; probably not useful for Blueprints
	    SoftObjectReference,
	    Double,
		Array,
	    Map,
	    Set,
	    // CPT_FieldPath = 32 Not useful for Blueprints
	};
}

/**
 * Allows a property if it is of a certain type.
 *
 * Use case: You want to include only int properties.
 */
UCLASS()
class LEVELSNAPSHOTFILTERS_API UPropertyTypeFilter : public UPropertySelectorFilter
{
	GENERATED_BODY()
public:

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	//~ Begin ULevelSnapshotFilter Interface

private:

	/* The property types that you want to allow */
	UPROPERTY(EditAnywhere, Category = "Config")
	TSet<TEnumAsByte<EBlueprintPropertyType::Type>> AllowedTypes;
	
};
