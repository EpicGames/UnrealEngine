// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"
#include "FilterBlueprintFunctionLibrary.generated.h"

UCLASS()
class LEVELSNAPSHOTFILTERS_API UFilterBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	static ULevelSnapshotFilter* CreateFilterByClass(const TSubclassOf<ULevelSnapshotFilter>& Class, FName Name = NAME_None, UObject* Outer = nullptr);
	
};
