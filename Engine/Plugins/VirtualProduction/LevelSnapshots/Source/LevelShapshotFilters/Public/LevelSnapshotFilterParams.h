// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyContainerHandle.h"
#include "LevelSnapshotFilterParams.generated.h"

class AActor;
class UActorComponent;

USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FIsActorValidParams
{
	GENERATED_BODY()

	/** The actor saved in the snapshot */
	UPROPERTY(BlueprintReadWrite, Category = "Level Snapshots")
	TObjectPtr<AActor> SnapshotActor = nullptr;
	
	/** The actor equivalent to LevelActor: it exists in the world */
	UPROPERTY(BlueprintReadWrite, Category = "Level Snapshots")
	TObjectPtr<AActor> LevelActor = nullptr;

	FIsActorValidParams() = default;
	FIsActorValidParams(AActor* InSnapshotActor, AActor* InLevelActor)
		: SnapshotActor(InSnapshotActor)
		, LevelActor(InLevelActor)
	{}
};

USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FIsPropertyValidParams
{
	GENERATED_BODY()

	/** The actor saved in the snapshot */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<AActor> SnapshotActor = nullptr;
	
	/** The actor equivalent to LevelActor: it exists in the world */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<AActor> LevelActor = nullptr;

	/** For passing to FProperty::ContainerPtrToValuePtr. This is either SnapshotActor or a subobject thereof. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	FPropertyContainerHandle SnapshotPropertyContainer;
	
	/** For passing to FProperty::ContainerPtrToValuePtr. This is either LevelPropertyContainers or a subobject thereof. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	FPropertyContainerHandle LevelPropertyContainers;

	/** The property that we may want to rollback. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TFieldPath<FProperty> Property;

	/**
	 * Each elements is the name of a subobject name leading to this property. The last element is the property name.
	 * The first element is either the name of a component or a struct/subobject in the root actor.
	 *
	 * Examples:
	 *	- MyCustomComponent -> MyCustomStructPropertyName -> PropertyName
	 *  - MyCustomComponent -> MyCustomStructPropertyName
	 *	- StructPropertyNameInActor -> PropertyName
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TArray<FString> PropertyPath;

	FIsPropertyValidParams() = default;
	FIsPropertyValidParams(AActor* InSnapshotActor, AActor* InLevelActor, const FPropertyContainerHandle& InSnapshotPropertyContainer, const FPropertyContainerHandle& InLevelPropertyContainers, const TFieldPath<FProperty>& InProperty, const TArray<FString>& InPropertyPath)
		: SnapshotActor(InSnapshotActor)
		, LevelActor(InLevelActor)
		, SnapshotPropertyContainer(InSnapshotPropertyContainer)
		, LevelPropertyContainers(InLevelPropertyContainers)
		, Property(InProperty)
		, PropertyPath(InPropertyPath)
	{}
};

USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FIsDeletedActorValidParams
{
	GENERATED_BODY()

	/**
	 * Holds path info for an actor that was deleted since the snapshot was taken.
	 * Contains the path to the original actor before it was deleted.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	FSoftObjectPath SavedActorPath;

	/** Helper property for deserializing SavedActorPath. */
	TFunction<AActor* (const FSoftObjectPath&)> HelperForDeserialization;

	/** Helper for getting the class of the deleted actor */
	TFunction<UClass* (const FSoftObjectPath&)> ClassGetter;

	FIsDeletedActorValidParams() = default;
	FIsDeletedActorValidParams(
		FSoftObjectPath SavedActorPath, 
		TFunction<AActor*(const FSoftObjectPath&)> HelperForDeserialization,
		TFunction<UClass* (const FSoftObjectPath&)> ClassGetter
	)
		: SavedActorPath(SavedActorPath)
		, HelperForDeserialization(HelperForDeserialization)
		, ClassGetter(ClassGetter)
	{}
};

USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FIsAddedActorValidParams
{
	GENERATED_BODY()

	/** This actor was added to the level since the snapshot was taken. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<AActor> NewActor = nullptr;

	FIsAddedActorValidParams() = default;
	explicit FIsAddedActorValidParams(AActor* NewActor)
		: NewActor(NewActor)
	{}
};

USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FIsDeletedComponentValidParams
{
	GENERATED_BODY()

	/** This component was removed from the actor. This instance exists in a transient snapshot world; it does not exist in the editor world. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<UActorComponent> DeletedComponent = nullptr;

	/** The actor the component was removed from; This instance exists in the editor world. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<AActor> EditorActor = nullptr;

	FIsDeletedComponentValidParams() = default;
	explicit FIsDeletedComponentValidParams(UActorComponent* DeletedComponent, AActor* EditorActor)
		: DeletedComponent(DeletedComponent)
		, EditorActor(EditorActor)
	{}
};

USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FIsAddedComponentValidParams
{
	GENERATED_BODY()

	/** This component was added to the actor. This is an instance in the editor world. */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<UActorComponent> AddedComponent = nullptr;

	FIsAddedComponentValidParams() = default;
	explicit FIsAddedComponentValidParams(UActorComponent* AddedComponent)
		: AddedComponent(AddedComponent)
	{}
};