// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "LevelSnapshotFilterParams.h"
#include "LevelSnapshotFilters.generated.h"

UENUM(BlueprintType)
namespace EFilterResult
{
	enum Type : int
	{
		/* This actor / property will be included. 
		 */
		Include,
		/* This actor / property will be excluded.
		 */
		Exclude,

		/* The filter does not care what happens to this actor / property.
		 * Another filter will decide. If all filters don't care, actor / property is included.
		 *
		 * Use this for filters that only implement one function: IsActorValid or IsPropertyValid.
		 */
		DoNotCare
	};
}

namespace EFilterResult
{
	static bool ShouldInclude(EFilterResult::Type Type)
	{
		return Type == Include;
	}

	static bool CanInclude(EFilterResult::Type Type)
	{
		return Type == Include || Type == DoNotCare;
	}

	static bool CanNegate(EFilterResult::Type Type)
	{
		return Type != DoNotCare;
	}

	static EFilterResult::Type Negate(EFilterResult::Type Type)
	{
		if (CanNegate(Type))
		{
			return Type == Include ? Exclude : Include;
		}
		return Type;
	}
}

/**
 * Base-class for filtering a level snapshot.
 * Native C++ classes should inherit directly from this class.
 */
UCLASS(Abstract, BlueprintType, EditInlineNew)
class LEVELSNAPSHOTFILTERS_API ULevelSnapshotFilter : public UObject
{
	GENERATED_BODY()
public:

	/* @return Whether this actor should be considered when applying a snapshot to the world. */
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const;
	
	/* @return Whether the property in LevelActor should be overriden with the value in SnapshotActor.*/
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const;

	/**
	 * This is called when an actor was removed from the world since the snapshot had been taken.
	 * @return Whether to track the removed actor
	 */
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const;

	/**
	 * This is called when an actor was added to the world since the snapshot had been taken. 
	 * @return Whether to track the added actor
	 */
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const;
	
	/**
	 * This is called when a component was removed from an actor since the snapshot had been taken. 
	 * @return Whether to track the removed component
	 */
	virtual EFilterResult::Type IsDeletedComponentValid(const FIsDeletedComponentValidParams& Params) const;
	
	/**
	 * This is called when a component was added to the world since the snapshot had been taken. 
	 * @return Whether to track the added component
	 */
	virtual EFilterResult::Type IsAddedComponentValid(const FIsAddedComponentValidParams& Params) const;
};

/**
 * Base-class for filtering a level snapshot in Blueprints.
 */
UCLASS(Abstract, Blueprintable)
class LEVELSNAPSHOTFILTERS_API ULevelSnapshotBlueprintFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()

public:

	/**
	 * @return Whether the actor should be considered for the level snapshot.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Level Snapshots")
	EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;

	/**
	 * @return Whether this property should be considered for rolling back to the version in the snapshot. 
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Level Snapshots")
	EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;

	/**
	* This is called when an actor was removed from the world since the snapshot had been taken.
	* @return Whether to track the removed actor
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Level Snapshots")
	EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;

	/**
	* This is called when an actor was added to the world since the snapshot had been taken. 
	* @return Whether to track the added actor
	*/
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Level Snapshots")
	EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;

	/**
	 * This is called when a component was removed from an actor since the snapshot had been taken. 
	 * @return Whether to track the removed component
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Level Snapshots")
	EFilterResult::Type IsDeletedComponentValid(const FIsDeletedComponentValidParams& Params) const;

	/**
	 * This is called when a component was added to the world since the snapshot had been taken. 
	 * @return Whether to track the added component
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Level Snapshots")
	EFilterResult::Type IsAddedComponentValid(const FIsAddedComponentValidParams& Params) const;
};