// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilterParams.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PropertyBlueprintFunctionLibrary.generated.h"

UCLASS()
class LEVELSNAPSHOTFILTERS_API UPropertyBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/* Returns a path containing information which class declare the property.*/
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	static FString GetPropertyOriginPath(const TFieldPath<FProperty>& Property);
	
	/* Gets only the property name of a property. */
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	static FString GetPropertyName(const TFieldPath<FProperty>& Property);

	/* Loads the actor identified by Params. You can use this for advanced filter queries.*/
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	static AActor* LoadSnapshotActor(const FIsDeletedActorValidParams& Params);

	/* Finds the actor class from the given FIsDeletedActorValidParams. Use this for advanced filter queries */
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	static UClass* GetActorClassFromDeletedParams(const FIsDeletedActorValidParams& Params);
	
};
