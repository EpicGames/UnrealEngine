// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyContainerHandle.generated.h"

/* Blueprint wrapper for FProperty containers */
USTRUCT(BlueprintType)
struct LEVELSNAPSHOTFILTERS_API FPropertyContainerHandle
{
	GENERATED_BODY()
public:

	/* Pass this to FProperty::ContainerPtrToValuePtr(PropertyContainer). Never null. */
	void* PropertyContainerPtr;

	FPropertyContainerHandle() = default;
	FPropertyContainerHandle(void* InPropertyContainerPtr)
        : PropertyContainerPtr(InPropertyContainerPtr)
	{}
	
};


