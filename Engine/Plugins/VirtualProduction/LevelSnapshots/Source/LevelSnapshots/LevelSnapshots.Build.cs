// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LevelSnapshots : ModuleRules
{
    public LevelSnapshots(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicIncludePaths.AddRange(
	        new string[] 
	        {
		        Path.Combine(ModuleDirectory, "Public", "Data"),
		        Path.Combine(ModuleDirectory, "Public", "Restorability"),
		        Path.Combine(ModuleDirectory, "Public", "Settings")
	        }
        );
        PrivateIncludePaths.AddRange(
	        new string[] 
	        {
		        Path.Combine(ModuleDirectory, "Private", "Archive"),
		        Path.Combine(ModuleDirectory, "Private", "Data")
	        }
        );
        
        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "TraceLog"
            }
            );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "Core",
                "CoreUObject",
                "Engine",
                "Landscape",
                "LevelSnapshotFilters"
            }
            );

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange( new string[]
			{
				"UnrealEd", // This is needed for undo / redo system in editor
				"Settings",	// For registering settings
				"Projects"	// For determining enabled plugins
			});
		}
	}
}
