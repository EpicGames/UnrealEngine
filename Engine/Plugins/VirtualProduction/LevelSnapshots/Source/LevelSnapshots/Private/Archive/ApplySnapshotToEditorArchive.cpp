// Copyright Epic Games, Inc. All Rights Reserved.

#include "Archive/ApplySnapshotToEditorArchive.h"

#include "Archive/ClassDefaults/ApplyClassDefaulDataArchive.h"
#include "Filtering/PropertySelection.h"
#include "Data/Util/ClassDataUtil.h"
#include "Data/Util/ObjectDependencyUtil.h"
#include "Data/WorldSnapshotData.h"
#include "LevelSnapshotsLog.h"
#include "Restoration/Scope/ApplySnapshotPropertiesScope.h"

#include "Serialization/ObjectReader.h"
#include "UObject/UnrealType.h"

void UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::ApplyToExistingEditorWorldObject(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	FSnapshotDataCache& Cache,
	UObject* InOriginalObject,
	UObject* InDeserializedVersion,
	const FPropertySelectionMap& InSelectionMapForResolvingSubobjects,
	TOptional<FClassDataIndex> ClassIndex)
{
	const FPropertySelection* Selection = InSelectionMapForResolvingSubobjects.GetObjectSelection(InOriginalObject).GetPropertySelection();
	if (!Selection || Selection->IsEmpty())
	{
		return;
	}

	UE_LOGF(LogLevelSnapshots, Verbose, "Applying to existing object %ls (class %ls)", *InOriginalObject->GetPathName(), *InOriginalObject->GetClass()->GetPathName());
	const FApplySnapshotPropertiesScope NotifySnapshotListeners({ InOriginalObject, InSelectionMapForResolvingSubobjects, Selection, true });
#if WITH_EDITOR
	InOriginalObject->Modify();
	InOriginalObject->PreEditChange(nullptr);
#endif
	
	// 1. Serialize archetype first to handle the case were the archetype has changed properties since the snapshot was taken
	if (ClassIndex)
	{
		const FSubobjectArchetypeFallbackInfo ClassFallbackInfo{ InOriginalObject->GetOuter(), InOriginalObject->GetFName(), InOriginalObject->GetFlags() };
		SerializeSelectedClassDefaultsIntoSubobject(InOriginalObject, InSharedData, *ClassIndex, Cache, ClassFallbackInfo, *Selection);
	}
	
	// Step 2: Serialise  properties that were different from CDO at time of snapshotting and that are still different from CDO
	FApplySnapshotToEditorArchive ApplySavedData(InObjectData, InSharedData, InOriginalObject, InSelectionMapForResolvingSubobjects, Selection, Cache);
	InOriginalObject->Serialize(ApplySavedData);

#if WITH_EDITOR
	InOriginalObject->PostEditChange();
#endif
}

void UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithArchetype(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	FSnapshotDataCache& Cache,
	UObject* InOriginalObject,
	const FPropertySelectionMap& InSelectionMapForResolvingSubobjects)
{
	UE_LOGF(LogLevelSnapshots, Verbose, "Applying to recreated object %ls (class %ls)", *InOriginalObject->GetPathName(), *InOriginalObject->GetClass()->GetPathName());
	const FApplySnapshotPropertiesScope NotifySnapshotListeners({ InOriginalObject, InSelectionMapForResolvingSubobjects, {}, true });
	
	// Apply all properties that we saved into the target actor.
	FApplySnapshotToEditorArchive ApplySavedData(InObjectData, InSharedData, InOriginalObject, InSelectionMapForResolvingSubobjects, {}, Cache);
	InOriginalObject->Serialize(ApplySavedData);
}

void UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithoutArchetype(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	FSnapshotDataCache& Cache,
	UObject* InOriginalObject,
	const FPropertySelectionMap& InSelectionMapForResolvingSubobjects,
	FClassDataIndex ClassIndex)
{
	// 1. Serialize archetype first to handle the case were the archetype has changed properties since the snapshot was taken
	const FSubobjectArchetypeFallbackInfo ClassFallbackInfo{ InOriginalObject->GetOuter(), InOriginalObject->GetFName(), InOriginalObject->GetFlags() };
	SerializeClassDefaultsIntoSubobject(InOriginalObject, InSharedData, ClassIndex, Cache, ClassFallbackInfo);
	
	// Step 2: Apply the data that was different from CDO at time of snapshotting
	FApplySnapshotToEditorArchive ApplySavedData(InObjectData, InSharedData, InOriginalObject, InSelectionMapForResolvingSubobjects, {}, Cache);
	InOriginalObject->Serialize(ApplySavedData);
}

bool UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::ShouldSkipProperty(const FProperty* InProperty) const
{
	SCOPED_SNAPSHOT_CORE_TRACE(ShouldSkipProperty);
	
	bool bShouldSkipProperty = Super::ShouldSkipProperty(InProperty);
	if (!bShouldSkipProperty && !ShouldSerializeAllProperties())
	{
		const bool bIsAllowed = SelectionSet.GetValue()->ShouldSerializeProperty(GetSerializedPropertyChain(), InProperty);  
		bShouldSkipProperty = !bIsAllowed;
	}
	
	return bShouldSkipProperty;
}

UObject* UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const
{
	FString LocalizationNamespace;
#if USE_STABLE_LOCALIZATION_KEYS
	LocalizationNamespace = GetLocalizationNamespace();
#endif
	return ResolveObjectDependencyForEditorWorld(GetSharedData(), Cache, ObjectIndex, LocalizationNamespace, SelectionMapForResolvingSubobjects);
}

UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::FApplySnapshotToEditorArchive(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	UObject* InOriginalObject,
	const FPropertySelectionMap& InSelectionMapForResolvingSubobjects,
	TOptional<const FPropertySelection*> InSelectionSet,
	FSnapshotDataCache& Cache)
        : Super(InObjectData, InSharedData, true, InOriginalObject)
		, SelectionMapForResolvingSubobjects(InSelectionMapForResolvingSubobjects)
		, SelectionSet(InSelectionSet)
		, Cache(Cache)
{
#if UE_BUILD_DEBUG
	UE_LOGF(LogLevelSnapshots, VeryVerbose, "FApplySnapshotToEditorArchive: %ls (%ls)", *InOriginalObject->GetPathName(), *InOriginalObject->GetClass()->GetPathName());
#endif
}

bool UE::LevelSnapshots::Private::FApplySnapshotToEditorArchive::ShouldSerializeAllProperties() const
{
	return !SelectionSet.IsSet();
}
