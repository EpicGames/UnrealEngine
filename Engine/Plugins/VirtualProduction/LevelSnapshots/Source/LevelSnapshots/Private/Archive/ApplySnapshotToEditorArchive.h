// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filtering/PropertySelection.h"
#include "Filtering/PropertySelectionMap.h"
#include "SnapshotArchive.h"
#include "SnapshotUtilTypes.h"

struct FSnapshotDataCache;

namespace UE::LevelSnapshots::Private
{
	/* For writing data into an editor object. */
	class FApplySnapshotToEditorArchive : public FSnapshotArchive
	{
		using Super = FSnapshotArchive;
	public:

		/**
		 * Serializes selected snapshot data into an object that existed in the world.
		 * The property selection map contains a set of properties that are supposed to be serialized.
		 */
		static void ApplyToExistingEditorWorldObject(
			FObjectSnapshotData& InObjectData,
			FWorldSnapshotData& InSharedData,
			FSnapshotDataCache& Cache,
			UObject* InOriginalObject,
			UObject* InDeserializedVersion,
			const FPropertySelectionMap& InSelectionMapForResolvingSubobjects,
			TOptional<FClassDataIndex> ClassIndex = {}
			);

		/**
		 * Serializes all snapshot data into an object that was recreated and was templated using the snapshot saved archetype / CDO.
		 * Because the object was recreated from an archetype, we can skip applying CDO's data into the object.
		 */
		static void ApplyToEditorWorldObjectRecreatedWithArchetype(
			FObjectSnapshotData& InObjectData,
			FWorldSnapshotData& InSharedData,
			FSnapshotDataCache& Cache,
			UObject* InOriginalObject,
			const FPropertySelectionMap& InSelectionMapForResolvingSubobjects
			);

		/**
		 * Serializes all snapshot data into an object that was recreated and was templated using the snapshot saved archetype / CDO.
		 * Because the object was not recreated from any archetype, we must also apply the saved CDO's data to the object.
		 */
		static void ApplyToEditorWorldObjectRecreatedWithoutArchetype(
			FObjectSnapshotData& InObjectData,
			FWorldSnapshotData& InSharedData,
			FSnapshotDataCache& Cache,
			UObject* InOriginalObject,
			const FPropertySelectionMap& InSelectionMapForResolvingSubobjects,
			FClassDataIndex ClassIndex
			);
	
		//~ Begin FTakeSnapshotArchiveV2 Interface
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
		//~ End FTakeSnapshotArchiveV2 Interface

	protected:

		//~ Begin FSnapshotArchive Interface
		virtual UObject* ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const override;
		//~ End FSnapshotArchive Interface

	private:

		FApplySnapshotToEditorArchive(
			FObjectSnapshotData& InObjectData,
			FWorldSnapshotData& InSharedData,
			UObject* InOriginalObject,
			const FPropertySelectionMap& InSelectionMapForResolvingSubobjects,
			TOptional<const FPropertySelection*> InSelectionSet,
			FSnapshotDataCache& Cache
			);

		bool ShouldSerializeAllProperties() const;

		/** Needed so subobjects can be fully reconstructed */
		const FPropertySelectionMap& SelectionMapForResolvingSubobjects; 
	
		/** Immutable list of properties we are supposed to serialise. Serialize all properties if unset. */
		TOptional<const FPropertySelection*> SelectionSet;

		FSnapshotDataCache& Cache;
	};
}

