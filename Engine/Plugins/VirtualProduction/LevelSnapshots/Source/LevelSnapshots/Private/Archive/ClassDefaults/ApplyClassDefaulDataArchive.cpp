// Copyright Epic Games, Inc. All Rights Reserved.

#include "Archive/ClassDefaults/ApplyClassDefaulDataArchive.h"

#include "Data/Util/ObjectDependencyUtil.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "ObjectSnapshotData.h"
#include "SelectionSet.h"
#include "WorldSnapshotData.h"

void UE::LevelSnapshots::Private::FApplyClassDefaulDataArchive::SerializeClassDefaultObject(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InCDO)
{
	check(!FLevelSnapshotsModule::GetInternalModuleInstance().ShouldSkipClassDefaultSerialization(InCDO->GetClass()));
	
	FApplyClassDefaulDataArchive Archive(InObjectData, InSharedData, InCDO, ESerialisationMode::RestoringCDO);
	InCDO->Serialize(Archive);
}

void UE::LevelSnapshots::Private::FApplyClassDefaulDataArchive::RestoreChangedClassDefaults(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InObjectToRestore)
{
	FApplyClassDefaulDataArchive Archive(InObjectData, InSharedData, InObjectToRestore, ESerialisationMode::RestoringChangedDefaults);
	InObjectToRestore->Serialize(Archive);
}

void UE::LevelSnapshots::Private::FApplyClassDefaulDataArchive::RestoreSelectedChangedClassDefaults(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InObjectToRestore, const FPropertySelection& PropertiesToRestore)
{
	FApplyClassDefaulDataArchive Archive(InObjectData, InSharedData, InObjectToRestore, ESerialisationMode::RestoringChangedDefaults, &PropertiesToRestore);
	InObjectToRestore->Serialize(Archive);
}

bool UE::LevelSnapshots::Private::FApplyClassDefaulDataArchive::ShouldSkipProperty(const FProperty* InProperty) const
{
	bool bShouldSkipProperty = Super::ShouldSkipProperty(InProperty);
	const bool bShouldSerializeAllProperties = !SelectionSet.IsSet();
	if (!bShouldSkipProperty && !bShouldSerializeAllProperties)
	{
		const bool bIsAllowed = SelectionSet.GetValue()->ShouldSerializeProperty(GetSerializedPropertyChain(), InProperty);  
		bShouldSkipProperty = !bIsAllowed;
	}
	
	return bShouldSkipProperty;
}

UE::LevelSnapshots::Private::FApplyClassDefaulDataArchive::FApplyClassDefaulDataArchive(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	UObject* InObjectToRestore,
	ESerialisationMode InSerialisationMode,
	TOptional<const FPropertySelection*> InSelectionSet)
	: Super(InObjectData, InSharedData, true, InObjectToRestore)
	, SelectionSet(InSelectionSet)
{
	// Only overwrite transient properties when actually serializing into a snapshot CDO
	ArSerializingDefaults = InSerialisationMode == ESerialisationMode::RestoringCDO;

#if UE_BUILD_DEBUG
	UE_LOGF(LogLevelSnapshots, VeryVerbose, "FApplyClassDefaulDataArchive: %ls (%ls)", *InObjectToRestore->GetPathName(), *InObjectToRestore->GetClass()->GetPathName());
#endif
}

UObject* UE::LevelSnapshots::Private::FApplyClassDefaulDataArchive::ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const
{
	return ResolveObjectDependencyForClassDefaultObject(GetSharedData(), ObjectIndex, CurrentValue);
}
