// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorldSnapshotData.h"
#include "Archive/ClassDefaults/BaseClassDefaultArchive.h"

class UObject;
struct FObjectSnapshotData;
struct FPropertySelection;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/* Used when we're taking a snapshot of the world. */
	class FApplyClassDefaulDataArchive : public FBaseClassDefaultArchive
	{
		using Super = FBaseClassDefaultArchive;
	public:

		/* Archive will be used for reconstructing a CDO */
		static void SerializeClassDefaultObject(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InCDO);
		/* Archive will be used to to apply any non-object properties before the object receives its saved snapshot values. This is to handle when a CDO has changed its default values since a snapshot was taken. */
		static void RestoreChangedClassDefaults(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* ObjectToRestore);
		static void RestoreSelectedChangedClassDefaults(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InObjectToRestore, const FPropertySelection& PropertiesToRestore);

		//~ Begin FArchive Interface
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
		//~ End FArchive Interface
		
	protected:
	
		enum class ESerialisationMode
		{
			RestoringCDO,
			RestoringChangedDefaults
		};

		/** Immutable list of properties we are supposed to serialise. Serialize all properties if unset. */
		TOptional<const FPropertySelection*> SelectionSet;

		FApplyClassDefaulDataArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InObjectToRestore, ESerialisationMode InSerialisationMode, TOptional<const FPropertySelection*> InSelectionSet = {});

		//~ Begin FSnapshotArchive Interface
		virtual UObject* ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const override;
		//~ End FSnapshotArchive Interface
	};
}