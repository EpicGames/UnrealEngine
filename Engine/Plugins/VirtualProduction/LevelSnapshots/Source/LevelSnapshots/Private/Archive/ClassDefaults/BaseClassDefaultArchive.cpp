// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseClassDefaultArchive.h"

#include "WorldSnapshotData.h"

UE::LevelSnapshots::Private::FBaseClassDefaultArchive::FBaseClassDefaultArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, bool bIsLoading, UObject* InObjectToRestore)
	:
	Super(InObjectData, InSharedData, bIsLoading, InObjectToRestore)
{
	// Description of CPF_Transient: "Property is transient: shouldn't be saved or loaded, except for Blueprint CDOs."
	ExcludedPropertyFlags = CPF_BlueprintAssignable | CPF_Deprecated
		// Do not save any instanced references when serialising CDOs
		| CPF_ContainsInstancedReference | CPF_InstancedReference | CPF_PersistentInstance;
	
	// Otherwise we are not allowed to serialize transient properties
	ArSerializingDefaults = true;
}