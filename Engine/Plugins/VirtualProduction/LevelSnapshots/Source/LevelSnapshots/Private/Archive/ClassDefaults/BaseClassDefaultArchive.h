// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Archive/SnapshotArchive.h"

class UObject;
struct FObjectSnapshotData;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/** Shared logic for serializing class defaults. */
	class FBaseClassDefaultArchive : public FSnapshotArchive
	{
		using Super = FSnapshotArchive;
	protected:
	
		FBaseClassDefaultArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, bool bIsLoading, UObject* InObjectToRestore);

		// Class archetypes do not need fixing up... they only reference assets and subobjects. Subobjects are handled entirely by the the archetype itself.
		virtual void FixUpReference(UObject*& Value) const override {}
	};
}