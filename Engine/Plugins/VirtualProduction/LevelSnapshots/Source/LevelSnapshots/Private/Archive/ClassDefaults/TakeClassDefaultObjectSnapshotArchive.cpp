// Copyright Epic Games, Inc. All Rights Reserved.

#include "Archive/ClassDefaults/TakeClassDefaultObjectSnapshotArchive.h"

#include "LevelSnapshotsLog.h"
#include "ObjectSnapshotData.h"
#include "WorldSnapshotData.h"

namespace UE::LevelSnapshots::Private
{
	class FCollectSubobjectPropertyArchive : public FArchiveUObject
	{
		using Super = FArchiveUObject;
		const UObject* SerializedObject;
	public:

		TSet<const FProperty*> PropertiesContainingSubobjects;

		FCollectSubobjectPropertyArchive(const UObject* SerializedObject)
			: SerializedObject(SerializedObject)
		{
			Super::SetIsSaving(true);
			Super::SetIsPersistent(true);
			// Intentionally not setting ArIsObjectReferenceCollector because we want to find them "naturally" to determine referencing properties
		}
		
		virtual FArchive& operator<<(UObject*& Object) override
		{
			if (IsSkippedObject(Object) && GetSerializedProperty())
			{
				PropertiesContainingSubobjects.Add(GetSerializedProperty());
			}
			return *this;
		}

		bool IsSkippedObject(const UObject* Object)
		{
			if (Object)
			{
				const bool bIsClassDefault = Object->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
				const bool bIsPointingToDefaultSubobject = Object->HasAnyFlags(RF_DefaultSubObject);
				const bool bIsPointingToSelf = Object == SerializedObject;
				const bool bIsPointingToSubobject = Object->IsIn(SerializedObject);
				const bool bShouldSkip = bIsClassDefault || bIsPointingToDefaultSubobject || bIsPointingToSelf || bIsPointingToSubobject;

				// We don't ever want to interfere with the default subobjects that the class archetype assigns so the reference is marked so we know it is supposed to be skipped when restoring 
				return bShouldSkip;
			}
			return false;
		}
	};
}

void UE::LevelSnapshots::Private::FTakeClassDefaultObjectSnapshotArchive::SaveClassDefaultObject(FClassSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* SerializedObject)
{
	// We don't want to interfere with subobjects created by archetypes / CDOs because restoring those is very involved so we skip properties that contain them
	FCollectSubobjectPropertyArchive CollectSubobjectProperties(SerializedObject);
	SerializedObject->Serialize(CollectSubobjectProperties);
	
	FTakeClassDefaultObjectSnapshotArchive SaveClass(InObjectData, InSharedData, SerializedObject, MoveTemp(CollectSubobjectProperties.PropertiesContainingSubobjects));
	SerializedObject->Serialize(SaveClass);
	InObjectData.ObjectFlags = SerializedObject->GetFlags();
	InObjectData.ClassPath = SerializedObject->GetClass();
	InObjectData.ClassFlags = SerializedObject->GetClass()->GetFlags();
}

bool UE::LevelSnapshots::Private::FTakeClassDefaultObjectSnapshotArchive::ShouldSkipProperty(const FProperty* InProperty) const
{
	const bool bIsSkippedProperty = SkippedProperties.Contains(InProperty);
	UE_CLOGF(bIsSkippedProperty, LogLevelSnapshots, Verbose, "Skipping property %ls of object %ls because it references a subobject.", *InProperty->GetName(), *GetSerializedObject()->GetPathName());
	return bIsSkippedProperty || Super::ShouldSkipProperty(InProperty);
}

UE::LevelSnapshots::Private::FTakeClassDefaultObjectSnapshotArchive::FTakeClassDefaultObjectSnapshotArchive(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	UObject* InSerializedObject,
	TSet<const FProperty*> PropertiesToSkip
	)
	: Super(InObjectData, InSharedData, false, InSerializedObject)
	, SkippedProperties(MoveTemp(PropertiesToSkip))
{
#if UE_BUILD_DEBUG
	UE_LOGF(LogLevelSnapshots, VeryVerbose, "FTakeClassDefaultObjectSnapshotArchive: %ls (%ls)", *InSerializedObject->GetPathName(), *InSerializedObject->GetClass()->GetPathName());
#endif
}
