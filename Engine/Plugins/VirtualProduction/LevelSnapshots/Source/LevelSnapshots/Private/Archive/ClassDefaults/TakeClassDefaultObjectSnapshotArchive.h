// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Archive/ClassDefaults/BaseClassDefaultArchive.h"

class UObject;
struct FClassSnapshotData;
struct FObjectSnapshotData;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/* Used when we're taking a snapshot of the world. */
	class FTakeClassDefaultObjectSnapshotArchive final : public FBaseClassDefaultArchive
	{
		using Super = FBaseClassDefaultArchive;
	public:

		static void SaveClassDefaultObject(FClassSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* SerializedObject);

		//~ Begin FArchive Interface
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
		//~ End FArchive Interface

	protected:

		//~ Begin FSnapshotArchive Interface
		virtual UObject* ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const override { checkNoEntry(); return nullptr; }
		virtual void OnAddObjectDependency(int32 ObjectIndex, UObject* Object) const override {}
		//~ End FSnapshotArchive Interface
		
	private:
	
		FTakeClassDefaultObjectSnapshotArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InSerializedObject, TSet<const FProperty*> PropertiesToSkip);

		const TSet<const FProperty*> SkippedProperties;
	};
}


