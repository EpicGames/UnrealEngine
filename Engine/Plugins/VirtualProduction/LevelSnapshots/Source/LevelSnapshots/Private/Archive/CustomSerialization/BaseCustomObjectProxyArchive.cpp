// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCustomObjectProxyArchive.h"

#include "Serialization/ArchiveUObject.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::LevelSnapshots::CustomSerialization
{
	FArchive& FBaseCustomObjectProxyArchive::operator<<(FWeakObjectPtr& Obj)
	{
		return FArchiveUObject::SerializeWeakObjectPtr(*this, Obj);
	}
		
	FArchive& FBaseCustomObjectProxyArchive::operator<<(FSoftObjectPtr& Value)
	{
		if (IsLoading())
		{
			Value.ResetWeakPtr();
		}
		*this << Value.GetUniqueID();
		return *this;
	}
		
	FArchive& FBaseCustomObjectProxyArchive::operator<<(FObjectPtr& Obj)
	{
		return FArchiveUObject::SerializeObjectPtr(*this, Obj);
	}
}
