// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/NameAsStringProxyArchive.h"

namespace UE::LevelSnapshots::CustomSerialization
{
	/** Primarily used for serializing object annotation data for implementing ICustomSnapshotSerializationData. */
	class FBaseCustomObjectProxyArchive : public FNameAsStringProxyArchive
	{
	public:
		
		FBaseCustomObjectProxyArchive(FArchive& InInnerArchive)
			: FNameAsStringProxyArchive(InInnerArchive)
		{}

		virtual FArchive& operator<<(UObject*& Obj) override = 0;
		virtual FArchive& operator<<(FSoftObjectPath& Value) override = 0;
		
		virtual FArchive& operator<<(FWeakObjectPtr& Obj) override;
		virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
		virtual FArchive& operator<<(FObjectPtr& Obj) override;
	};
}