// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadCustomObjectProxyArchive.h"

#include "UObject/SoftObjectPath.h"

namespace UE::LevelSnapshots::CustomSerialization
{
	FLoadCustomObjectProxyArchive::FLoadCustomObjectProxyArchive(FArchive& InInnerArchive, const TArray<FSoftObjectPath>& TargetArray)
			: FBaseCustomObjectProxyArchive(InInnerArchive)
			, TargetArray(TargetArray)
	{
		check(InInnerArchive.IsLoading());
	}

	FArchive& FLoadCustomObjectProxyArchive::operator<<(UObject*& Obj)
	{
		FSoftObjectPath Path;
		*this << Path;
		Obj = Path.TryLoad();
		return *this;
	}
		
	FArchive& FLoadCustomObjectProxyArchive::operator<<(FSoftObjectPath& Value)
	{
		int32 ReferenceIndex;
		*this << ReferenceIndex;
		Value = TargetArray[ReferenceIndex];
		return *this;
	}
}
