// Copyright Epic Games, Inc. All Rights Reserved.

#include "SaveCustomObjectProxyArchive.h"

#include "UObject/SoftObjectPath.h"

namespace UE::LevelSnapshots::CustomSerialization
{
	FSaveCustomObjectProxyArchive::FSaveCustomObjectProxyArchive(FArchive& InInnerArchive, TArray<FSoftObjectPath>& TargetArray)
			: FBaseCustomObjectProxyArchive(InInnerArchive)
			, TargetArray(TargetArray)
	{
		check(InInnerArchive.IsSaving());
	}

	FArchive& FSaveCustomObjectProxyArchive::operator<<(UObject*& Obj)
	{
		// This could be a world reference... we ignore this case for now. It has an effect when we restore into the snapshot world.
		FSoftObjectPath Path = Obj;
		*this << Path;
		return *this;
	}
		
	FArchive& FSaveCustomObjectProxyArchive::operator<<(FSoftObjectPath& Value)
	{
		int32 Index = TargetArray.AddUnique(Value);
		*this << Index;
		return *this;
	}
}