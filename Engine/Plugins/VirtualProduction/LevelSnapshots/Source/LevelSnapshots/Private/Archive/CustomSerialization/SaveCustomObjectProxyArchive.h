// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseCustomObjectProxyArchive.h"

namespace UE::LevelSnapshots::CustomSerialization
{
	class FSaveCustomObjectProxyArchive : public FBaseCustomObjectProxyArchive
	{
		TArray<FSoftObjectPath>& TargetArray;
	public:
		
		FSaveCustomObjectProxyArchive(FArchive& InInnerArchive, TArray<FSoftObjectPath>& TargetArray);
		
		virtual FArchive& operator<<(UObject*& Obj) override;
		virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	};
}

