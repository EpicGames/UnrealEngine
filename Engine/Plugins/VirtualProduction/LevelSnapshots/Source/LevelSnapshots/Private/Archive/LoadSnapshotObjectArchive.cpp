// Copyright Epic Games, Inc. All Rights Reserved.

#include "Archive/LoadSnapshotObjectArchive.h"

#include "Data/Util/ObjectDependencyUtil.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "WorldSnapshotData.h"

#include "Serialization/ObjectWriter.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/TextPackageNamespaceUtil.h"

void UE::LevelSnapshots::Private::FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	FSnapshotDataCache& Cache,
	UObject* InObjectToRestore,
	FProcessObjectDependency ProcessObjectDependency,
	UPackage* InLocalisationSnapshotPackage)
{
	ApplyToSnapshotWorldObject(
		InObjectData,
		InSharedData,
		Cache,
		InObjectToRestore,
		ProcessObjectDependency,
#if USE_STABLE_LOCALIZATION_KEYS
		TextNamespaceUtil::EnsurePackageNamespace(InLocalisationSnapshotPackage)
#else
		FString()
#endif
		);
}

void UE::LevelSnapshots::Private::FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(
	FObjectSnapshotData& InObjectData,
	FWorldSnapshotData& InSharedData,
	FSnapshotDataCache& Cache,
	UObject* InObjectToRestore,
	FProcessObjectDependency ProcessObjectDependency,
	const FString& InLocalisationNamespace)
{
	UE_LOGF(LogLevelSnapshots, Verbose, "Loading snapshot object %ls (class %ls)", *InObjectToRestore->GetPathName(), *InObjectToRestore->GetClass()->GetPathName());
	
	FLoadSnapshotObjectArchive Archive(InObjectData, InSharedData, InObjectToRestore, ProcessObjectDependency, Cache);
#if USE_STABLE_LOCALIZATION_KEYS
	Archive.SetLocalizationNamespace(InLocalisationNamespace);
#endif

	InObjectToRestore->Serialize(Archive);
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPostLoadSnapshotObject({ InObjectToRestore, InSharedData });
}

UObject* UE::LevelSnapshots::Private::FLoadSnapshotObjectArchive::ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const
{
	FString LocalizationNamespace;
#if USE_STABLE_LOCALIZATION_KEYS
	LocalizationNamespace = GetLocalizationNamespace();
#endif

	ProcessObjectDependency(ObjectIndex);
	return ResolveObjectDependencyForSnapshotWorld(GetSharedData(), Cache, ObjectIndex, ProcessObjectDependency, LocalizationNamespace);
}

UE::LevelSnapshots::Private::FLoadSnapshotObjectArchive::FLoadSnapshotObjectArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InSerializedObject, FProcessObjectDependency ProcessObjectDependency, FSnapshotDataCache& Cache)
	: Super(InObjectData, InSharedData, true, InSerializedObject)
	, ProcessObjectDependency(ProcessObjectDependency)
	, Cache(Cache)
{
#if UE_BUILD_DEBUG
	UE_LOGF(LogLevelSnapshots, VeryVerbose, "FLoadSnapshotObjectArchive: %ls (%ls)", *InSerializedObject->GetPathName(), *InSerializedObject->GetClass()->GetPathName());
#endif
}
