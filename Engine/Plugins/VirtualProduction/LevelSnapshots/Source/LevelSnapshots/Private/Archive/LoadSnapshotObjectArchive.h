// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SnapshotArchive.h"
#include "Util/ObjectDependencyCallback.h"

struct FSnapshotDataCache;

namespace UE::LevelSnapshots::Private
{
	/* For writing data into an editor object. */
	class FLoadSnapshotObjectArchive : public FSnapshotArchive
	{
		using Super = FSnapshotArchive;
	public:
	
		static void ApplyToSnapshotWorldObject(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, FSnapshotDataCache& Cache, UObject* InObjectToRestore, FProcessObjectDependency ProcessObjectDependency, UPackage* InLocalisationSnapshotPackage);
		static void ApplyToSnapshotWorldObject(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, FSnapshotDataCache& Cache, UObject* InObjectToRestore, FProcessObjectDependency ProcessObjectDependency, const FString& InLocalisationNamespace);

	protected:

		//~ Begin FSnapshotArchive Interface
		virtual UObject* ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const override;
		//~ End FSnapshotArchive Interface

	private:

		FLoadSnapshotObjectArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InSerializedObject, FProcessObjectDependency ProcessObjectDependency, FSnapshotDataCache& Cache);

		FProcessObjectDependency ProcessObjectDependency;
		FSnapshotDataCache& Cache;
	};
}
