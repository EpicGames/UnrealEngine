// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotArchive.h"

#include "Data/Util/ObjectDependencyUtil.h"
#include "LevelSnapshotsModule.h"
#include "ObjectSnapshotData.h"
#include "SnapshotRestorability.h"
#include "SnapshotVersion.h"
#include "WorldSnapshotData.h"
#if UE_BUILD_DEBUG
#include "SnapshotConsoleVariables.h"
#endif

#include "LevelSnapshotsLog.h"
#include "Engine/Level.h"
#include "UObject/ObjectMacros.h"

FString UE::LevelSnapshots::Private::FSnapshotArchive::GetArchiveName() const
{
	return TEXT("UE::LevelSnapshots::Private::FSnapshotArchive");
}

int64 UE::LevelSnapshots::Private::FSnapshotArchive::TotalSize()
{
	return ObjectData.SerializedData.Num();
}

int64 UE::LevelSnapshots::Private::FSnapshotArchive::Tell()
{
	return DataIndex;
}

void UE::LevelSnapshots::Private::FSnapshotArchive::Seek(int64 InPos)
{
	checkSlow(InPos <= TotalSize());
	DataIndex = InPos;
}

bool UE::LevelSnapshots::Private::FSnapshotArchive::ShouldSkipProperty(const FProperty* InProperty) const
{
	// In debug builds only because this has big potential of impacting performance
#if UE_BUILD_DEBUG
	FString PropertyToDebug = ConsoleVariables::CVarBreakOnSerializedPropertyName.GetValueOnAnyThread();
	if (!PropertyToDebug.IsEmpty() && InProperty->GetName().Equals(PropertyToDebug, ESearchCase::IgnoreCase))
	{
		UE_DEBUG_BREAK();
	}

	const FString PropertyChain = [this, InProperty]()
	{
		if (GetSerializedPropertyChain())
		{
			FString Result;
			for (int32 i = 0; i < GetSerializedPropertyChain()->GetNumProperties(); ++i)
			{
				Result += GetSerializedPropertyChain()->GetPropertyFromRoot(i)->GetAuthoredName();
				Result += TEXT(".");
			}
			Result += InProperty->GetAuthoredName();
			return Result;
		}
		
		return InProperty->GetAuthoredName();
	}();
	UE_LOGF(LogLevelSnapshots, VeryVerbose, "ShouldSkipProperty(%ls %ls)", *PropertyChain, InProperty->IsNative() ? TEXT("C++") : TEXT("BP"));
#endif
	
	const bool bIsPropertyUnsupported = InProperty->HasAnyPropertyFlags(ExcludedPropertyFlags);
	return bIsPropertyUnsupported || !Restorability::IsPropertyDesirableForCapture(InProperty);
}

FArchive& UE::LevelSnapshots::Private::FSnapshotArchive::operator<<(FName& Value)
{
	if (IsLoading())
	{
		int32 NameIndex;
		*this << NameIndex;

		if (!ensureAlwaysMsgf(SharedData.SerializedNames.IsValidIndex(NameIndex), TEXT("Data appears to be corrupted")))
		{
			SetError();
			return *this;
		}
		
		Value = SharedData.SerializedNames[NameIndex];
	}
	else
	{
		int32 NameIndex;
		if (const int32* ExistingIndex = SharedData.NameToIndex.Find(Value))
		{
			NameIndex = *ExistingIndex;
		}
		else
		{
			NameIndex = SharedData.SerializedNames.Add(Value);
			SharedData.NameToIndex.Add(Value, NameIndex);
		}
		*this << NameIndex;
	}
	
	return *this;
}

FArchive& UE::LevelSnapshots::Private::FSnapshotArchive::operator<<(UObject*& Value)
{
	if (IsLoading())
	{
		int32 ReferencedIndex;
		*this << ReferencedIndex;

		if (!ensureAlwaysMsgf(SharedData.SerializedObjectReferences.IsValidIndex(ReferencedIndex), TEXT("Data appears to be corrupted")))
		{
			SetError();
			return *this;
		}
		
		const FSoftObjectPath& ObjectPath = SharedData.SerializedObjectReferences[ReferencedIndex];
		if (ObjectPath.IsNull())
		{
			Value = nullptr;
			return *this;
		}

		Value = ResolveObjectDependency(ReferencedIndex, Value);
		FixUpReference(Value);
	}
	else
	{
		int32 ReferenceIndex = AddObjectDependency(SharedData, Value);
		OnAddObjectDependency(ReferenceIndex, Value);
		*this << ReferenceIndex;
	}
	
	return *this;
}

void UE::LevelSnapshots::Private::FSnapshotArchive::Serialize(void* Data, int64 Length)
{
	if (Length <= 0)
	{
		return;
	}

	if (IsLoading())
	{
		if (!ensure(DataIndex + Length <= TotalSize()))
		{
			SetError();
			return;
		}
		
		FMemory::Memcpy(Data, &ObjectData.SerializedData[DataIndex], Length);
		DataIndex += Length;
	}
	else
	{
		const int64 RequiredEndIndex = DataIndex + Length;
		const int32 ToAlloc = RequiredEndIndex - TotalSize();
		if (ToAlloc > 0)
		{
			ObjectData.SerializedData.AddUninitialized(ToAlloc);
		}
		FMemory::Memcpy(&ObjectData.SerializedData[DataIndex], Data, Length);
		DataIndex = RequiredEndIndex;
	}
}

void UE::LevelSnapshots::Private::FSnapshotArchive::FixUpReference(UObject*& Value) const
{
	if (IsValid(Value))
	{
		const ULevel* ResolvedOwningLevel = Value->GetTypedOuter<ULevel>();
		const ULevel* SerializedObjectOwningLevel = SerializedObject->GetTypedOuter<ULevel>();
		if (ResolvedOwningLevel && ResolvedOwningLevel != SerializedObjectOwningLevel)
		{
			// Example how this could happen:
			// 1. Make actor a reference actor b
			// 2. Move actor b from level Foo to level Bar (select actor > right-click Bar in Level tab > Move selected actors to level).
			UE_LOGF(LogLevelSnapshots, Warning, "Resolved world object %ls for %ls. Referencing objects from other world / levels is not permitted: Nulling reference.", *Value->GetPathName(), *SerializedObject->GetPathName());
			Value = nullptr;
		}
	}
}

UE::LevelSnapshots::Private::FSnapshotArchive::FSnapshotArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, bool bIsLoading, UObject* InSerializedObject)
	:
	ExcludedPropertyFlags(CPF_BlueprintAssignable | CPF_Transient | CPF_Deprecated),
	SerializedObject(InSerializedObject),
    ObjectData(InObjectData),
    SharedData(InSharedData)
{
	Super::SetWantBinaryPropertySerialization(false);
	Super::SetIsTransacting(false);
	Super::SetIsPersistent(true);
	ArNoDelta = true;

	if (bIsLoading)
	{
		// Serialize properties that were valid in a previous version and are deprecated now. PostSerialize is responsible to migrate the data.
		ExcludedPropertyFlags &= ~CPF_Deprecated; 
		FArchive::SetPortFlags(PPF_UseDeprecatedProperties);
		
		Super::SetIsLoading(true);
		Super::SetIsSaving(false);
	}
	else
	{
		Super::SetIsLoading(false);
		Super::SetIsSaving(true);
	}

	if (bIsLoading)
	{
		InSharedData.SnapshotVersionInfo.ApplyToArchive(*this);
	}
}
