// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/ObjectMacros.h"

class UObject;
class UPackage;
struct FObjectSnapshotData;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/* Handles shared logic for saving and loading data for snapshots. */
	class FSnapshotArchive : public FArchiveUObject
	{
		using Super = FArchiveUObject;
	
	public:
	
		//~ Begin FArchive Interface
		virtual FString GetArchiveName() const override;
		virtual int64 TotalSize() override;
		virtual int64 Tell() override;
		virtual void Seek(int64 InPos) override;
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
		virtual FArchive& operator<<(FName& Value) override;
		virtual FArchive& operator<<(UObject*& Value) override;
		virtual void Serialize(void* Data, int64 Length) override;
		//~ End FArchive Interface

	protected:
	
		/* Allocates and serializes an object dependency, or gets the object, if it already exists. */
		virtual UObject* ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const = 0;
		/** Called when an object dependency is saved. */
		virtual void OnAddObjectDependency(int32 ObjectIndex, UObject* Object) const {}

		virtual void FixUpReference(UObject*& Value) const;
	
		FWorldSnapshotData& GetSharedData() const { return SharedData;}
		UObject* GetSerializedObject() const { return SerializedObject; }

		EPropertyFlags ExcludedPropertyFlags;

		/**
		* @param InObjectData Holds the array we're loading from or saving to
		* @param InSharedData Used to store shared data, e.g. references
		* @param bIsLoading Whether to load or save
		*/
		FSnapshotArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, bool bIsLoading, UObject* InSerializedObject);
	
	private:
	
		UObject* SerializedObject;
	
		/*  Where in ObjectData we're currently writing to. */
		int64 DataIndex = 0;

		/* The object's serialized data */
		FObjectSnapshotData& ObjectData;
		/* Stores shared data, e.g. FNames and FSoftObjectPaths */
		FWorldSnapshotData& SharedData;
	};
}