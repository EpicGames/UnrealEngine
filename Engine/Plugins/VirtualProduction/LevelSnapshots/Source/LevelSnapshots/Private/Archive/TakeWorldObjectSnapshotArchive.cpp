// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeWorldObjectSnapshotArchive.h"

#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "ObjectSnapshotData.h"
#include "WorldSnapshotData.h"

#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/UnrealType.h"

void UE::LevelSnapshots::Private::FTakeWorldObjectSnapshotArchive::TakeSnapshot(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InOriginalObject)
{
	check(InOriginalObject);

	FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();
	Module.OnPreTakeObjectSnapshot({ InOriginalObject });
	{
		FTakeWorldObjectSnapshotArchive Archive(InObjectData, InSharedData, InOriginalObject);
		InOriginalObject->Serialize(Archive);
		InObjectData.ObjectFlags = InOriginalObject->GetFlags();
	}
	Module.OnPostTakeObjectSnapshot({ InOriginalObject });
}

UE::LevelSnapshots::Private::FTakeWorldObjectSnapshotArchive::FTakeWorldObjectSnapshotArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InOriginalObject)
	: Super(InObjectData, InSharedData, false, InOriginalObject)
	, Archetype(InOriginalObject->GetArchetype())
{
#if UE_BUILD_DEBUG
	UE_LOGF(LogLevelSnapshots, VeryVerbose, "FTakeWorldObjectSnapshotArchive: %ls (%ls)", *InOriginalObject->GetPathName(), *InOriginalObject->GetClass()->GetPathName());
#endif
}

bool UE::LevelSnapshots::Private::FTakeWorldObjectSnapshotArchive::ShouldSkipProperty(const FProperty* InProperty) const
{
	const bool bSuperWantsToSkip = Super::ShouldSkipProperty(InProperty);

	if (!bSuperWantsToSkip)
	{
		// The GetSerializedObject() may have overriden UObject::Serialize and may be serializing some custom, non-reflected data.
		// For example, the object may have a non-reflected FMyStruct but manually call SerializeTaggedProperties on the FMyStruct instance.
		// In that case, we cannot call Identical_InContainer with GetSerializedObject() as owning object is not necessarily the container.
		// Solution: Just allow snapshotting all custom properties. Saving the additional property data is the safest thing to do.
		if (!bIsSerializingScriptProperties)
		{
			return false;
		}
		
		const bool bIsRootProperty = [this]()
		{
			const FArchiveSerializedPropertyChain* PropertyChain = GetSerializedPropertyChain();
			return PropertyChain == nullptr || PropertyChain->GetNumProperties() == 0;
		}();
		if (!bIsRootProperty)
		{
			// We are within a struct property and we have already checked that the struct property does not have equal values
			// Simply allow all properties but only ones which are not deprecated.
			// TODO: We could save more disk space here by checking the subproperties
			return InProperty->HasAnyPropertyFlags(CPF_Deprecated | CPF_Transient);
		}
		
		UObject* OriginalContainer = GetSerializedObject();
		UObject* ClassDefaultContainer = Archetype;
		for (int32 ArrayDim = 0; ArrayDim < InProperty->ArrayDim; ++ArrayDim)
		{
			if (!InProperty->Identical_InContainer(OriginalContainer, ClassDefaultContainer, ArrayDim))
			{
				return false; 
			}
		}
	}
	
	return true;
}
