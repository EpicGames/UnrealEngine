// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SnapshotArchive.h"

struct FObjectSnapshotData;
struct FWorldSnapshotData;
class UObject;

namespace UE::LevelSnapshots::Private
{
	/* Used when we're taking a snapshot of the world. */
	class FTakeWorldObjectSnapshotArchive final : public FSnapshotArchive
	{
		using Super = FSnapshotArchive;
	public:

		static void TakeSnapshot(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InOriginalObject);

		//~ Begin FSnapshotArchive Interface
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
		virtual void MarkScriptSerializationStart(const UObject* Obj) override { bIsSerializingScriptProperties = true; }
		virtual void MarkScriptSerializationEnd(const UObject* Obj) override { bIsSerializingScriptProperties = false; }
		//~ End FSnapshotArchive Interface

	protected:
	
		//~ Begin FSnapshotArchive Interface
		virtual UObject* ResolveObjectDependency(int32 ObjectIndex, UObject* CurrentValue) const { checkNoEntry(); return nullptr; }
		//~ End FSnapshotArchive Interface

	private:
	
		FTakeWorldObjectSnapshotArchive(FObjectSnapshotData& InObjectData, FWorldSnapshotData& InSharedData, UObject* InOriginalObject);

		UObject* Archetype;

		/**
		 * Whether currently serializing UPROPERTY()s on the SerializedObject.
		 * This is used to detect whether some custom properties are being serialized.
		 */
		bool bIsSerializingScriptProperties = false;
	};
}