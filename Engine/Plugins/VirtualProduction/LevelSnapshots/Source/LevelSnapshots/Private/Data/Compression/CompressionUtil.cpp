// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompressionUtil.h"

#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsSettings.h"
#include "WorldSnapshotData.h"

#include "Compression/CompressedBuffer.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/CustomVersion.h"

namespace UE::LevelSnapshots
{
	namespace Private
	{
		/** Saves FName and FSoftObjectPath out into arrays to be saved / loaded separately by the FLinkerSave / FLinkerLoad. */
		class FExtractNamesAndObjectPathsToArraysArchive : public FArchiveProxy
		{
		public:
		
			TArray<FName> Names;
			TArray<FSoftObjectPath> Paths;
		
			FExtractNamesAndObjectPathsToArraysArchive(FArchive& InInnerArchive)
				: FArchiveProxy(InInnerArchive)
			{
				SetIsSaving(true);
			}
			FExtractNamesAndObjectPathsToArraysArchive(FArchive& InInnerArchive, TArray<FName> Names, TArray<FSoftObjectPath> Paths)
				: FArchiveProxy(InInnerArchive)
				, Names(MoveTemp(Names))
				, Paths(MoveTemp(Paths))
			{
				SetIsLoading(true);
			}

			virtual FArchive& operator<<(FName& Value) override
			{
				if (IsSaving())
				{
					int32 Index = Names.AddUnique(Value);
					*this << Index;
				}
				else
				{
					int32 Index;
					*this << Index;
					Value = Names[Index];
				}
				return *this;
			}
		
			virtual FArchive& operator<<(FSoftObjectPath& Value) override
			{
				if (IsSaving())
				{
					int32 Index = Paths.AddUnique(Value);
					*this << Index;
				}
				else
				{
					int32 Index;
					*this << Index;
					Value = Paths[Index];
				}
				return *this;
			}
		};

		static FCompressedBuffer CompressData(const FSharedBuffer& RawData, const ECompressedBufferCompressor Compressor, const ECompressedBufferCompressionLevel CompressionLevel);
		static void SerializeUncompressableProperties(FArchive& Archive, FWorldSnapshotData& Data);
		static void SerializeCompressableProperties(FArchive& Archive, FWorldSnapshotData& Data);
		static void CopyVersionInfo(const FArchive& Source, FArchive& Target);
	}
	
	void Compress(FArchive& Ar, FWorldSnapshotData& Data)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(SerializeAndCompress);
		check(Ar.IsSaving());
		
		// 1. Uncompressed data
		Private::SerializeUncompressableProperties(Ar, Data);

		FBufferArchive UncompressedDataArchive(Ar.IsPersistent());
		UncompressedDataArchive.SetIsSaving(true);
		Private::FExtractNamesAndObjectPathsToArraysArchive ProxyArchive(UncompressedDataArchive);
		Private::SerializeCompressableProperties(ProxyArchive, Data);
		// The compressed properties may have added version info themselves which must be propagated to the root archive.
		Private::CopyVersionInfo(ProxyArchive, Ar);

		// 2. Auxiliary data - harvesting stage of save system requires this
		Ar << ProxyArchive.Names;
		Ar << ProxyArchive.Paths;

		// 3. Compressed data
		ULevelSnapshotsSettings* Settings = ULevelSnapshotsSettings::Get();
		const ECompressedBufferCompressor Compressor = Compression::CastCompressor(Settings->CompressionSettings.CompressorAlgorithm);
		const ECompressedBufferCompressionLevel CompressionLevel = Compression::CastCompressionLevel(Settings->CompressionSettings.CompressionLevel);
		const FCompressedBuffer CompressedData = Private::CompressData(FSharedBuffer::MakeView(UncompressedDataArchive.GetData(), UncompressedDataArchive.Num()), Compressor, CompressionLevel);
		CompressedData.Save(Ar);
	}

	void Decompress(FArchive& Ar, FWorldSnapshotData& Data)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(Decompress);
		check(Ar.IsLoading());
		
		// 1. Uncompressed data
		Private::SerializeUncompressableProperties(Ar, Data);
		
		// 2. Auxiliary data - harvesting stage of save system requires this
		TArray<FName> Names;
		TArray<FSoftObjectPath> Paths;
		Ar << Names;
		Ar << Paths;
		
		// 3. Compressed data
		const FCompressedBuffer CompressedData = FCompressedBuffer::Load(Ar);
		const FSharedBuffer UncompressedData = CompressedData.Decompress();
		FBufferReader UncompressedDataArchive(const_cast<void*>(UncompressedData.GetData()), UncompressedData.GetSize(), false, Ar.IsPersistent());
		
		// Needed so the compressed data can migrate
		UncompressedDataArchive.SetUEVer(Ar.UEVer());
		UncompressedDataArchive.SetEngineVer(Ar.EngineVer());
		// This will also include versions propagated by CopyVersionInfo during save
		UncompressedDataArchive.SetCustomVersions(Ar.GetCustomVersions()); 
		
		UncompressedDataArchive.SetIsLoading(true);
		Private::FExtractNamesAndObjectPathsToArraysArchive ProxyArchive(UncompressedDataArchive, MoveTemp(Names), MoveTemp(Paths));
		Private::SerializeCompressableProperties(ProxyArchive, Data);
	}

	namespace Private
	{
		static FCompressedBuffer CompressData(const FSharedBuffer& RawData, const ECompressedBufferCompressor Compressor, const ECompressedBufferCompressionLevel CompressionLevel)
		{
			SCOPED_SNAPSHOT_CORE_TRACE(Compress);
			return FCompressedBuffer::Compress(RawData, Compressor, CompressionLevel);
		}
		
		class FWorldDataArchive : public FArchiveProxy
		{
			TSet<const FProperty*> AllowedProperties;
		public:

			FWorldDataArchive(FArchive& Archive, TSet<const FProperty*> AllowedProperties)
				: FArchiveProxy(Archive)
				, AllowedProperties(MoveTemp(AllowedProperties))
			{}

			virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
			{
				const bool bIsAllowedProperty = AllowedProperties.Contains(InProperty);
				const bool bIsInAllowedProperty = IsAllowedPropertyInPropertyChain();
				return !bIsAllowedProperty && !bIsInAllowedProperty;
			}

			bool IsAllowedPropertyInPropertyChain() const
			{
				if (const FArchiveSerializedPropertyChain* Chain = GetSerializedPropertyChain())
				{
					for (int32 i = 0; i < Chain->GetNumProperties(); ++i)
					{
						if (AllowedProperties.Contains(Chain->GetPropertyFromRoot(i)))
						{
							return true;
						}
					}
				}
				return false;
			}
		};

		static TSet<const FProperty*> GetCompressedProperties()
		{
			return {
				FWorldSnapshotData::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FWorldSnapshotData, SnapshotVersionInfo)),
				FWorldSnapshotData::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FWorldSnapshotData, ClassData)),
				FWorldSnapshotData::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FWorldSnapshotData, ActorData)),
				FWorldSnapshotData::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FWorldSnapshotData, Subobjects)),
				FWorldSnapshotData::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FWorldSnapshotData, CustomSubobjectSerializationData))
			};
		}

		static void SerializeUncompressableProperties(FArchive& Archive, FWorldSnapshotData& Data)
		{
			// Names and References are collected during the harvesting state when a packet is saved
			// References must be saved normally. It seems like Names could be safe to compress, too, but the gain was minimal when tested a 4k actor map... let's not take risks for little gain...
			const TSet<const FProperty*> CompressedProperties = GetCompressedProperties();
			TSet<const FProperty*> UncompressedProperties;
			for (TFieldIterator<FProperty> PropertyIt(FWorldSnapshotData::StaticStruct()); PropertyIt; ++PropertyIt)
			{
				if (!PropertyIt->HasAnyPropertyFlags(CPF_Transient)
					&& !CompressedProperties.Contains(*PropertyIt))
				{
					UncompressedProperties.Add(*PropertyIt);
				}
			}
			
			FWorldDataArchive ArchiveProxy(Archive, UncompressedProperties);
			// SerializeTaggedProperties for forward compatibility, e.g. renaming properties
			FWorldSnapshotData::StaticStruct()->SerializeTaggedProperties(FStructuredArchiveFromArchive(ArchiveProxy).GetSlot(), (uint8*)&Data, FWorldSnapshotData::StaticStruct(), nullptr);
		}
		
		static void SerializeCompressableProperties(FArchive& Archive, FWorldSnapshotData& Data)
		{
			FWorldDataArchive ArchiveProxy(Archive, GetCompressedProperties());
			// SerializeTaggedProperties for forward compatibility, e.g. renaming properties
			FWorldSnapshotData::StaticStruct()->SerializeTaggedProperties(FStructuredArchiveFromArchive(ArchiveProxy).GetSlot(), (uint8*)&Data, FWorldSnapshotData::StaticStruct(), nullptr);
		}

		static void CopyVersionInfo(const FArchive& Source, FArchive& Target)
		{
			for (const FCustomVersion& Version : Source.GetCustomVersions().GetAllVersions())
			{
				Target.UsingCustomVersion(Version.Key);
			}
		}
	}
}
