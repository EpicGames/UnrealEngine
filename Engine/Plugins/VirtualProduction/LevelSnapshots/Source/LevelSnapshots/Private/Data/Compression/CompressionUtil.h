// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FWorldSnapshotData;

namespace UE::LevelSnapshots
{
	/** Compresses the data when saving */
	void Compress(FArchive& Ar, FWorldSnapshotData& Data);

	/** Decompresses the data when loading */
	void Decompress(FArchive& Ar, FWorldSnapshotData& Data);
}
