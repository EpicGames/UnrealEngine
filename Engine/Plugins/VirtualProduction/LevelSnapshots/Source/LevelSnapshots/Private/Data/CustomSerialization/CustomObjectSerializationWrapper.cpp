// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/CustomSerialization/CustomObjectSerializationWrapper.h"

#include "Archive/ApplySnapshotToEditorArchive.h"
#include "Archive/LoadSnapshotObjectArchive.h"
#include "CustomSerialization/CustomSerializationDataManager.h"
#include "Data/Util/ObjectDependencyUtil.h"
#include "Filtering/PropertySelectionMap.h"
#include "Restoration/Actor/ActorUtil.h"
#include "Data/WorldSnapshotData.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"

#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"

namespace UE::LevelSnapshots::Private::Internal
{
	using FSerializationDataGetter = TFunction<FCustomSerializationData*()>;
	
	static FRestoreObjectScope PreObjectRestore_SnapshotWorld(
		UObject* SnapshotObject,
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		const FProcessObjectDependency& ProcessObjectDependency,
		UPackage* LocalisationSnapshotPackage,
		FSerializationDataGetter SerializationDataGetter
		)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_PostSnapshotRestore);
		
		FLevelSnapshotsModule& LevelSnapshots = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		TSharedPtr<ICustomObjectSnapshotSerializer> CustomSerializer = LevelSnapshots.GetCustomSerializerForClass(SnapshotObject->GetClass());
		if (!CustomSerializer.IsValid() || !ensure(SerializationDataGetter() != nullptr))
		{
			return FRestoreObjectScope(nullptr);	
		}

		FCustomSerializationDataReader SerializationDataReader = FCustomSerializationDataReader(FCustomSerializationDataGetter_ReadOnly::CreateLambda([SerializationDataGetter](){ return SerializationDataGetter();}), WorldData);
		CustomSerializer->PreApplyToSnapshotObject(SnapshotObject, SerializationDataReader);
		return FRestoreObjectScope([SnapshotObject, SerializationDataGetter, &WorldData, &Cache, &ProcessObjectDependency, LocalisationSnapshotPackage, SerializationDataReader, CustomSerializer]()
		{
			for (int32 i = 0; i < SerializationDataReader.GetNumSubobjects(); ++i)
			{
				const TSharedPtr<ISnapshotSubobjectMetaData> MetaData = SerializationDataReader.GetSubobjectMetaData(i);
				UObject* SnapshotSubobject = CustomSerializer->FindOrRecreateSubobjectInSnapshotWorld(SnapshotObject, *MetaData, SerializationDataReader);
				if (!SnapshotSubobject || !ensure(SnapshotSubobject->IsIn(SnapshotObject)))
				{
					UE_LOGF(LogLevelSnapshots, Warning, "FindOrRecreateSubobjectInSnapshotWorld did not return any valid subobject. Skipping subobject restoration...");
					return;
				}

				// Recursively check whether subobjects also have a registered ICustomObjectSnapshotSerializer
				const FRestoreObjectScope FinishRestore = PreObjectRestore_SnapshotWorld(SnapshotSubobject, WorldData, Cache, ProcessObjectDependency, LocalisationSnapshotPackage,
					[&WorldData, OriginalPath = MetaData->GetOriginalPath()](){ return UE::LevelSnapshots::Private::FindCustomSubobjectData(WorldData, OriginalPath);});
				FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(SerializationDataGetter()->Subobjects[i], WorldData, Cache, SnapshotSubobject, ProcessObjectDependency, LocalisationSnapshotPackage);
				CustomSerializer->OnPostSerializeSnapshotSubobject(SnapshotSubobject, *MetaData, SerializationDataReader);
			}

			CustomSerializer->PostApplyToSnapshotObject(SnapshotObject, SerializationDataReader);
		});	
	}

	static FRestoreObjectScope PreObjectRestore_EditorWorld(
		UObject* SnapshotObject,
		UObject* EditorObject,
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		const FPropertySelectionMap& SelectionMap,
		UPackage* LocalisationSnapshotPackage,
		FSerializationDataGetter SerializationDataGetter
		)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_PostEditorRestore);
		
		FLevelSnapshotsModule& LevelSnapshots = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		TSharedPtr<ICustomObjectSnapshotSerializer> CustomSerializer = LevelSnapshots.GetCustomSerializerForClass(EditorObject->GetClass());
		if (!CustomSerializer.IsValid())
		{
			return FRestoreObjectScope(nullptr);	
		}
		
		FCustomSerializationDataReader SerializationDataReader = FCustomSerializationDataReader(FCustomSerializationDataGetter_ReadOnly::CreateLambda([SerializationDataGetter](){ return SerializationDataGetter(); }), WorldData);
		CustomSerializer->PreApplyToEditorObject(EditorObject, SerializationDataReader, SelectionMap);
		return FRestoreObjectScope([SnapshotObject, EditorObject, &WorldData, &Cache, &SelectionMap, LocalisationSnapshotPackage, SerializationDataGetter, SerializationDataReader, CustomSerializer]()
		{
			for (int32 i = 0; i < SerializationDataReader.GetNumSubobjects(); ++i)
			{	
				const TSharedPtr<ISnapshotSubobjectMetaData> MetaData = SerializationDataReader.GetSubobjectMetaData(i);
				UObject* EditorSubobject = CustomSerializer->FindOrRecreateSubobjectInEditorWorld(EditorObject, *MetaData, SerializationDataReader);
				UObject* SnapshotSubobject = CustomSerializer->FindOrRecreateSubobjectInSnapshotWorld(SnapshotObject, *MetaData, SerializationDataReader);
				if (!SnapshotSubobject || !ensure(SnapshotSubobject->IsIn(SnapshotObject)))
				{
					UE_LOGF(LogLevelSnapshots, Warning, "FindOrRecreateSubobjectInSnapshotWorld did not return any valid snapshot subobject. Skipping this subobject for %ls", *EditorObject->GetPathName());
					continue;
				}
				if (!EditorSubobject || !ensure(EditorSubobject->IsIn(EditorObject)))
				{
					UE_LOGF(LogLevelSnapshots, Warning, "FindOrRecreateSubobjectInSnapshotWorld did not return any valid editor subobject. Skipping this subobject for %ls", *EditorObject->GetPathName());
					continue;
				}

				const bool bShouldSkipProperties = SelectionMap.IsSubobjectMarkedForReferenceRestorationOnly(EditorSubobject); 
				if (bShouldSkipProperties)
				{
					continue;
				}

				if (SelectionMap.GetObjectSelection(EditorSubobject).GetPropertySelection() != nullptr)
				{
					// Recursively check whether subobjects also have a registered ICustomObjectSnapshotSerializer
					const FRestoreObjectScope FinishRestore = PreObjectRestore_EditorWorld(SnapshotSubobject, EditorSubobject, WorldData, Cache, SelectionMap, LocalisationSnapshotPackage,
						[&WorldData, OriginalPath = MetaData->GetOriginalPath()](){ return UE::LevelSnapshots::Private::FindCustomSubobjectData(WorldData, OriginalPath);} );
				
					FCustomSerializationData* SerializationData = SerializationDataGetter();
					FCustomSubbjectSerializationData& DataToRestore = SerializationData->Subobjects[i];
					// ClassIndex is only saved starting 5.3. Previous data will default to INDEX_NONE. We don't want to cause a bunch of warning logs by passing in a set class index.
					const TOptional<FClassDataIndex> ClassIndex = DataToRestore.ClassIndex == INDEX_NONE ? TOptional<uint32>{} : static_cast<FClassDataIndex>(DataToRestore.ClassIndex);
					FApplySnapshotToEditorArchive::ApplyToExistingEditorWorldObject(DataToRestore,
						WorldData,
						Cache,
						EditorSubobject,
						SnapshotSubobject,
						SelectionMap,
						ClassIndex
						);
					CustomSerializer->OnPostSerializeEditorSubobject(EditorSubobject, *MetaData, SerializationDataReader);
					continue;
				}

				const FCustomSubobjectRestorationInfo* RestorationInfo = SelectionMap.GetObjectSelection(EditorObject).GetCustomSubobjectSelection();
				const bool bWasMissingFromExistingActor = RestorationInfo && RestorationInfo->CustomSnapshotSubobjectsToRestore.Contains(SnapshotSubobject);
				const bool bWasActorRecreated = SelectionMap.GetDeletedActorsToRespawn().Contains(EditorSubobject->GetTypedOuter<AActor>());
				if (bWasMissingFromExistingActor || bWasActorRecreated)
				{
					// Recursively check whether subobjects also have a registered ICustomObjectSnapshotSerializer
					const FRestoreObjectScope FinishRestore = PreObjectRestore_EditorWorld(SnapshotSubobject, EditorSubobject, WorldData, Cache, SelectionMap, LocalisationSnapshotPackage,
						[&WorldData, OriginalPath = MetaData->GetOriginalPath()](){ return UE::LevelSnapshots::Private::FindCustomSubobjectData(WorldData, OriginalPath);} );

					FCustomSerializationData* SerializationData = SerializationDataGetter();
					FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithArchetype(SerializationData->Subobjects[i], WorldData, Cache, EditorSubobject, SelectionMap);
					CustomSerializer->OnPostSerializeEditorSubobject(EditorSubobject, *MetaData, SerializationDataReader);
					continue;
				}
				
				UE_LOGF(LogLevelSnapshots, Warning, "Editor subobject %ls was not restored", *EditorSubobject->GetPathName());	
			}

			CustomSerializer->PostApplyToEditorObject(EditorObject, SerializationDataReader, SelectionMap);
		});	
	}
}

void UE::LevelSnapshots::Private::TakeSnapshotOfActorCustomSubobjects(
	AActor* EditorActor,
	FCustomSerializationData& ActorSerializationData,
	FWorldSnapshotData& WorldData)
{
	SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_TakeActorSnapshot);
	
	FLevelSnapshotsModule& LevelSnapshots = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
	TSharedPtr<ICustomObjectSnapshotSerializer> CustomSerializer = LevelSnapshots.GetCustomSerializerForClass(EditorActor->GetClass());
	if (!CustomSerializer.IsValid())
	{
		return;
	}
	
	FCustomSerializationDataWriter SerializationDataWriter = FCustomSerializationDataWriter(
		FCustomSerializationDataGetter_ReadWrite::CreateLambda([&ActorSerializationData]() { return &ActorSerializationData; }),
		WorldData,
		EditorActor
		);
	CustomSerializer->OnTakeSnapshot(EditorActor, SerializationDataWriter);
}

void UE::LevelSnapshots::Private::TakeSnapshotForSubobject(
	UObject* Subobject,
	FWorldSnapshotData& WorldData)
{
	SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_TakeSubobjectSnapshot);
	
	FLevelSnapshotsModule& LevelSnapshots = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
	TSharedPtr<ICustomObjectSnapshotSerializer> CustomSerializer = LevelSnapshots.GetCustomSerializerForClass(Subobject->GetClass());
	if (!CustomSerializer.IsValid())
	{
		return;
	}

	const int32 SubobjectIndex = AddCustomSubobjectDependency(WorldData, Subobject); 
	FCustomSerializationDataWriter SerializationDataWriter = FCustomSerializationDataWriter(
		FCustomSerializationDataGetter_ReadWrite::CreateLambda([&WorldData, SubobjectIndex]() { return WorldData.CustomSubobjectSerializationData.Find(SubobjectIndex); }),
		WorldData,
		Subobject
		);
	CustomSerializer->OnTakeSnapshot(Subobject, SerializationDataWriter);
}

UE::LevelSnapshots::Private::FRestoreObjectScope UE::LevelSnapshots::Private::PreActorRestore_SnapshotWorld(
	AActor* SnapshotActor,
	FCustomSerializationData& ActorSerializationData,
	FWorldSnapshotData& WorldData,
	FSnapshotDataCache& Cache,
	const FProcessObjectDependency& ProcessObjectDependency,
	UPackage* LocalisationSnapshotPackage)
{
	SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_PreSnapshotActorRestore);
	
	return Internal::PreObjectRestore_SnapshotWorld(
		SnapshotActor,
		WorldData,
		Cache,
		ProcessObjectDependency,
		LocalisationSnapshotPackage,
		[&ActorSerializationData](){ return &ActorSerializationData; }
		);
}

UE::LevelSnapshots::Private::FRestoreObjectScope UE::LevelSnapshots::Private::PreActorRestore_EditorWorld(
	AActor* EditorActor,
	FCustomSerializationData& ActorSerializationData,
	FWorldSnapshotData& WorldData,
	FSnapshotDataCache& Cache,
	const FPropertySelectionMap& SelectionMap,
	UPackage* LocalisationSnapshotPackage)
{
	SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_PreEditorRestore);
	
	const TOptional<TNonNullPtr<AActor>> SnapshotActor = GetDeserializedActor(EditorActor, WorldData, Cache, LocalisationSnapshotPackage);
	if (!ensure(SnapshotActor))
	{
		return FRestoreObjectScope([](){});
	}
	
	return Internal::PreObjectRestore_EditorWorld(
		SnapshotActor.GetValue(),
		EditorActor,
		WorldData,
		Cache,
		SelectionMap,
		LocalisationSnapshotPackage,
		[&ActorSerializationData](){ return &ActorSerializationData;}
		);
}

UE::LevelSnapshots::Private::FRestoreObjectScope UE::LevelSnapshots::Private::PreSubobjectRestore_SnapshotWorld(
	UObject* Subobject,
	const FSoftObjectPath& OriginalSubobjectPath,
	FWorldSnapshotData& WorldData,
	FSnapshotDataCache& Cache,
	const FProcessObjectDependency& ProcessObjectDependency,
	UPackage* LocalisationSnapshotPackage
	)
{
	SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_PreSnapshotRestore);
	
	return Internal::PreObjectRestore_SnapshotWorld(
		Subobject,
		WorldData,
		Cache,
		ProcessObjectDependency,
		LocalisationSnapshotPackage,
		[&WorldData, OriginalSubobjectPath](){ return FindCustomSubobjectData(WorldData, OriginalSubobjectPath); }
		);
}

UE::LevelSnapshots::Private::FRestoreObjectScope UE::LevelSnapshots::Private::PreSubobjectRestore_EditorWorld(
	UObject* SnapshotObject,
	UObject* EditorObject,
	FWorldSnapshotData& WorldData,
	FSnapshotDataCache& Cache,
	const FPropertySelectionMap& SelectionMap,
	UPackage* LocalisationSnapshotPackage
	)
{
	SCOPED_SNAPSHOT_CORE_TRACE(CustomObjectSerialization_PreEditorRestore);
	
	const FSoftObjectPath SubobjectPath(EditorObject);
	if (!FindCustomSubobjectData(WorldData, SubobjectPath))
	{
		return FRestoreObjectScope(nullptr);	
	}
	
	return Internal::PreObjectRestore_EditorWorld(
		SnapshotObject,
		EditorObject,
		WorldData,
		Cache,
		SelectionMap,
		LocalisationSnapshotPackage,
		[&WorldData, SubobjectPath](){ return FindCustomSubobjectData(WorldData, SubobjectPath); }
		);
}

void UE::LevelSnapshots::Private::ForEachMatchingCustomSubobjectPair(const FWorldSnapshotData& WorldData, UObject* SnapshotObject, UObject* WorldObject, FHandleCustomSubobjectPair HandleCustomSubobjectPair,  FHandleUnmatchedCustomSnapshotSubobject HandleUnmachtedCustomSnapshotSubobject)
{
	FLevelSnapshotsModule& LevelSnapshots = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
    TSharedPtr<ICustomObjectSnapshotSerializer> CustomSerializer = LevelSnapshots.GetCustomSerializerForClass(WorldObject->GetClass());
    if (!CustomSerializer.IsValid())
    {
    	return;
    }

    const FCustomSerializationData* SubobjectData = FindCustomActorOrSubobjectData(WorldData, WorldObject);
    if (!SubobjectData)
    {
    	return;
    }

    FCustomSerializationDataReader SerializationDataReader = FCustomSerializationDataReader(FCustomSerializationDataGetter_ReadOnly::CreateLambda([SubobjectData](){ return SubobjectData;}), WorldData);
    for (int32 i = 0; i < SerializationDataReader.GetNumSubobjects(); ++i)
    {
    	const TSharedPtr<ISnapshotSubobjectMetaData> MetaData = SerializationDataReader.GetSubobjectMetaData(i);
        UObject* EditorSubobject = CustomSerializer->FindSubobjectInEditorWorld(WorldObject, *MetaData, SerializationDataReader);
        UObject* SnapshotCounterpart = CustomSerializer->FindOrRecreateSubobjectInSnapshotWorld(SnapshotObject, *MetaData, SerializationDataReader);
    	if (!SnapshotCounterpart || !ensure(SnapshotCounterpart->IsIn(SnapshotObject)))
    	{
    		UE_LOGF(LogLevelSnapshots, Warning, "FindOrRecreateSubobjectInSnapshotWorld did not return any valid subobject. Skipping subobject restoration...");
    		continue;
    	}

    	if (!EditorSubobject)
    	{
    		HandleUnmachtedCustomSnapshotSubobject(SnapshotCounterpart);
    	}
    	else if (ensure(EditorSubobject->IsIn(WorldObject)))
    	{
    		HandleCustomSubobjectPair(SnapshotCounterpart, EditorSubobject);
    	}
    }
}
