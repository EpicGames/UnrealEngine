// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Util/ObjectDependencyCallback.h"
#include "Templates/Function.h"

struct FSnapshotDataCache;
class AActor;
class FCustomSerializationDataManager;
class ICustomObjectSnapshotSerializer;

struct FCustomSerializationData;
struct FPropertySelectionMap;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/* Utility for calling final functions when using custom object serialization. */
	class FRestoreObjectScope
	{
		friend class FCustomObjectSerializationWrapper;
		using FFinaliseRestorationCallback = TFunction<void()>;

		FFinaliseRestorationCallback Callback;

		FRestoreObjectScope() = delete;
		FRestoreObjectScope(const FRestoreObjectScope& Other) = delete;

	public:

		FRestoreObjectScope(FFinaliseRestorationCallback Callback)
			: Callback(MoveTemp(Callback))
		{}

		~FRestoreObjectScope()
		{
			if (Callback)
			{
				Callback();
			}
		}

		FRestoreObjectScope(FRestoreObjectScope&& Other)
			: Callback(Other.Callback)
		{
			Other.Callback = nullptr;
		}
	};

	void TakeSnapshotOfActorCustomSubobjects(
			AActor* EditorActor,
			FCustomSerializationData& ActorSerializationData,
			FWorldSnapshotData& WorldData
			);
	
	void TakeSnapshotForSubobject(
		UObject* Subobject,
		FWorldSnapshotData& WorldData
		);


	
	[[nodiscard]] FRestoreObjectScope PreActorRestore_SnapshotWorld(
		AActor* EditorActor,
		FCustomSerializationData& ActorSerializationData,
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		const FProcessObjectDependency& ProcessObjectDependency,
		UPackage* LocalisationSnapshotPackage
		);

	[[nodiscard]] FRestoreObjectScope PreActorRestore_EditorWorld(
		AActor* EditorActor,
		FCustomSerializationData& ActorSerializationData,
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		const FPropertySelectionMap& SelectionMap,
		UPackage* LocalisationSnapshotPackage
		);
	

	[[nodiscard]] FRestoreObjectScope PreSubobjectRestore_SnapshotWorld(
		UObject* Subobject,
		const FSoftObjectPath& OriginalSubobjectPath,
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		const FProcessObjectDependency& ProcessObjectDependency,
		UPackage* LocalisationSnapshotPackage
		);
	
	[[nodiscard]] FRestoreObjectScope PreSubobjectRestore_EditorWorld(
		UObject* SnapshotObject,
		UObject* EditorObject,
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		const FPropertySelectionMap& SelectionMap,
		UPackage* LocalisationSnapshotPackage
		);

	
	using FHandleCustomSubobjectPair = TFunctionRef<void(UObject* SnapshotSubobject, UObject* EditorSubobject)>;
	using FHandleUnmatchedCustomSnapshotSubobject = TFunctionRef<void(UObject* UnmatchedSnapshotSubobject)>;
	
	void ForEachMatchingCustomSubobjectPair(const FWorldSnapshotData& WorldData, UObject* SnapshotObject, UObject* WorldObject, FHandleCustomSubobjectPair HandleCustomSubobjectPair, FHandleUnmatchedCustomSnapshotSubobject HandleUnmachtedCustomSnapshotSubobject);
}

	
