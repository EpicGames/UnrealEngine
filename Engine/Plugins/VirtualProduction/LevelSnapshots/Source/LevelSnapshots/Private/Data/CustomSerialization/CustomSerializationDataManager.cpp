// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomSerializationDataManager.h"

#include "CustomObjectSerializationWrapper.h"
#include "CustomSerialization/LoadCustomObjectProxyArchive.h"
#include "CustomSerialization/SaveCustomObjectProxyArchive.h"
#include "Data/Util/ObjectDependencyUtil.h"
#include "LevelSnapshotsLog.h"
#include "SnapshotCustomVersion.h"
#include "TakeWorldObjectSnapshotArchive.h"
#include "Util/ClassDataUtil.h"

#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

namespace UE::LevelSnapshots::Private::Internal
{
	class FSnapshotSubobjectMetaDataReader
		:
		public ISnapshotSubobjectMetaData,
		public TSharedFromThis<FSnapshotSubobjectMetaDataReader>
	{
		const FSoftObjectPath CachedSubobjectPath;
		const int32 OwningDataSubobjectIndex;
		const FWorldSnapshotData& WorldData;
	
		const FCustomSerializationDataGetter_ReadOnly SerializationDataGetter_ReadOnly;
	protected:
		
		int32 GetOwningDataSubobjectIndex() const { return OwningDataSubobjectIndex; }
		
	public:

		FSnapshotSubobjectMetaDataReader(
			FSoftObjectPath CachedSubobjectPath,
			int32 SubobjectIndex,
			const FWorldSnapshotData& WorldData,
			FCustomSerializationDataGetter_ReadOnly SerializationDataGetter_ReadOnly
			)
			: CachedSubobjectPath(CachedSubobjectPath)
			, OwningDataSubobjectIndex(SubobjectIndex)
			, WorldData(WorldData)
			, SerializationDataGetter_ReadOnly(SerializationDataGetter_ReadOnly)
		{}

		virtual FSoftObjectPath GetOriginalPath() const override
		{
			return CachedSubobjectPath;
		}
		
		virtual void WriteObjectAnnotation(const FObjectAnnotator& Writer) override
		{
			checkNoEntry(); // Not allowed on read-only object
		}
		
		virtual void ReadObjectAnnotation(const FObjectAnnotator& Reader) const override
		{
			if (Reader.IsBound() && ensure(SerializationDataGetter_ReadOnly.IsBound()))
			{
				if (WorldData.SnapshotVersionInfo.GetSnapshotCustomVersion() >= FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
				{
					FMemoryReader MemoryReader(SerializationDataGetter_ReadOnly.Execute()->Subobjects[OwningDataSubobjectIndex].SubobjectAnnotationData, true);
					CustomSerialization::FLoadCustomObjectProxyArchive RootArchive(MemoryReader, WorldData.SerializedObjectReferences);
					WorldData.SnapshotVersionInfo.ApplyToArchive(RootArchive);
					Reader.Execute(RootArchive);
				}
				else
				{
					FMemoryReader MemoryReader(SerializationDataGetter_ReadOnly.Execute()->Subobjects[OwningDataSubobjectIndex].SubobjectAnnotationData, true);
					FObjectAndNameAsStringProxyArchive RootArchive(MemoryReader, true);
					WorldData.SnapshotVersionInfo.ApplyToArchive(RootArchive);
					Reader.Execute(RootArchive);
				}
			}
		}
	};

	class FSnapshotSubobjectMetaDataWriter : public FSnapshotSubobjectMetaDataReader
	{
		FWorldSnapshotData& WorldData_ReadWrite;
		const FCustomSerializationDataGetter_ReadWrite SerializationDataGetter_ReadWrite;
	public:

		FSnapshotSubobjectMetaDataWriter(
			FSoftObjectPath CachedSubobjectPath,
			int32 SubobjectIndex,
			FWorldSnapshotData& WorldData,
			FCustomSerializationDataGetter_ReadOnly SerializationDataGetter_ReadOnly,
			FCustomSerializationDataGetter_ReadWrite SerializationDataGetter_ReadWrite
			)
			: FSnapshotSubobjectMetaDataReader(CachedSubobjectPath, SubobjectIndex, WorldData, SerializationDataGetter_ReadOnly)
			, WorldData_ReadWrite(WorldData)
			, SerializationDataGetter_ReadWrite(SerializationDataGetter_ReadWrite)
		{}
	
		virtual void WriteObjectAnnotation(const FObjectAnnotator& Writer) override
		{
			if (Writer.IsBound() && ensure(SerializationDataGetter_ReadWrite.IsBound()))
			{
				if (WorldData_ReadWrite.SnapshotVersionInfo.GetSnapshotCustomVersion() >= FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
				{
					FMemoryWriter MemoryWriter(SerializationDataGetter_ReadWrite.Execute()->Subobjects[GetOwningDataSubobjectIndex()].SubobjectAnnotationData, true);
					CustomSerialization::FSaveCustomObjectProxyArchive RootArchive(MemoryWriter, WorldData_ReadWrite.SerializedObjectReferences);
					WorldData_ReadWrite.SnapshotVersionInfo.ApplyToArchive(RootArchive);
					Writer.Execute(RootArchive);
				}
				else
				{
					FMemoryWriter MemoryWriter(SerializationDataGetter_ReadWrite.Execute()->Subobjects[GetOwningDataSubobjectIndex()].SubobjectAnnotationData, true);
					FObjectAndNameAsStringProxyArchive RootArchive(MemoryWriter, false);
					Writer.Execute(RootArchive);
				}
			}
		}
	};
}

UE::LevelSnapshots::Private::FCustomSerializationDataReader::FCustomSerializationDataReader(FCustomSerializationDataGetter_ReadOnly SerializationDataGetter, const FWorldSnapshotData& ConstWorldData)
	: SerializationDataGetter_ReadOnly(SerializationDataGetter)
	, WorldData_ReadOnly(ConstWorldData)
{
	if (SerializationDataGetter.IsBound())
	{
		const FCustomSerializationData* SerializationData = SerializationDataGetter_ReadOnly.Execute();
		for (int32 i = 0; i < SerializationData->Subobjects.Num(); ++i)
		{
			const FCustomSubbjectSerializationData& SubobjectData = SerializationData->Subobjects[i];
			CachedSubobjectMetaData.Add(
				MakeShared<Internal::FSnapshotSubobjectMetaDataReader>(
						ConstWorldData.SerializedObjectReferences[SubobjectData.ObjectPathIndex],
						i,
						WorldData_ReadOnly,
						SerializationDataGetter_ReadOnly
					)
				);
		}
	}
}

void UE::LevelSnapshots::Private::FCustomSerializationDataReader::WriteObjectAnnotation(const FObjectAnnotator& Writer)
{
	checkNoEntry();
}

void UE::LevelSnapshots::Private::FCustomSerializationDataReader::ReadObjectAnnotation(const FObjectAnnotator& Reader) const
{
	if (Reader.IsBound())
	{
		if (WorldData_ReadOnly.SnapshotVersionInfo.GetSnapshotCustomVersion() >= FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
		{
			FMemoryReader MemoryReader(SerializationDataGetter_ReadOnly.Execute()->RootAnnotationData, true);
			CustomSerialization::FLoadCustomObjectProxyArchive RootArchive(MemoryReader, WorldData_ReadOnly.SerializedObjectReferences);
			WorldData_ReadOnly.SnapshotVersionInfo.ApplyToArchive(RootArchive);
			Reader.Execute(RootArchive);
		}
		else
		{
			FMemoryReader MemoryReader(SerializationDataGetter_ReadOnly.Execute()->RootAnnotationData, true);
			FObjectAndNameAsStringProxyArchive RootArchive(MemoryReader, true);
			WorldData_ReadOnly.SnapshotVersionInfo.ApplyToArchive(RootArchive);
			Reader.Execute(RootArchive);
		}
	}
}

int32 UE::LevelSnapshots::Private::FCustomSerializationDataReader::AddSubobjectSnapshot(UObject* Subobject)
{
	return INDEX_NONE;	
}

TSharedPtr<UE::LevelSnapshots::ISnapshotSubobjectMetaData> UE::LevelSnapshots::Private::FCustomSerializationDataReader::GetSubobjectMetaData(int32 Index)
{
	return ensure(CachedSubobjectMetaData.IsValidIndex(Index)) ? CachedSubobjectMetaData[Index] : nullptr;
}

const TSharedPtr<UE::LevelSnapshots::ISnapshotSubobjectMetaData> UE::LevelSnapshots::Private::FCustomSerializationDataReader::GetSubobjectMetaData(int32 Index) const
{
	return ensure(CachedSubobjectMetaData.IsValidIndex(Index)) ? CachedSubobjectMetaData[Index] : nullptr;
}

int32 UE::LevelSnapshots::Private::FCustomSerializationDataReader::GetNumSubobjects() const
{
	return SerializationDataGetter_ReadOnly.Execute()->Subobjects.Num();
}

UE::LevelSnapshots::Private::FCustomSerializationDataWriter::FCustomSerializationDataWriter(FCustomSerializationDataGetter_ReadWrite SerializationDataGetter, FWorldSnapshotData& WorldData, UObject* SerializedObject)
	: FCustomSerializationDataReader(FCustomSerializationDataGetter_ReadOnly(), WorldData)
	, SerializationDataGetter_ReadWrite(SerializationDataGetter)
	, WorldData_ReadWrite(WorldData)
	, SerializedObject(SerializedObject)
{
	SerializationDataGetter_ReadOnly = FCustomSerializationDataGetter_ReadOnly::CreateLambda([this]() -> const FCustomSerializationData* { return SerializationDataGetter_ReadWrite.Execute(); });
	
	const FCustomSerializationData* SerializationData = SerializationDataGetter_ReadOnly.Execute();
	for (int32 i = 0; i < SerializationData->Subobjects.Num(); ++i)
	{
		const FCustomSubbjectSerializationData& SubobjectData = SerializationData->Subobjects[i];
		CachedSubobjectMetaData.Add(
			MakeShared<Internal::FSnapshotSubobjectMetaDataWriter>(
				WorldData.SerializedObjectReferences[SubobjectData.ObjectPathIndex],
					i,
					WorldData,
					SerializationDataGetter_ReadOnly,
					SerializationDataGetter_ReadWrite
				)
			);
	}
}

void UE::LevelSnapshots::Private::FCustomSerializationDataWriter::WriteObjectAnnotation(const FObjectAnnotator& Writer)
{
	if (Writer.IsBound())
	{
		if (WorldData_ReadOnly.SnapshotVersionInfo.GetSnapshotCustomVersion() >= FSnapshotCustomVersion::CustomSubobjectSoftObjectPathRefactor)
		{
			FMemoryWriter MemoryWriter(SerializationDataGetter_ReadWrite.Execute()->RootAnnotationData, true);
			CustomSerialization::FSaveCustomObjectProxyArchive RootArchive(MemoryWriter, WorldData_ReadWrite.SerializedObjectReferences);
			WorldData_ReadOnly.SnapshotVersionInfo.ApplyToArchive(RootArchive);
			Writer.Execute(RootArchive);
		}
		else
		{
			FMemoryWriter MemoryWriter(SerializationDataGetter_ReadWrite.Execute()->RootAnnotationData, true);
			FObjectAndNameAsStringProxyArchive RootArchive(MemoryWriter, false);
			WorldData_ReadOnly.SnapshotVersionInfo.ApplyToArchive(RootArchive);
			Writer.Execute(RootArchive);
		}
	}
}

int32 UE::LevelSnapshots::Private::FCustomSerializationDataWriter::AddSubobjectSnapshot(UObject* Subobject)
{
	if (!ensure(Subobject) || !ensure(Subobject->IsIn(SerializedObject)))
	{
		const FString SubobjectPath = Subobject ? Subobject->GetPathName() : FString("None");
		const FString SerializedObjectPath = SerializedObject ? SerializedObject->GetPathName() : FString("None");
		UE_LOGF(LogLevelSnapshots, Error, "%ls is not in %ls!", *SubobjectPath, *SerializedObjectPath);
		return INDEX_NONE;
	}

	FCustomSerializationData* SerializationData = SerializationDataGetter_ReadWrite.Execute();
	// This may be slow because the array may have thousands of elements but AddSubobject shouldn't be called that often hopefully ...
	const int32 ObjectIndex = WorldData_ReadWrite.SerializedObjectReferences.Find(Subobject);
	const int32 ExistingSubobjectIndex = SerializationData->Subobjects.FindLastByPredicate([ObjectIndex](const FCustomSubbjectSerializationData& Data)
	{
		return Data.ObjectPathIndex == ObjectIndex;
	});

	if (!ensure(ObjectIndex == INDEX_NONE))
	{
		UE_LOGF(LogLevelSnapshots, Error, "You tried to register an object (%ls) which was already found by standard Level Snapshot serialisation. Is your subobject referenced by an property with a CPF_Edit flag?", *Subobject->GetPathName());
		return ExistingSubobjectIndex;
	}
	if (!ensure(ExistingSubobjectIndex == INDEX_NONE))
	{
		UE_LOGF(LogLevelSnapshots, Error, "You tried to register the same subobject (%ls) twice.", *Subobject->GetPathName());
		return ExistingSubobjectIndex;
	}

	FCustomSubbjectSerializationData SubobjectData;
	SubobjectData.ObjectPathIndex = AddObjectDependency(WorldData_ReadWrite, Subobject, false);
	SubobjectData.ClassIndex = AddClassArchetype(WorldData_ReadWrite, Subobject);
	const int32 SubobjectIndex = SerializationData->Subobjects.Emplace(
		MoveTemp(SubobjectData) // Not profiled
		);
	// Optimisation: Serialize into the allocated SerializationData directly avoiding a possibly large copy of the serialized data
	FTakeWorldObjectSnapshotArchive::TakeSnapshot(SerializationData->Subobjects[SubobjectIndex], WorldData_ReadWrite, Subobject);
	
	CachedSubobjectMetaData.Add(MakeShared<Internal::FSnapshotSubobjectMetaDataWriter>(Subobject, SubobjectIndex, WorldData_ReadWrite, SerializationDataGetter_ReadOnly, SerializationDataGetter_ReadWrite));

	TakeSnapshotForSubobject(Subobject, WorldData_ReadWrite);
	return SubobjectIndex;
}
