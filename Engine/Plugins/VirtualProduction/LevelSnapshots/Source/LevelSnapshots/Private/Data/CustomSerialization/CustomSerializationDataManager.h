// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomSerializationData.h"
#include "WorldSnapshotData.h"
#include "Params/ObjectSnapshotSerializationData.h"

namespace UE::LevelSnapshots::Private
{
	DECLARE_DELEGATE_RetVal(const FCustomSerializationData*, FCustomSerializationDataGetter_ReadOnly);
	DECLARE_DELEGATE_RetVal(FCustomSerializationData*, FCustomSerializationDataGetter_ReadWrite);

	class FCustomSerializationDataReader : public ICustomSnapshotSerializationData
	{
	public:
		
		FCustomSerializationDataReader(FCustomSerializationDataGetter_ReadOnly SerializationDataGetter, const FWorldSnapshotData& ConstWorldData);

		virtual void WriteObjectAnnotation(const FObjectAnnotator& Writer) override;
		virtual void ReadObjectAnnotation(const FObjectAnnotator& Reader) const override;
		virtual int32 AddSubobjectSnapshot(UObject* Subobject) override;
		virtual TSharedPtr<ISnapshotSubobjectMetaData> GetSubobjectMetaData(int32 Index) override;
		virtual const TSharedPtr<ISnapshotSubobjectMetaData> GetSubobjectMetaData(int32 Index) const override;
		virtual int32 GetNumSubobjects() const override;

	protected:
		
		/** To avoid dynamically allocating memory every time GetSubobjectMetaData is called, we cache the results here. Just to improve heap memory footprint a tiny bit. */
		TArray<TSharedPtr<ISnapshotSubobjectMetaData>> CachedSubobjectMetaData;
		
		/** Looks up actor or subobject  serialization data. Used instead of reference because some functions, like TakeSubobjectSnapshot, cause FWorldSnapshotData::CustomSubobjectSerializationData to reallocate. */
		FCustomSerializationDataGetter_ReadOnly SerializationDataGetter_ReadOnly;

		/* Needed to look up soft object paths by index */
		const FWorldSnapshotData& WorldData_ReadOnly;
	};

	class FCustomSerializationDataWriter : public FCustomSerializationDataReader
	{
	public:

		FCustomSerializationDataWriter(FCustomSerializationDataGetter_ReadWrite SerializationDataGetter, FWorldSnapshotData& WorldData, UObject* SerializedObject);
		
		virtual void WriteObjectAnnotation(const FObjectAnnotator& Writer) override;
		virtual int32 AddSubobjectSnapshot(UObject* Subobject) override;

	private:

		/** Looks up actor or subobject  serialization data. Used instead of reference because some functions, like TakeSubobjectSnapshot, cause FWorldSnapshotData::CustomSubobjectSerializationData to reallocate. */
		FCustomSerializationDataGetter_ReadWrite SerializationDataGetter_ReadWrite;

		/* Needed to look up soft object paths by index */
		FWorldSnapshotData& WorldData_ReadWrite;

		/* Has no function: just used for validating. */
		UObject* SerializedObject;
	};
}


