// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorHashUtil.h"

#include "Data/Hashing/ActorSnapshotHash.h"
#include "Data/Hashing/HashArchive.h"
#include "Data/Hashing/SerializeObjectState.h"
#include "LevelSnapshotsLog.h"
#include "GameFramework/Actor.h"

#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryHasher.h"

FHashSettings UE::LevelSnapshots::Private::GHashSettings;

void UE::LevelSnapshots::Private::PopulateActorHash(FActorSnapshotHash& ActorData, AActor* WorldActor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(PopulateActorHash);

	// We compute both CRC32 and MD5 because:
		// 1. Future proofing: if it turns out one has too many collisions, we can switch to the other without migration
		// 2. Sometimes one is faster than the other. When loading, we use the one that took the least time.
		// 3. The overhead is ok. Both together take 1 ms on average.

	if (GHashSettings.bCanUseCRC)
	{
		const double StartTime = FPlatformTime::Seconds();

		// FArchiveComputeFullCrc32 is faster than FArchiveComputeIncrementalCrc32 for actors
		FArchiveComputeFullCrc32 CrcArchive;
		SerializeObjectState::SerializeWithSubobjects(CrcArchive, WorldActor);
		ActorData.Crc32DataLength = CrcArchive.GetSerializedData().Num();
		ActorData.Crc32 = CrcArchive.GetCrc32();
		ActorData.MicroSecondsForCrc = FPlatformTime::Seconds() - StartTime;
	}
	
	if (GHashSettings.bCanUseMD5)
	{
		const double StartTime = FPlatformTime::Seconds();
		
		// FArchiveComputeIncrementalMD5 is faster than FArchiveComputeFullMD5 for actors
		FArchiveComputeFullMD5 MD5Archive;
		SerializeObjectState::SerializeWithSubobjects(MD5Archive, WorldActor);
		ActorData.MD5DataLength = MD5Archive.GetSerializedData().Num();
		ActorData.MD5 = MD5Archive.GetMD5();
		ActorData.MicroSecondsForMD5 = FPlatformTime::Seconds() - StartTime;
	}
}

namespace UE::LevelSnapshots::Private::Internal
{
	enum class EHashAlgorithm
	{
		None,
		CRC32,
		MD5
	};

	static EHashAlgorithm GetCRC32Algorithm(const FActorSnapshotHash& ActorData)
	{
		return ActorData.HasCrc32() ? EHashAlgorithm::CRC32 : EHashAlgorithm::None;
	}
	static EHashAlgorithm GetMD5Algorithm(const FActorSnapshotHash& ActorData)
	{
		return ActorData.HasMD5() ? EHashAlgorithm::MD5 : EHashAlgorithm::None;
	}
	static EHashAlgorithm GetFastestAlgorithm(const FActorSnapshotHash& ActorData)
	{
		const bool bCrcIsFaster = ActorData.MicroSecondsForCrc < ActorData.MicroSecondsForMD5;
		return bCrcIsFaster ?
				ActorData.HasCrc32() ? EHashAlgorithm::CRC32 : GetMD5Algorithm(ActorData)
				:
				ActorData.HasMD5() ? EHashAlgorithm::MD5 : GetCRC32Algorithm(ActorData);
	}
	
	
	static EHashAlgorithm DetermineHashAlgorithm(const FActorSnapshotHash& ActorData)
	{
		if (!GHashSettings.bUseHashForLoading)
		{
			return EHashAlgorithm::None;
		}
		
		switch (GHashSettings.SnapshotDiffAlgorithm)
		{
			case EHashAlgorithmChooseBehavior::UseCrc32:
				return GetCRC32Algorithm(ActorData);
			case EHashAlgorithmChooseBehavior::UseMD5:
				return GetMD5Algorithm(ActorData);
			
			case EHashAlgorithmChooseBehavior::UseFastest:
			default:
				return GetFastestAlgorithm(ActorData);
		}
	}
}

bool UE::LevelSnapshots::Private::HasMatchingHash(const FActorSnapshotHash& ActorData, AActor* WorldActor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(HasMatchingHash);
	
	switch (Internal::DetermineHashAlgorithm(ActorData))
	{
		case Internal::EHashAlgorithm::CRC32:
			if (ActorData.MicroSecondsForCrc < GHashSettings.HashCutoffSeconds)
			{
				FArchiveComputeFullCrc32 Archive;
				SerializeObjectState::SerializeWithSubobjects(Archive, WorldActor);
				return ActorData.Crc32DataLength == Archive.GetSerializedData().Num() && Archive.GetSerializedData().Num() && ActorData.Crc32 == Archive.GetCrc32();
			}
			return false;
			
		case Internal::EHashAlgorithm::MD5:
			if (ActorData.MicroSecondsForCrc < GHashSettings.HashCutoffSeconds)
			{
				FArchiveComputeFullMD5 Archive;
				SerializeObjectState::SerializeWithSubobjects(Archive, WorldActor);
				return ActorData.MD5DataLength == Archive.GetSerializedData().Num() && ActorData.MD5 == Archive.GetMD5();
			}
			return false;
		
		case Internal::EHashAlgorithm::None:
		default:
			return false;
	}
}