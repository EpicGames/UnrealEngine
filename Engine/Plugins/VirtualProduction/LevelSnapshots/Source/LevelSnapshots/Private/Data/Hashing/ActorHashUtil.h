// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HashSettings.h"

class AActor;
class UObject;

struct FActorSnapshotData;
struct FWorldSnapshotData;
struct FActorSnapshotHash;

namespace UE::LevelSnapshots::Private
{
	/** Updated by project settings */
	extern FHashSettings GHashSettings;
	
	/**
	 * Computes and populates hash data for actor.
	 */
	void PopulateActorHash(FActorSnapshotHash& HashData, AActor* WorldActor);

	/**
	 * Computes hash for actor and compares it to saved data.
	 * @return Whether the computed hash matches the saved hash.
	 */
	bool HasMatchingHash(const FActorSnapshotHash& ActorData, AActor* WorldActor);
}
