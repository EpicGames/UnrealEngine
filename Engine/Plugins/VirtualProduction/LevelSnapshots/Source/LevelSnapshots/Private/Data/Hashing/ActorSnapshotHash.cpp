// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hashing/ActorSnapshotHash.h"

#include "SnapshotCustomVersion.h"

bool FActorSnapshotHash::Serialize(FArchive& Archive)
{
	using namespace UE::LevelSnapshots;
	Archive.UsingCustomVersion(FSnapshotCustomVersion::GUID);

	if (Archive.CustomVer(FSnapshotCustomVersion::GUID) >= FSnapshotCustomVersion::FixActorHashSerialize)
	{
		Archive << MicroSecondsForCrc;
		Archive << MicroSecondsForMD5;
		Archive << Crc32DataLength;
		Archive << MD5DataLength;
		Archive << Crc32;
		Archive << MD5;
		return true;
	}

	// Use tagged property serialization
	return false;
}