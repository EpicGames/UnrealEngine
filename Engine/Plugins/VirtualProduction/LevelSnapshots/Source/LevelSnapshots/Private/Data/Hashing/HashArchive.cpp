// Copyright Epic Games, Inc. All Rights Reserved.

#include "HashArchive.h"

#include "Containers/UnrealString.h"

FArchiveHashBase::FArchiveHashBase()
{
	ArIgnoreOuterRef = true;
	SetIsSaving(true);
}

FArchive& FArchiveHashBase::operator<<(FName& Name)
{
	FString SavedString(Name.ToString());
	*this << SavedString;
	return *this;
}

FMD5Hash FArchiveComputeFullMD5::GetMD5()
{
	FMD5 Hasher;
	Hasher.Update(SerializedData.GetData(), SerializedData.Num());
	FMD5Hash Result;
	Result.Set(Hasher);
	return Result;
}

void FArchiveComputeFullMD5::Serialize(void* Data, int64 Length)
{
	Writer.Serialize(Data, Length);
}

FMD5Hash FArchiveComputeIncrementalMD5::GetMD5()
{
	FMD5Hash Result;
	Result.Set(HashBuilder);
	return Result;
}

void FArchiveComputeIncrementalMD5::Serialize(void* Data, int64 Length)
{
	HashBuilder.Update(static_cast<uint8*>(Data), Length);
}

uint32 FArchiveComputeFullCrc32::GetCrc32() const
{
	return FCrc::MemCrc32(SerializedData.GetData(), SerializedData.Num(), CRC);
}

void FArchiveComputeFullCrc32::Serialize(void* Data, int64 Length)
{
	Writer.Serialize(Data, Length);
}

uint32 FArchiveComputeIncrementalCrc32::GetCrc32() const
{
	return CRC;
}

void FArchiveComputeIncrementalCrc32::Serialize(void* Data, int64 Length)
{
	CRC = FCrc::MemCrc32(Data, Length, CRC);
}
