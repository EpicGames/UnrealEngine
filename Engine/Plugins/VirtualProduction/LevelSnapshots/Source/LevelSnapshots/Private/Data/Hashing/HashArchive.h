// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/SecureHash.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"

struct FMD5Hash;

class FArchiveHashBase : public FArchiveUObject
{
public:

	FArchiveHashBase();

	//~ Begin FArchive Interface
	virtual FArchive& operator<<(class FName& Name);
	//~ End FArchive Interface
};

/**
 * First collects all serialized data, and then computes a Crc32 hash using GetCrc32.
 * 
 * FArchiveComputeFullMD5 and FArchiveComputeIncrementalMD5 produce the same hash. They have roughly the same speed
 * but sometimes one is faster than the other. Profile it if you want to know.
 */
class FArchiveComputeFullMD5 : public FArchiveHashBase
{
public:

	FArchiveComputeFullMD5()
		: FArchiveHashBase()
		, Writer(SerializedData)
	{}

	FMD5Hash GetMD5();
	const TArray<uint8>& GetSerializedData() const { return SerializedData; }

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Length);
	virtual FString GetArchiveName() const override { return TEXT("FArchiveComputeFullMD5"); }
	//~ End FArchive Interface

private:
	
	TArray<uint8> SerializedData;
	FMemoryWriter Writer;
};

/**
 * Updates the MD5 hash with every Serialize call.
 *
 * FArchiveComputeFullMD5 and FArchiveComputeIncrementalMD5 produce the same hash. They have roughly the same speed
 * but sometimes one is faster than the other. Profile it if you want to know.
 */
class FArchiveComputeIncrementalMD5 : public FArchiveHashBase
{
public:

	FMD5Hash GetMD5();

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Length);
	virtual FString GetArchiveName() const override { return TEXT("FArchiveComputeIncrementalMD5"); }
	//~ End FArchive Interface

private:
	
	FMD5 HashBuilder;
};

/**
 * First collects all serialized data, and then computes a Crc32 hash using GetCrc32.
 * 
 * FArchiveComputeFullCrc32 is often a little faster than FArchiveComputeIncrementalCrc32; there are also many cases where it is slower though.
 * Profile it if you're unsure. The difference can be is usually up to 30%.
 * On i9 9100k, average computation time for 'average' actors is 0.2ms to 0.6ms.
 * 
 * Both archives yield the same CRC.
 * This archive has one advantage: before computing CRCs by calling GetCrc32, you can call GetSerializedData().Num() and compare.
 * With this you may avoid having to compute CRCs altogether.
 *
 * Example of flexibility:
 *	bool AreActorsEqual(AActor* FirstActor, AActor* SecondActor)
 *	{
 *		FArchiveComputeFullCrc32 FirstArchive, SecondArchive;
 *		SerializeObjectState::SerializeWithoutObjectName(FirstArchive, FirstActor);
 *		SerializeObjectState::SerializeWithoutObjectName(SecondArchive, SecondActor);
 *		if (FirstArchive.GetSerializedData().Num() != SecondArchive.GetSerializedData().Num())
 *			return false; // Save time by not computing CRC
 *
 *		return FirstArchive.GetCrc32() == SecondArchive.GetCrc32();
 *	} 
 */
class FArchiveComputeFullCrc32 : public FArchiveHashBase
{
public:

	FArchiveComputeFullCrc32(uint32 CRC = 0)
		: FArchiveHashBase()
		, Writer(SerializedData)
		, CRC(CRC)
	{}
	
	uint32 GetCrc32() const;
	const TArray<uint8>& GetSerializedData() const { return SerializedData; }

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Length);
	virtual FString GetArchiveName() const override { return TEXT("FArchiveComputeFullCrc32"); }
	//~ End FArchive Interface

private:

	TArray<uint8> SerializedData;
	FMemoryWriter Writer;
	uint32 CRC = 0;
};

/**
 * Computes a Crc32 hash with every Serialize call.
 *
 * FArchiveComputeFullCrc32 is often a little faster than FArchiveComputeIncrementalCrc32; there are also many cases where it is slower though.
 * Profile it if you're unsure. The difference can be is usually up to 30%.
 * On i9 9100k, average computation time for 'average' actors is 0.2ms to 0.6ms.
 * 
 * Both archives yield the same CRC.
 */
class FArchiveComputeIncrementalCrc32 : public FArchiveHashBase
{
public:

	FArchiveComputeIncrementalCrc32(uint32 CRC = 0)
		: FArchiveHashBase()
		, CRC(CRC)
	{}
	
	uint32 GetCrc32() const;

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Length);
	virtual FString GetArchiveName() const override { return TEXT("FArchiveComputeIncrementalCrc32"); }
	//~ End FArchive Interface

private:

	uint32 CRC = 0;
};
