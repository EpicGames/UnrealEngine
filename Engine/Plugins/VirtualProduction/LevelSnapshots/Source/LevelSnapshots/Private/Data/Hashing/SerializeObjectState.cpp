// Copyright Epic Games, Inc. All Rights Reserved.

#include "SerializeObjectState.h"

#include "Containers/Queue.h"
#include "Serialization/ArchiveProxy.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

namespace
{
	/** Serializes all subobjects */
	class FSubobjectArchive : public FArchiveProxy
	{
	public:

		UObject* SerializedObject;
		
		/** Internal queue of object references awaiting serialization */
		TQueue<UObject*> ObjectsToSerialize;
		
		explicit FSubobjectArchive(FArchive& Archive)
			: FArchiveProxy(Archive)
			, SerializedObject()
		{}

		virtual FArchive& operator<<(FObjectPtr& Object) override
		{
			UObject* UObject = Object.Get();
			return HandleObject(UObject);
		}

		virtual FArchive& operator<<(UObject*& Object) override
		{
			return HandleObject(Object);
		}

		FArchive& HandleObject(UObject*& Object)
		{
			FString UniqueName = GetPathNameSafe(Object);
			*this << UniqueName;
			
			if (Object && Object->IsIn(SerializedObject))
			{
				ObjectsToSerialize.Enqueue(Object);
			}

			return *this;
		}
	};

	void SerializeObjectStateInternal(FSubobjectArchive& Archive, UObject* Root)
	{
		if (Root)
		{
			TSet<UObject*> SerializedObjects;
			Archive.ObjectsToSerialize.Enqueue(Root);
			while (Archive.ObjectsToSerialize.Dequeue(Archive.SerializedObject))
			{
				bool bAlreadyProcessed = false;
				SerializedObjects.Add(Archive.SerializedObject, &bAlreadyProcessed);

				if (!bAlreadyProcessed)
				{
					Archive.SerializedObject->Serialize(Archive);
				}
			}
		}
	}
}

void SerializeObjectState::SerializeWithSubobjects(FArchive& Archive, UObject* Root)
{
	FSubobjectArchive Proxy(Archive);
	SerializeObjectStateInternal(Proxy, Root);
}
