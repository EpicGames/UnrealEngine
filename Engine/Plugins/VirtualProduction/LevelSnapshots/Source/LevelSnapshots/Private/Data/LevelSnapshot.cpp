// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/LevelSnapshot.h"

#include "Data/Hashing/ActorHashUtil.h"
#include "Data/Util/ClassDataUtil.h"
#include "Data/Util/WorldDataUtil.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Restoration/Actor/ActorUtil.h"
#include "LevelSnapshotsSettings.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "Restorability/SnapshotRestorability.h"
#include "SnapshotCustomVersion.h"
#include "SnapshotConsoleVariables.h"
#include "Util/SortedScopedLog.h"
#include "Util/SoftObjectPathUtil.h"

#include "Algo/Accumulate.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "HAL/IConsoleManager.h"
#include "GameFramework/Actor.h"
#include "Misc/ScopeExit.h"
#include "UObject/Package.h"
#if WITH_EDITOR
#include "Editor.h"
#endif
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Logging/MessageLog.h"
#endif

void ULevelSnapshot::ApplySnapshotToWorld(UWorld* TargetWorld, const FPropertySelectionMap& SelectionSet)
{
	SCOPED_SNAPSHOT_CORE_TRACE(ApplyToWorld);
	if (TargetWorld == nullptr || !ensure(IsInGameThread()))
	{
		return;
	}
	
	UE_LOGF(LogLevelSnapshots, Log, "Applying snapshot %ls to world %ls. %ls", *GetPathName(), *TargetWorld->GetPathName(), *GenerateDebugLogInfo());
	UE_CLOGF(MapPath != FSoftObjectPath(TargetWorld), LogLevelSnapshots, Log, "Snapshot was taken for different world called '%ls'", *MapPath.ToString());
	ON_SCOPE_EXIT
	{
		UE_LOGF(LogLevelSnapshots, Log, "Finished applying snapshot");
	};
	
	EnsureWorldInitialised();

	using namespace UE::LevelSnapshots;
	using namespace UE::LevelSnapshots::Private;
	OnPreApplySnapshotDelegate.Broadcast();
	ApplyToWorld(
		SerializedData,
		Cache,
		TargetWorld,
		GetPackage(),
		SelectionSet,
		[this, &SelectionSet](){ FLevelSnapshotsModule::GetInternalModuleInstance().OnPreApplySnapshot({FApplySnapshotSharedParams{ *this, SelectionSet }}); },
		[this, &SelectionSet](){ FLevelSnapshotsModule::GetInternalModuleInstance().OnPostApplySnapshot({ FApplySnapshotSharedParams{ *this, SelectionSet }}); }
		);
	OnPostApplySnapshotDelegate.Broadcast();
}

bool ULevelSnapshot::SnapshotWorld(UWorld* TargetWorld)
{
	SCOPED_SNAPSHOT_CORE_TRACE(SnapshotWorld);
	
	if (!ensure(TargetWorld))
	{
		UE_LOGF(LogLevelSnapshots, Warning, "Unable To Snapshot World as World was invalid");
		return false;
	}

	if (TargetWorld->WorldType != EWorldType::Editor
		&& TargetWorld->WorldType != EWorldType::EditorPreview) // To support tests in editor preview maps
	{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (TargetWorld->IsPlayInEditor())
		{
			FMessageLog("PIE").Warning(
			NSLOCTEXT("LevelSnapshots", "IncompatibleWorlds", "Taking snapshots in PIE is an experimental feature. The snapshot will work in the same PIE session but may no longer work when you start a new PIE session.")
			);
		}
#endif // WITH_EDITOR
		UE_LOGF(LogLevelSnapshots, Warning, "Level snapshots currently only support editors. Snapshots taken in other world types are experimental any may not function as expected.");
	}

	UE::LevelSnapshots::Private::FLevelSnapshotsModule& Module = UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance();
	if (!Module.CanTakeSnapshot({ this }))
	{
		return false;
	}
	Module.OnPreTakeSnapshot().Broadcast({ this, TargetWorld });

	MapPath = TargetWorld;
	CaptureTime = FDateTime::UtcNow();
	SerializedData = UE::LevelSnapshots::Private::SnapshotWorld(TargetWorld);
	// Existing snapshot world actors are now invalid - clear previous cached data
	RecreateSnapshotWorld();
	ClearCache();
	Cache.InitFor(SerializedData);
	// Should we delete preexisting actors from the snapshot world?

	Module.OnPostTakeSnapshot().Broadcast({ this, TargetWorld });

	return true;
}

bool ULevelSnapshot::HasChangedSinceSnapshotWasTaken(AActor* WorldActor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(HasChangedSinceSnapshotWasTaken);
	const FSoftObjectPath ActorPath(WorldActor);

	FActorSnapshotData* SavedActorData = SerializedData.ActorData.Find(ActorPath);
	if (!SavedActorData)
	{
		UE_LOGF(LogLevelSnapshots, Warning, "No data found for actor %ls", *ActorPath.ToString());
		return false;
	}

	// Address issue where unsupported actor were saved. This should be checked regardless though in case project settings were changes since the snapshot was taken.
	if (!UE::LevelSnapshots::Restorability::IsActorDesirableForCapture(WorldActor))
	{
		return false;
	}

	ECachedDiffResult& CachedDiffResult = CachedDiffedActors.FindOrAdd(WorldActor);
	if (CachedDiffResult != ECachedDiffResult::NotInitialized
#if !WITH_EDITOR
		&& bIsDiffCacheEnabled
#endif
	)
	{
		return CachedDiffResult == ECachedDiffResult::HadChanges;
	}

	const bool bHasChanged = [&]()
	{
		// Do not slow down old snapshots by computing hash if they had none saved
		const bool bHasHashInfo = SerializedData.SnapshotVersionInfo.GetSnapshotCustomVersion() >= UE::LevelSnapshots::FSnapshotCustomVersion::ActorHash;
		// If the object is already loaded, there is no point in wasting time and computing the hash
		const bool bNeedsHash = Cache.ActorCache.Find(WorldActor) == nullptr;
		
		if (!bHasHashInfo || !bNeedsHash || !UE::LevelSnapshots::Private::HasMatchingHash(SavedActorData->Hash, WorldActor))
		{
			const TOptional<TNonNullPtr<AActor>> DeserializedActor = GetDeserializedActor(WorldActor);
			UE_CLOGF(!DeserializedActor, LogLevelSnapshots, Warning, "Failed to load snapshot actor %ls. Actor will not be restored.", *WorldActor->GetPathName());
			return DeserializedActor && HasOriginalChangedPropertiesSinceSnapshotWasTaken(DeserializedActor.GetValue(), WorldActor);
		}

		return false;
	}();

	CachedDiffResult = bHasChanged ? ECachedDiffResult::HadChanges : ECachedDiffResult::HadNoChanges;
	return bHasChanged;
}

bool ULevelSnapshot::HasOriginalChangedPropertiesSinceSnapshotWasTaken(AActor* SnapshotActor, AActor* WorldActor)
{
	return UE::LevelSnapshots::Private::HasOriginalChangedPropertiesSinceSnapshotWasTaken(this, SnapshotActor, WorldActor);
}

FString ULevelSnapshot::GetActorLabel(const FSoftObjectPath& OriginalActorPath) const
{
#if WITH_EDITORONLY_DATA
	const FActorSnapshotData* SerializedActor = SerializedData.ActorData.Find(OriginalActorPath);
	if (SerializedActor && !SerializedActor->ActorLabel.IsEmpty())
	{
		return SerializedActor->ActorLabel;
	}
#endif

	return UE::LevelSnapshots::Private::ExtractLastSubobjectName(OriginalActorPath); 
}

TOptional<TNonNullPtr<AActor>> ULevelSnapshot::GetDeserializedActor(const FSoftObjectPath& OriginalActorPath)
{
	EnsureWorldInitialised();
	return UE::LevelSnapshots::Private::GetDeserializedActor(OriginalActorPath, SerializedData, Cache, GetPackage());
}

TOptional<TNonNullPtr<UClass>> ULevelSnapshot::GetActorClass(const FSoftObjectPath& OriginalActorPath)
{
	return UE::LevelSnapshots::Private::GetActorClass(OriginalActorPath, SerializedData);
}

int32 ULevelSnapshot::GetNumSavedActors() const
{
	return SerializedData.ActorData.Num();
}

namespace UE::LevelSnapshots::Private::Internal
{
	void ConditionalBreakOnActor(const FString& NameToSearchFor, const FSoftObjectPath& ActorPath)
	{
		if (!NameToSearchFor.IsEmpty() && ActorPath.ToString().Contains(NameToSearchFor))
		{
			UE_DEBUG_BREAK();
		}
	}
}

void ULevelSnapshot::DiffWorld(UWorld* World, FActorPathConsumer HandleMatchedActor, FActorPathConsumer HandleRemovedActor, FActorConsumer HandleAddedActor)
{
	DiffWorld(World, FActorConsumer::CreateLambda([&HandleMatchedActor](AActor* Actor){ HandleMatchedActor.Execute(Actor); }), HandleRemovedActor, HandleAddedActor);
}

void ULevelSnapshot::DiffWorld(UWorld* World, FActorConsumer HandleMatchedActor, FActorPathConsumer HandleRemovedActor, FActorConsumer HandleAddedActor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(DiffWorld);
	
	if (!ensure(World && HandleMatchedActor.IsBound() && HandleRemovedActor.IsBound() && HandleAddedActor.IsBound()))
	{
		return;
	}
	UE_LOGF(LogLevelSnapshots, Log, "Diffing snapshot %ls in world %ls. %ls", *GetPathName(), *World->GetPathName(), *GenerateDebugLogInfo());
	ON_SCOPE_EXIT
	{
		UE_LOGF(LogLevelSnapshots, Log, "Finished diffing snapshot");
	};

	const FString DebugActorName = UE::LevelSnapshots::ConsoleVariables::CVarBreakOnDiffMatchedActor.GetValueOnAnyThread();

	// Find actors that are not present in the snapshot
	TSet<AActor*> AllActors;
	TSet<FSoftObjectPath> LoadedLevels;
	{
		SCOPED_SNAPSHOT_CORE_TRACE(DiffWorld_FindAllActors);

		
		const int32 NumActorsInWorld = Algo::Accumulate(World->GetLevels(), 0, [](int64 Size, const ULevel* Level){ return Size + Level->Actors.Num(); });
		AllActors.Reserve(NumActorsInWorld);
		for (ULevel* Level : World->GetLevels())
		{
			LoadedLevels.Add(UE::LevelSnapshots::Private::ExtractPathWithoutSubobjects(Level));
			
			for (AActor* ActorInLevel : Level->Actors)
			{
				AllActors.Add(ActorInLevel);

				// Warning: ActorInLevel can be null, e.g. when an actor was just removed from the world (and still in undo buffer)
				if (!IsValid(ActorInLevel))
				{
					continue;
				}
			
				FText ExclusionReason;
				UE::LevelSnapshots::Private::Internal::ConditionalBreakOnActor(DebugActorName, ActorInLevel);
				if (!SerializedData.HasMatchingSavedActor(ActorInLevel) && UE::LevelSnapshots::Restorability::ShouldConsiderNewActorForRemoval(ActorInLevel, &ExclusionReason))
				{
					HandleAddedActor.Execute(ActorInLevel);
				}
				else
				{
					UE_CLOGF(!ExclusionReason.IsEmpty(), LogLevelSnapshots, Warning, "Actor %ls was not considered for removal. Reason: %ls", *ActorInLevel->GetPathName(), *ExclusionReason.ToString());
				}
			}
		}
	}
	


	// Try to find world actors and call appropriate callback
	{
		SCOPED_SNAPSHOT_CORE_TRACE(DiffWorld_IteratorAllActors);
		ULevelSnapshotsSettings* Settings = GetMutableDefault<ULevelSnapshotsSettings>();
		
		const bool bShouldLogDiffWorldTimes = UE::LevelSnapshots::ConsoleVariables::CVarLogTimeDiffingMatchedActors.GetValueOnAnyThread();
		FConditionalSortedScopedLog SortedItems(bShouldLogDiffWorldTimes);
		
		SerializedData.ForEachOriginalActor([this, World, &HandleMatchedActor, &HandleRemovedActor, &HandleAddedActor, &AllActors, &LoadedLevels, Settings, &DebugActorName, &SortedItems](const FSoftObjectPath& OriginalActorPath, const FActorSnapshotData& SavedData)
		{
			const FSoftObjectPath LevelPath = OriginalActorPath.GetAssetPathString();
			if (!LoadedLevels.Contains(LevelPath))
			{
				UE_LOGF(LogLevelSnapshots, Log, "Skipping actor %ls because level %ls is not loaded or does not exist (see Levels window).", *OriginalActorPath.ToString(), *LevelPath.ToString());
				return;
			}
			
			const FSoftClassPath ClassPath = UE::LevelSnapshots::Private::GetClass(SavedData, SerializedData);
			UClass* const ActorClass = ClassPath.TryLoadClass<AActor>();
			if (!ActorClass)
			{
				UE_LOGF(LogLevel, Warning, "Cannot find class %ls. Saved actor %ls will not be restored.", *ClassPath.ToString(), *OriginalActorPath.ToString());
				return;
			}
			
			AActor* ResolvedActor = Cast<AActor>(OriginalActorPath.ResolveObject());
			// OriginalActorPath may still resolve to a live actor if it was just removed. We need to check the ULevel::Actors to see whether it was removed.
			const bool bWasRemovedFromWorld = ResolvedActor == nullptr || !AllActors.Contains(Cast<AActor>(ResolvedActor));
			if (bWasRemovedFromWorld)
			{
				FText ExclusionReason;
				auto GetDeserializedActorFunc = [this, &OriginalActorPath](){ return GetDeserializedActor(OriginalActorPath); };
				const UE::LevelSnapshots::FCanRecreateActorParams Params{ World, ActorClass, OriginalActorPath, SavedData.SerializedActorData.GetObjectFlags(), SerializedData, GetDeserializedActorFunc };
				if (UE::LevelSnapshots::Restorability::ShouldConsiderRemovedActorForRecreation(Params, &ExclusionReason))
				{
					HandleRemovedActor.Execute(OriginalActorPath);
				}
				
				UE_CLOGF(!ExclusionReason.IsEmpty(), LogLevelSnapshots, Warning, "Actor %ls was not considered for recreation. Reason: %ls", *OriginalActorPath.ToString(), *ExclusionReason.ToString());
				return;
			}
			if (Settings->SkippedClasses.SkippedClasses.Contains(ActorClass))
			{
				return;
			}
			
			FText ExclusionReason;
			// Possible scenario: Right-click actor > Replace Selected Actors with; deletes the original and replaces it with new actor.
			if (ResolvedActor->GetClass() != ActorClass)
			{
				HandleRemovedActor.Execute(OriginalActorPath);
				HandleAddedActor.Execute(Cast<AActor>(ResolvedActor));
			}
			else if (UE::LevelSnapshots::Restorability::ShouldConsiderMatchedActorForRestoration({ *ResolvedActor, SerializedData }, &ExclusionReason))
			{
				const FScopedLogItem Log = SortedItems.AddScopedLogItem(OriginalActorPath.ToString());
				UE::LevelSnapshots::Private::Internal::ConditionalBreakOnActor(DebugActorName, OriginalActorPath);
				SCOPED_SNAPSHOT_CORE_TRACE(HandleMatchedActor)
				
				UE_LOGF(LogLevelSnapshots, VeryVerbose, "Matched actor %ls", *OriginalActorPath.ToString());
				HandleMatchedActor.Execute(Cast<AActor>(ResolvedActor));
			}
			
			UE_CLOGF(!ExclusionReason.IsEmpty(), LogLevelSnapshots, Warning, "Actor %ls was skipped for modification. Reason: %ls", *OriginalActorPath.ToString(), *ExclusionReason.ToString());
		});
	}
}

void ULevelSnapshot::SetSnapshotName(const FName& InSnapshotName)
{
	SnapshotName = InSnapshotName;
}

void ULevelSnapshot::SetSnapshotDescription(const FString& InSnapshotDescription)
{
	SnapshotDescription = InSnapshotDescription;
}

void ULevelSnapshot::BeginDestroy()
{
	if (RootSnapshotWorld)
	{
		DestroyWorld();
	}
	
	Super::BeginDestroy();
}

FString ULevelSnapshot::GenerateDebugLogInfo() const
{
	FSnapshotVersionInfo Current;
	Current.Initialize();
	
	return FString::Printf(TEXT("CaptureTime: %s. SnapshotVersionInfo: %s. Current engine version: %s."), *CaptureTime.ToString(), *GetSerializedData().SnapshotVersionInfo.ToString(), *Current.ToString());
}

namespace UE::LevelSnapshots::Private
{
	static UWorld* CreateWorld(UObject* Outer, FName WorldName);
	static TSet<FName> ExtractLevelNames(const FWorldSnapshotData& WorldData);
}

void ULevelSnapshot::EnsureWorldInitialised()
{
	using namespace UE::LevelSnapshots::Private;
	
	if (RootSnapshotWorld == nullptr)
	{
		TransientWorldPackage = CreatePackage(*FString::Printf(TEXT("/Temp/LevelSnapshots/%s/%s"), *GetName(), *FGuid::NewGuid().ToString()));
		TransientWorldPackage->SetFlags(RF_Transient);
		
		const FName RootWorldName = *ExtractPathWithoutSubobjects(MapPath).GetAssetName();
		RootSnapshotWorld = CreateWorld(TransientWorldPackage, RootWorldName);
		
		TSet<FName> Sublevels = ExtractLevelNames(SerializedData);
		Sublevels.Remove(RootWorldName);
		SnapshotSublevels.Reserve(Sublevels.Num());
		for (const FName& SublevelName : Sublevels)
		{
			SnapshotSublevels.Add(CreateWorld(TransientWorldPackage, SublevelName));
		}
		
		// Destroy our temporary world when the editor (or game) world is destroyed. Reasons:
		// 1. After unloading a map checks for world GC leaks; it would fatally crash if we did not clear here.
		// 2. Our temp map stores a "copy" of actors from the original world: the original world is no longer relevant, so neither is our temp world.
		if (ensure(GEngine))
		{
			Handle = GEngine->OnWorldDestroyed().AddLambda([WeakThis = TWeakObjectPtr<ULevelSnapshot>(this)](UWorld* WorldBeingDestroyed)
	        {
				const bool bIsEditorOrGameMap = WorldBeingDestroyed->WorldType == EWorldType::Editor || WorldBeingDestroyed->WorldType == EWorldType::Game;
	            if (ensureAlways(WeakThis.IsValid()) && bIsEditorOrGameMap)
	            {
	                WeakThis->DestroyWorld();
	            }
	        });
		}
		
		Cache.InitFor(SerializedData);
#if WITH_EDITOR
		FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &ULevelSnapshot::ResetDiffCacheToUninitialized);

		// Deleting assets will cause GC to destroy objects in the order they appear in GUnreachableObjects.
		// It's possible that objects in one of our UWorlds appear before the actual UWorlds. So ConditionalBeginDestroy is invoked on objects before the UWorld instance.
		// Problem: Certain objects expect UWorld::CleanupWorld to be called before their own BeginDestroy (e.g. subsystems). Not doing so causes all kinds of failing checks and ensures.
		// Solution: We pre-emptively clean up the UWorld before actual deletion; this triggers important destruction & clean-up callbacks (e.g for subsystems, rendering, etc.).
		// Caveat of solution: This code should probably run with FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy instead. However,
		//	1. when that callback executes, there is no API for knowing whether our LevelSnapshot is in GUnreachableObjects.
		//	2. user code in actors may call CollectGarbage while applying a snapshot. Calling DestroyWorld in that context can be disastrous.
		// Decision: Now (= 5.4.2), the below events are the only known cases where our snapshot is deleted... hence we do not use PreGarbageCollectConditionalBeginDestroy due to lack of querying API.
		// For future reference: If the PreGarbageCollectConditionalBeginDestroy API ever returns the list of objects to be destroyed, then you should
		// replace OnAssetsPreDelete and OnPreDestructiveAssetActionWithArgs with PreGarbageCollectConditionalBeginDestroy.
		FEditorDelegates::OnAssetsPreDelete.AddWeakLambda(this, [this](const TArray<UObject*>& ObjectsAboutToDelete)
		{
			const bool bHasInitializedWorld = RootSnapshotWorld != nullptr;
			if (bHasInitializedWorld && ObjectsAboutToDelete.Contains(this))
			{
				DestroyWorld();
			}
		});
		FEditorDelegates::OnPreDestructiveAssetActionWithArgs.AddWeakLambda(this, [this](FOnPreDestructiveAssetActionArgs& Args)
		{
			const bool bHasInitializedWorld = RootSnapshotWorld != nullptr;
			if (bHasInitializedWorld && Args.Objects.Contains(this))
			{
				DestroyWorld();
			}
		});
#endif
	}

	SerializedData.SnapshotWorld = RootSnapshotWorld;
	SerializedData.SnapshotSublevels = SnapshotSublevels;
}

namespace UE::LevelSnapshots::Private
{
	static void InitializeWorld(UWorld* World)
	{
		World->InitializeNewWorld(UWorld::InitializationValues()
			.InitializeScenes(false)		// This is memory only world: no rendering
			.AllowAudioPlayback(false)
			.RequiresHitProxies(false)		
			.CreatePhysicsScene(false)
			.CreateNavigation(false)
			.CreateAISystem(false)
			.ShouldSimulatePhysics(false)
			.EnableTraceCollision(false)
			.SetTransactional(false)
			.CreateFXSystem(false)
			);
	}
	
	static UWorld* CreateWorld(UObject* Outer, FName WorldName)
	{
		UWorld* World = NewObject<UWorld>(Outer, WorldName);
		World->WorldType = EWorldType::Inactive;

		// Note: Do NOT create a FWorldContext for this world.
		// If you do, the render thread will send render commands every tick (and crash cuz we do not init the scene below).
		InitializeWorld(World);
		return World;
	}

	static TSet<FName> ExtractLevelNames(const FWorldSnapshotData& WorldData)
	{
		TSet<FName> Result;
		for (auto SavedActorIt = WorldData.ActorData.CreateConstIterator(); SavedActorIt; ++SavedActorIt)
		{
			const FName WorldName = *ExtractPathWithoutSubobjects(SavedActorIt->Key).GetAssetName();
			Result.Add(WorldName);
		}
		return Result;
	}
}

void ULevelSnapshot::DestroyWorld()
{
	if (ensureAlwaysMsgf(RootSnapshotWorld, TEXT("World was already destroyed.")))
	{
		if (ensure(GEngine))
		{
			GEngine->OnWorldDestroyed().Remove(Handle);
			Handle.Reset();
		}
#if WITH_EDITOR
		FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
		FEditorDelegates::OnPreDestructiveAssetActionWithArgs.RemoveAll(this);
#endif
		
		CleanUpWorld();
	}
}

void ULevelSnapshot::CleanUpWorld()
{
	SerializedData.SnapshotWorld.Reset();
	SerializedData.SnapshotSublevels.Reset();
	ClearCache();

	// Cleanup world clears RF_Standalone flag...
	const bool bWasStandalone = HasAnyFlags(RF_Standalone);
	
	RootSnapshotWorld->CleanupWorld();
	for (UWorld* Sublevel : SnapshotSublevels)
	{
		Sublevel->CleanupWorld();
	}
	TransientWorldPackage = nullptr;
	RootSnapshotWorld = nullptr;
	SnapshotSublevels.Reset();

	// ... so restore it
	if (bWasStandalone)
	{
		SetFlags(RF_Standalone);
	}
}

void ULevelSnapshot::ClearCache() 
{
	Cache.Reset();
	CachedDiffedActors.Reset(); 
}

void ULevelSnapshot::RecreateSnapshotWorld()
{
	if (RootSnapshotWorld)
	{
		DestroyWorld();
	}
	EnsureWorldInitialised();
}

#if WITH_EDITOR
void ULevelSnapshot::ResetDiffCacheToUninitialized(TArrayView<AActor*> ModifiedActors)
{
	CachedDiffedActors.Reserve(ModifiedActors.Num());
	for (AActor* ModifiedActor : ModifiedActors)
	{
		CachedDiffedActors.FindOrAdd(ModifiedActor) = ECachedDiffResult::NotInitialized;
	}
}

void ULevelSnapshot::ResetDiffCacheToUninitialized(UObject* ModifiedObject)
{
	AActor* AsActor = ModifiedObject->IsA<AActor>() ? Cast<AActor>(ModifiedObject) : ModifiedObject->GetTypedOuter<AActor>(); 
	if (AsActor && SerializedData.HasMatchingSavedActor(AsActor))
	{
		CachedDiffedActors.FindOrAdd(AsActor) = ECachedDiffResult::NotInitialized;
	}
}
#endif
