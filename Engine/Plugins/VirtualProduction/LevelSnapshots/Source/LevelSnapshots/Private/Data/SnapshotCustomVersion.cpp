// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid UE::LevelSnapshots::FSnapshotCustomVersion::GUID(0x3EF5FDBD, 0x21AAAF1F, 0xBCD6F38F, 0xB3A32521);
FCustomVersionRegistration GRegisterSnapshotCustomVersion(UE::LevelSnapshots::FSnapshotCustomVersion::GUID, UE::LevelSnapshots::FSnapshotCustomVersion::LatestVersion, TEXT("LevelSnapshotVersion"));
