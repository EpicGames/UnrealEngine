// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/SnapshotVersion.h"

#include "SnapshotCustomVersion.h"

#include "Misc/EngineVersion.h"
#include "Serialization/CustomVersion.h"

void FSnapshotFileVersionInfo::Initialize()
{
	FileVersionUE4 = GPackageFileUEVersion.FileVersionUE4;
	FileVersionUE5 = GPackageFileUEVersion.FileVersionUE5;
	FileVersionLicensee = GPackageFileLicenseeUEVersion;
}

FString FSnapshotFileVersionInfo::ToString() const
{
	return FString::Printf(TEXT("FileVersionUE4=%d FileVersionUE5=%d Licensee=%d"), FileVersionUE4, FileVersionUE5, FileVersionLicensee);;
}

void FSnapshotEngineVersionInfo::Initialize(const FEngineVersion& InVersion)
{
	Major = InVersion.GetMajor();
	Minor = InVersion.GetMinor();
	Patch = InVersion.GetPatch();
	Changelist = InVersion.GetChangelist();
}

FString FSnapshotEngineVersionInfo::ToString() const
{
	return FString::Printf(TEXT("%u.%u.%u %u"), Major, Minor, Patch, Changelist);
}

void FSnapshotCustomVersionInfo::Initialize(const FCustomVersion& InVersion)
{
	FriendlyName = InVersion.GetFriendlyName();
	Key = InVersion.Key;
	Version = InVersion.Version;
}

void FSnapshotVersionInfo::Initialize(bool bWithoutSnapshotVersion)
{
	FileVersion.Initialize();
	EngineVersion.Initialize(FEngineVersion::Current());

	CustomVersions.Empty(CustomVersions.Num());
	FCustomVersionContainer AllCurrentVersions = FCurrentCustomVersions::GetAll();
	for (const FCustomVersion& EngineCustomVersion : AllCurrentVersions.GetAllVersions())
	{
		FSnapshotCustomVersionInfo& CustomVersion = CustomVersions.AddDefaulted_GetRef();
		if (!bWithoutSnapshotVersion || (bWithoutSnapshotVersion && CustomVersion.Key != UE::LevelSnapshots::FSnapshotCustomVersion::GUID))
		{
			CustomVersion.Initialize(EngineCustomVersion);
		}
	}
}

bool FSnapshotVersionInfo::IsInitialized() const
{
	return CustomVersions.Num() > 0;
}

void FSnapshotVersionInfo::ApplyToArchive(FArchive& Archive) const
{
	FPackageFileVersion UEVersion(FileVersion.FileVersionUE4, (EUnrealEngineObjectUE5Version)FileVersion.FileVersionUE5);

	Archive.SetUEVer(UEVersion);
	Archive.SetLicenseeUEVer(FileVersion.FileVersionLicensee);
	Archive.SetEngineVer(FEngineVersionBase(EngineVersion.Major, EngineVersion.Minor, EngineVersion.Patch, EngineVersion.Changelist));

	FCustomVersionContainer EngineCustomVersions;
	for (const FSnapshotCustomVersionInfo& CustomVersion : CustomVersions)
	{
		EngineCustomVersions.SetVersion(CustomVersion.Key, CustomVersion.Version, CustomVersion.FriendlyName);
	}
	Archive.SetCustomVersions(EngineCustomVersions);
}

FString FSnapshotVersionInfo::ToString() const
{
	return FString::Printf(TEXT("%s %s"), *EngineVersion.ToString(), *FileVersion.ToString());
}

int32 FSnapshotVersionInfo::GetSnapshotCustomVersion() const
{
	for (const FSnapshotCustomVersionInfo& CustomVersion : CustomVersions)
	{
		if (CustomVersion.Key == UE::LevelSnapshots::FSnapshotCustomVersion::GUID)
		{
			return CustomVersion.Version;
		}
	}

	return INDEX_NONE;
}
