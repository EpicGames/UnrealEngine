// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassDataUtil.h"

#include "ActorSnapshotData.h"
#include "ClassDefaults/ApplyClassDefaulDataArchive.h"
#include "ClassDefaults/TakeClassDefaultObjectSnapshotArchive.h"
#include "Data/WorldSnapshotData.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "SnapshotUtilTypes.h"
#include "SubobjectSnapshotData.h"

#include "Algo/ForEach.h"
#include "EngineUtils.h"
#include "SnapshotDataCache.h"
#include "Templates/NonNullPointer.h"
#include "UObject/Package.h"

namespace UE::LevelSnapshots::Private
{
	TOptional<TNonNullPtr<UObject>> GetSubobjectArchetype(FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo)
	{
		if (Cache.ArchetypeObjects[ClassIndex])
		{
			return { Cache.ArchetypeObjects[ClassIndex].Get() };
		}
		
		const TOptional<TNonNullPtr<FClassSnapshotData>> ArchetypeData = GetObjectArchetypeData(WorldData, ClassIndex, Cache, FallbackInfo);
		if (!ArchetypeData)
		{
			return {};
		}

		UClass* Class = (*ArchetypeData)->ClassPath.TryLoadClass<UObject>();
		if (!Class)
		{
			return {};
		}
		
		UObject* Archetype = NewObject<UObject>(
			GetTransientPackage(),
			Class,
			*FString("SnapshotArchetype_").Append(*MakeUniqueObjectName(GetTransientPackage(), Class).ToString())
			);
		FApplyClassDefaulDataArchive::SerializeClassDefaultObject(*ArchetypeData.GetValue(), WorldData, Archetype);
		Cache.ArchetypeObjects[ClassIndex] = Archetype;
		
		return { Archetype };
	}

	TOptional<TNonNullPtr<FClassSnapshotData>> GetObjectArchetypeData(FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo)
	{
		if (!ensureMsgf(WorldData.ClassData.IsValidIndex(ClassIndex), TEXT("This data was supposed to be saved when the snapshot was taken. Data is corrupt.")))
		{
			UE_LOGF(LogLevelSnapshots, Error, "Snapshot data is corrupt. ClassIndex %d is out of bounds for ClassData of size %d.", ClassIndex, WorldData.ClassData.Num());
			return {};
		}

		// Normal case: the data was saved
		FClassSnapshotData& SavedClassData = WorldData.ClassData[ClassIndex];
		const bool bHasArchetypeData = SavedClassData.SerializedData.Num() != 0
			&& (SavedClassData.SnapshotFlags & ESnapshotClassFlags::SerializationSkippedArchetypeData) == ESnapshotClassFlags::NoFlags;
		if (bHasArchetypeData)
		{
			return { &SavedClassData };
		}

		// This is an old snapshot (or snapshot config prevented data from being saved)
			// 1. Since serialization is expensive, look it up in the fallback cache
		TOptional<FClassSnapshotData>& FallbackData = Cache.FallbackArchetypeData[ClassIndex];
		if (FallbackData.IsSet())
		{
			return { &FallbackData.GetValue() };
		}

		UClass* Class = SavedClassData.ClassPath.TryLoadClass<UObject>();
		if (!Class)
		{
			UE_LOGF(LogLevelSnapshots, Error, "Failed to load class %ls", *SavedClassData.ClassPath.ToString());
			return {};
		}
		
		// 2. First time encountering this class, get the real archetype
		UObject* RealArchetype = UObject::GetArchetypeFromRequiredInfo(Class, FallbackInfo.SubobjectOuter, FallbackInfo.SubobjectName, FallbackInfo.SubobjectFlags);
		if (ensure(RealArchetype))
		{
			FallbackData = FClassSnapshotData(); // Optional needs to be assigned
			FTakeClassDefaultObjectSnapshotArchive::SaveClassDefaultObject(FallbackData.GetValue(), WorldData, RealArchetype);
			return { &FallbackData.GetValue() };
		}

		UE_LOGF(LogLevelSnapshots, Error, "Failed to determine archetype for class: %ls, outer: %ls name: %ls", *SavedClassData.ClassPath.ToString(), *FallbackInfo.SubobjectOuter->GetPathName(), *FallbackInfo.SubobjectName.ToString())
		return {};
	}

	void SerializeClassDefaultsIntoActor(AActor* Actor, FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache)
	{
		if (WorldData.ClassData.IsValidIndex(ClassIndex))
		{
			SerializeClassDefaultsInto(Actor, WorldData.ClassData[ClassIndex], WorldData);
		}
		else
		{
			UE_LOGF(LogLevelSnapshots, Warning,
				"No CDO saved for class '%ls'. If you changed some class default values for this class, then the affected objects will have the latest values instead of the class defaults at the time the snapshot was taken. Should be nothing major to worry about.",
				*Actor->GetClass()->GetName()
				);
		}
	}

	void SerializeClassDefaultsIntoSubobject(UObject* Object, FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo)
	{
		if (const TOptional<TNonNullPtr<FClassSnapshotData>> ArchetypeData = GetObjectArchetypeData(WorldData, ClassIndex, Cache, FallbackInfo))
		{
			SerializeClassDefaultsInto(Object, **ArchetypeData, WorldData);
		}
		else
		{
			UE_LOGF(LogLevelSnapshots, Warning,
				"No CDO saved for class '%ls'. If you changed some class default values for this class, then the affected objects will have the latest values instead of the class defaults at the time the snapshot was taken. Should be nothing major to worry about.",
				*Object->GetClass()->GetName()
				);
		}
	}

	void SerializeClassDefaultsInto(UObject* Object, FClassSnapshotData& DataToSerialize, FWorldSnapshotData& WorldData)
	{
		const bool bHasData = DataToSerialize.SerializedData.Num() > 0 && (DataToSerialize.SnapshotFlags & ESnapshotClassFlags::SerializationSkippedArchetypeData) == ESnapshotClassFlags::NoFlags;
		if (bHasData && !FLevelSnapshotsModule::GetInternalModuleInstance().ShouldSkipClassDefaultSerialization(Object->GetClass()))
		{
			FApplyClassDefaulDataArchive::RestoreChangedClassDefaults(DataToSerialize, WorldData, Object);
		}
	}
	
	void SerializeSelectedClassDefaultsIntoSubobject(UObject* Object, FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo, const FPropertySelection& PropertiesToRestore)
	{
		if (const TOptional<TNonNullPtr<FClassSnapshotData>> ArchetypeData = GetObjectArchetypeData(WorldData, ClassIndex, Cache, FallbackInfo))
		{
			FApplyClassDefaulDataArchive::RestoreSelectedChangedClassDefaults(**ArchetypeData, WorldData, Object, PropertiesToRestore);
		}
		else
		{
			UE_LOGF(LogLevelSnapshots, Warning,
				"No CDO saved for class '%ls'. If you changed some class default values for this class, then the affected objects will have the latest values instead of the class defaults at the time the snapshot was taken. Should be nothing major to worry about.",
				*Object->GetClass()->GetName()
				);
		}
	}

	FSoftClassPath GetClass(const FActorSnapshotData& Data, const FWorldSnapshotData& WorldData)
	{
		return WorldData.ClassData[Data.ClassIndex].ClassPath;
	}
	
	FSoftClassPath GetClass(const FSubobjectSnapshotData& Data, const FWorldSnapshotData& WorldData)
	{
		return WorldData.ClassData[Data.ClassIndex].ClassPath;
	}
	
	FClassDataIndex AddClassArchetype(FWorldSnapshotData& WorldData, UObject* SavedObject)
	{
		check(SavedObject);
		checkf(!SavedObject->IsA<UClass>(), TEXT("You are supposed to put in the original object, not its class."));

		UObject* Archetype = SavedObject->GetArchetype();
		if (const FClassDataIndex* Index = WorldData.ArchetypeToClassDataIndex.Find(Archetype))
		{
			return *Index;
		}
		
		UClass* Class = SavedObject->GetClass();
		FClassSnapshotData ClassData;
		ClassData.ClassPath = Class;
		ClassData.ClassFlags = Class->GetFlags();
		// Reserve an index and init cache now because we call SaveClassDefaultObject below
		// which by calling AddClassArchetype can reallocate ClassData and ClassToClassDataIndex...
		const int32 Index = WorldData.ClassData.Add(ClassData);
		WorldData.ArchetypeToClassDataIndex.Add(Archetype, Index);
		
		const bool bShouldSkip = FLevelSnapshotsModule::GetInternalModuleInstance().ShouldSkipClassDefaultSerialization(Class);
		if (bShouldSkip)
		{
			ClassData.SnapshotFlags = ESnapshotClassFlags::SerializationSkippedArchetypeData;
		}
		else
		{
			FTakeClassDefaultObjectSnapshotArchive::SaveClassDefaultObject(ClassData, WorldData, Archetype);
		}
		
		// ... now we can write the serialized data safely
		WorldData.ClassData[Index] = MoveTemp(ClassData);
		return Index;
	}
}