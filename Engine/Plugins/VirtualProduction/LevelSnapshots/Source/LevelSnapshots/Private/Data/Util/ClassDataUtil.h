// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SnapshotUtilTypes.h"
#include "Templates/NonNullPointer.h"
#include "UObject/ObjectMacros.h"

struct FPropertySelection;
class AActor;
class UObject;
struct FActorSnapshotData;
struct FClassSnapshotData;
struct FComponentSnapshotData;
struct FObjectSnapshotData;
struct FPropertySelectionMap;
struct FSnapshotDataCache;
struct FSoftClassPath;
struct FSubobjectSnapshotData;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/**
	 * Gives info to retrieve class defaults for a subobject when it was not saved.
	 * 
	 * Remember: Blueprints create new archetype objects for each component so the lookup depends on the context.
	 * These are basically just the args for UObject::GetArchetypeFromRequiredInfo.
	 */
	struct FSubobjectArchetypeFallbackInfo
	{
		UObject* SubobjectOuter;
		FName SubobjectName;
		EObjectFlags SubobjectFlags;
	};
	
	/** Gets the archetype to use for constructing a subobject.*/
	TOptional<TNonNullPtr<UObject>> GetSubobjectArchetype(FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo);
	
	/** Gets an archetype that was saved when the snapshot was taken. Each subobject in actors can have their own archetypes. */
	TOptional<TNonNullPtr<FClassSnapshotData>> GetObjectArchetypeData(FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo);

	/** Retrieves and serializes class default data into the given actor, if any was saved in the snapshot. */
	void SerializeClassDefaultsIntoActor(AActor* Actor, FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache);
	/** Retrieves and serializes class default data into the given actor, if any was saved in the snapshot. */
	void SerializeClassDefaultsIntoSubobject(UObject* Object, FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo);
	/** Helper that serializes already retrieved class default data into an object. Checks whether this feature is enabled for this class. */
	void SerializeClassDefaultsInto(UObject* Object, FClassSnapshotData& DataToSerialize, FWorldSnapshotData& WorldData);

	/** Serializes the class default data into the given subobject but only those allowed by PropertiesToRestore. */
	void SerializeSelectedClassDefaultsIntoSubobject(UObject* Object, FWorldSnapshotData& WorldData, FClassDataIndex ClassIndex, FSnapshotDataCache& Cache, const FSubobjectArchetypeFallbackInfo& FallbackInfo, const FPropertySelection& PropertiesToRestore);

	/** Helper for getting an actor's class. */
	FSoftClassPath GetClass(const FActorSnapshotData& Data, const FWorldSnapshotData& WorldData);
	/** Helper for getting an actor's subobject's class. */
	FSoftClassPath GetClass(const FSubobjectSnapshotData& Data, const FWorldSnapshotData& WorldData);
	
	/** Adds a class default entry in the world data */
	FClassDataIndex AddClassArchetype(FWorldSnapshotData& WorldData, UObject* SavedObject);
}