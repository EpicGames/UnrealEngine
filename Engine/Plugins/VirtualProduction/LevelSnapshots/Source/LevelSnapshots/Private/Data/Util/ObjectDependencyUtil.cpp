// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectDependencyUtil.h"

#include "Archive/ApplySnapshotToEditorArchive.h"
#include "Archive/LoadSnapshotObjectArchive.h"
#include "Archive/TakeWorldObjectSnapshotArchive.h"
#include "ClassDataUtil.h"
#include "LevelSnapshotsLog.h"
#include "Restoration/Actor/ActorUtil.h"
#include "SnapshotRestorability.h"
#include "SubobjectSnapshotData.h"
#include "Util/SoftObjectPathUtil.h"
#include "WorldSnapshotData.h"

#include "Components/ActorComponent.h"
#include "EngineUtils.h"
#include "ObjectTrace.h"
#include "SnapshotDataCache.h"
#include "GameFramework/Actor.h"
#include "Templates/NonNullPointer.h"

namespace UE::LevelSnapshots::Private::Internal
{
	static UObject* ResolveExternalReference(const FSoftObjectPath& ObjectPath)
	{
		UObject* ExternalReference = ObjectPath.ResolveObject();
		const bool bNeedsToLoadFromDisk = ExternalReference == nullptr;
		if (bNeedsToLoadFromDisk)
		{
			ExternalReference = ObjectPath.TryLoad();
		}
		
		// We're supposed to be dealing with an external (asset) reference, e.g. a UMaterial.
		// Note that Blueprints contain AActors and they are referenced by UChildActorComponent.
		ensureMsgf(ExternalReference == nullptr || !ExternalReference->IsInA(UWorld::StaticClass()), TEXT("Something is wrong. We just checked that the reference is not a world object but it is an actor or component."));
		return ExternalReference;
	}
	
	static UObject* CreateSubobjectForSnapshotWorld(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, int32 ObjectPathIndex, const FSoftObjectPath& SnapshotPathToSubobject, const FProcessObjectDependency& ProcessObjectDependency, const FString& LocalisationNamespace)
	{
		FSubobjectSnapshotData* SubobjectData = WorldData.Subobjects.Find(ObjectPathIndex);
		if (!SubobjectData || SubobjectData->bWasSkippedClass)
		{
			return nullptr;
		}
		if (const FSubobjectSnapshotCache* SubobjectCache = Cache.SubobjectCache.Find(SnapshotPathToSubobject); SubobjectCache && SubobjectCache->SnapshotObject)
		{
			return SubobjectCache->SnapshotObject.Get();
		}
		
		const int32 OuterIndex = SubobjectData->OuterIndex;
		check(OuterIndex != ObjectPathIndex);
		UObject* SubobjectOuter = ResolveObjectDependencyForSnapshotWorld(WorldData, Cache, OuterIndex, ProcessObjectDependency, LocalisationNamespace);
		if (!SubobjectOuter)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Failed to create '%ls' because its outer could not be created.", *SnapshotPathToSubobject.ToString());
			return nullptr;
		}
	
		const FName SubobjectName = *ExtractLastSubobjectName(SnapshotPathToSubobject);
		const bool bSubobjectNameIsTaken = [SubobjectOuter, SubobjectName]()
		{
			TArray<UObject*> Subobjects;
			GetObjectsWithOuter(SubobjectOuter, Subobjects, EGetObjectsFlags::IncludeNestedObjects);
			for (UObject* Object : Subobjects)
			{
				if (Object->GetFName() == SubobjectName)
				{
					return true;
				}
			}
			return false;
		}();
		if (bSubobjectNameIsTaken)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Failed to create '%ls' because subobject name was already taken.", *SnapshotPathToSubobject.ToString());
			return nullptr;
		}
		
		const FSubobjectArchetypeFallbackInfo Fallback{ SubobjectOuter, SubobjectName, SubobjectData->GetObjectFlags() };
		TOptional<TNonNullPtr<UObject>> Archetype = GetSubobjectArchetype(WorldData, SubobjectData->ClassIndex, Cache, Fallback);
		if (!Archetype)
		{
			const FSoftClassPath ClassPath = GetClass(*SubobjectData, WorldData);
			UE_LOGF(LogLevelSnapshots, Warning, "Failed to create '%ls' because class was '%ls' not found. Maybe it was removed?", *SnapshotPathToSubobject.ToString(), *ClassPath.ToString());
			return nullptr;
		}
		
		const UClass* Class = (*Archetype)->GetClass();
		UObject* Subobject = NewObject<UObject>(SubobjectOuter, Class, SubobjectName, SubobjectData->GetObjectFlags(), Archetype->Get());
		FSubobjectSnapshotCache& SubobjectCache = Cache.SubobjectCache.FindOrAdd(WorldData.SerializedObjectReferences[ObjectPathIndex]);
		SubobjectCache.SnapshotObject = Subobject;
		FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(*SubobjectData, WorldData, Cache, Subobject, ProcessObjectDependency, LocalisationNamespace)	;
		return Subobject;
	}
	
	static UObject* SerializeOrCreateSubobjectForEditorWorld(FWorldSnapshotData& WorldData, FSubobjectSnapshotData& SubobjectData, FSnapshotDataCache& Cache, UObject* ResolvedObject, const FSoftObjectPath& OriginalObjectPath, const FString& LocalisationNamespace, const FPropertySelectionMap& SelectionMap)
	{
		using namespace UE::LevelSnapshots::Private;

		if (SubobjectData.bWasSkippedClass)
		{
			return !IsValid(ResolvedObject) || ResolvedObject->IsUnreachable() ? nullptr : ResolvedObject;
		}

		FSubobjectSnapshotCache* SubobjectCache = Cache.SubobjectCache.Find(OriginalObjectPath);
		if (!ensureMsgf(SubobjectCache, TEXT("Subobject should already have been cached when snapshot world was created. Investigate.")))
		{
			return nullptr;
		}
		if (SubobjectCache->EditorObject.IsValid())
		{
			return SubobjectCache->EditorObject.Get();
		}

		const FSoftClassPath ClassPath = GetClass(SubobjectData, WorldData);
		const UClass* ExpectedClass = ClassPath.ResolveClass();
		if (!ExpectedClass)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Class '%ls' could not be resolved.", *ClassPath.ToString());
			return nullptr;
		}
		// Suppose ActorA and ActorB. ActorA is being serialized before ActorB. ActorA has a reference to a component in ActorB.
		// That component is instanced and was not recreated yet because, unluckily, we're serializing ActorA before ActorB.
		if (ExpectedClass->IsChildOf(UActorComponent::StaticClass()))
		{
			UE_LOGF(LogLevelSnapshots, Error, "Object depends on a component that must be recreated and is owned by another actor. Support for this is not yet implemented.");
			return nullptr;
		}
		
		// Need to clean rename dead objects otherwise we'll get a name clash when instantiating new object
		if (ResolvedObject)
		{
			const bool bIsReferencingDeadObject = !IsValid(ResolvedObject) || ResolvedObject->IsUnreachable();
			const bool bIsReferencingDifferentClass = ResolvedObject && !ResolvedObject->GetClass()->IsChildOf(ExpectedClass);
			if (bIsReferencingDeadObject || bIsReferencingDifferentClass)
			{
				const FName NewName = MakeUniqueObjectName(ResolvedObject->GetOuter(), ExpectedClass, *ResolvedObject->GetName().Append(TEXT("_TRASH")));
				ResolvedObject->Rename(*NewName.ToString());
				ResolvedObject = nullptr;
			}
		}

		if (!ensureMsgf(SubobjectCache->SnapshotObject, TEXT("Subobject pointer should have been set. Investigate.")))
		{
			return ResolvedObject;
		}
		
		UObject* SnapshotVersion = SubobjectCache->SnapshotObject.Get();
		if (ResolvedObject)
		{
			SubobjectCache->EditorObject = ResolvedObject;
			
			// We can't tell whether ResolvedObject is 1: a normal referenced object or 2: a dead object that just has not been collected yet
			// If it is referenced from somewhere (case 1), we expect it to be in the selection map.
			// Otherwise we'll treat it as a dead object that we fully restore.
			if (SelectionMap.IsSubobjectMarkedForReferenceRestorationOnly(OriginalObjectPath))
			{
				// No serialization is required. Just return it.
			}
			else if (SelectionMap.GetObjectSelection(OriginalObjectPath).GetPropertySelection() != nullptr)
			{
				FApplySnapshotToEditorArchive::ApplyToExistingEditorWorldObject(SubobjectData, WorldData, Cache, ResolvedObject, SnapshotVersion, SelectionMap, SubobjectData.ClassIndex);
			}
			else
			{
				// Reuse existing instance but treat it like it was just freshly allocated
				FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithArchetype(SubobjectData, WorldData, Cache, ResolvedObject, SelectionMap);
			}
		}
		else 
		{
			UObject* Outer = ResolveObjectDependencyForEditorWorld(WorldData, Cache, SubobjectData.OuterIndex, LocalisationNamespace, SelectionMap);
			if (!Outer)
			{
				UE_LOGF(LogLevelSnapshots, Error, "Failed to create object '%ls' because its outer '%ls' could not be resolved", *OriginalObjectPath.ToString(), *WorldData.SerializedObjectReferences[SubobjectData.OuterIndex].ToString());
				return nullptr;
			}
			
			const FName ObjectName = *ExtractLastSubobjectName(OriginalObjectPath);
			const FSubobjectArchetypeFallbackInfo Fallback{ Outer, ObjectName, SubobjectData.GetObjectFlags() };
			const TOptional<TNonNullPtr<UObject>> Archetype = GetSubobjectArchetype(WorldData, SubobjectData.ClassIndex, Cache, Fallback);
			if (!Archetype)
			{
				UE_LOGF(LogLevelSnapshots, Error, "Failed to create object '%ls' because its archetype could not be resolved", *OriginalObjectPath.ToString());
				return nullptr;	
			}
			
			// Must set RF_Transactional so object creation is transacted
			ResolvedObject = NewObject<UObject>(Outer, ExpectedClass, *ExtractLastSubobjectName(OriginalObjectPath), SubobjectData.GetObjectFlags() | RF_Transactional, Archetype.GetValue());
			if (ensureMsgf(ResolvedObject, TEXT("Failed to allocate '%s'"), *OriginalObjectPath.ToString()))
			{
				SubobjectCache->EditorObject = ResolvedObject;
				ResolvedObject->SetFlags(SubobjectData.GetObjectFlags());
				FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithArchetype(SubobjectData, WorldData, Cache, ResolvedObject, SelectionMap);
			}
		}
		
		return ResolvedObject;
	}

	static void AddSubobjectDependencyInternal(FWorldSnapshotData& WorldData, UObject* ReferenceFromOriginalObject, int32 ObjectIndex)
	{
		if (!Restorability::IsSubobjectDesirableForCapture(ReferenceFromOriginalObject))
		{
			WorldData.Subobjects.Add(ObjectIndex, FSubobjectSnapshotData::MakeSkippedSubobjectData());
			return;
		}
		
		const bool bNeedsToSaveSubobject = !WorldData.Subobjects.Contains(ObjectIndex);
		if (bNeedsToSaveSubobject)
		{
			// Avoid infinite recursion
			WorldData.Subobjects.Add(ObjectIndex);
			
			// Owning actor keeps reference to saved subobjects so they can be recreated faster when snapshot is loaded
			if (!ReferenceFromOriginalObject->IsA<UActorComponent>())
			{
				AActor* OwningActor = ReferenceFromOriginalObject->GetTypedOuter<AActor>();
				check(OwningActor);
				WorldData.ActorData.FindOrAdd(OwningActor).OwnedSubobjects.Add(ObjectIndex);
			}

			
			// Important: first allocate SubobjectData on stack... 
			FSubobjectSnapshotData SubobjectData; 
			SubobjectData.ClassIndex = AddClassArchetype(WorldData, ReferenceFromOriginalObject);
			SubobjectData.OuterIndex = AddObjectDependency(WorldData, ReferenceFromOriginalObject->GetOuter());
			// ... because serialisation recursively calls this function. FWorldSnapshotData::Subobjects is possibly reallocated
			FTakeWorldObjectSnapshotArchive::TakeSnapshot(SubobjectData, WorldData, ReferenceFromOriginalObject);
			
			WorldData.Subobjects[ObjectIndex] = MoveTemp(SubobjectData); // MoveTemp not profiled
		}
	}
	
	static void EnsureSubobjectIsSerialized(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UObject* ExistingObject, int32 ObjectPathIndex, const FProcessObjectDependency& ProcessObjectDependency, const FString& LocalisationNamespace)
	{
		FSubobjectSnapshotData* SubobjectData = WorldData.Subobjects.Find(ObjectPathIndex);
		if (!SubobjectData || SubobjectData->bWasSkippedClass)
		{
			return;
		}

		const FSoftClassPath ClassPath = GetClass(*SubobjectData, WorldData);
		if (ClassPath.ResolveClass() != ExistingObject->GetClass())
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Skipping serialisation of subobject because classes are different. Snapshot object '%ls' will not contain values saved in snapshot.", *ExistingObject->GetName());
			return;
		}

		FSubobjectSnapshotCache& SubobjectCache = Cache.SubobjectCache.FindOrAdd(WorldData.SerializedObjectReferences[ObjectPathIndex]);
		if (!SubobjectCache.SnapshotObject)
		{
			SubobjectCache.SnapshotObject = ExistingObject;
			FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(*SubobjectData, WorldData, Cache, ExistingObject, ProcessObjectDependency, LocalisationNamespace);
		}
	}
	
	static int32 AddOrFindObjectReference(FWorldSnapshotData& WorldData, const FSoftObjectPath& ObjectPath)
	{
		if (const int32* ExistingIndex = WorldData.ReferenceToIndex.Find(ObjectPath))
		{
			return *ExistingIndex;
		}
		
		const int32 Index = WorldData.SerializedObjectReferences.Add(ObjectPath);
		WorldData.ReferenceToIndex.Add(ObjectPath, Index);
		return Index;
	}
}

UObject* UE::LevelSnapshots::Private::ResolveObjectDependencyForSnapshotWorld(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, int32 ObjectPathIndex, const FProcessObjectDependency& ProcessObjectDependency, const FString& LocalisationNamespace)
{
	if (!ensure(WorldData.SerializedObjectReferences.IsValidIndex(ObjectPathIndex)))
	{
		return nullptr;
	}
	
	const FSoftObjectPath& OriginalObjectPath = WorldData.SerializedObjectReferences[ObjectPathIndex];
	bool bIsPathToActorSubobject;
	const TOptional<FSoftObjectPath> PathToActor = ExtractActorFromPath(OriginalObjectPath, bIsPathToActorSubobject);
	
	const bool bIsExternalAssetReference = !PathToActor.IsSet();
	if (bIsExternalAssetReference)
	{
		return Internal::ResolveExternalReference(OriginalObjectPath);
	}

	const TOptional<TNonNullPtr<AActor>> Result = GetPreallocated(*PathToActor, WorldData, Cache);
	if (!Result.IsSet())
	{
		return nullptr;
	}

	// Reference to actor
	AActor* SnapshotActor = Result.GetValue();
	const bool bIsPathToActor = !bIsPathToActorSubobject; 
	if (bIsPathToActor)
	{
		return SnapshotActor;
	}

	// Reference to default subobjects, such as components, or an object we already allocated.
	const FSoftObjectPath SnapshotPathToSubobject = UE::LevelSnapshots::Private::SetActorInPath(SnapshotActor, OriginalObjectPath);
	if (UObject* ExistingSubobject = SnapshotPathToSubobject.ResolveObject())
	{
		Internal::EnsureSubobjectIsSerialized(WorldData, Cache, ExistingSubobject, ObjectPathIndex, ProcessObjectDependency, LocalisationNamespace);
		return ExistingSubobject;
	}

	// The subobject is instanced: recreate it recursively
	return Internal::CreateSubobjectForSnapshotWorld(WorldData, Cache, ObjectPathIndex, SnapshotPathToSubobject, ProcessObjectDependency, LocalisationNamespace);
}

UObject* UE::LevelSnapshots::Private::ResolveObjectDependencyForEditorWorld(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, int32 ObjectPathIndex, const FString& LocalisationNamespace, const FPropertySelectionMap& SelectionMap)
{
	if (!ensure(WorldData.SerializedObjectReferences.IsValidIndex(ObjectPathIndex)))
	{
		return nullptr;
	}
	
	const FSoftObjectPath& OriginalObjectPath = WorldData.SerializedObjectReferences[ObjectPathIndex];
	bool bIsPathToActorSubobject;
	const TOptional<TNonNullPtr<FActorSnapshotData>> SnapshotData = FindSavedActorDataUsingObjectPath(WorldData.ActorData, OriginalObjectPath, bIsPathToActorSubobject);
	
	// Dealing with an external (asset) reference, e.g. a UMaterial?
	const bool bIsExternalAssetReference = !SnapshotData.IsSet();
	if (bIsExternalAssetReference)
	{
		return Internal::ResolveExternalReference(OriginalObjectPath);
	}

	// Can return immediately for actors and components: their serialization is handled by FActorSnapshotData.
	UObject* ResolvedObject = OriginalObjectPath.ResolveObject();
	if (ResolvedObject && (ResolvedObject->IsA<AActor>() || ResolvedObject->IsA<UActorComponent>()))
	{
		// Might resolve to a dead object which still exists in memory
		return IsValid(ResolvedObject) ? ResolvedObject : nullptr;
	}
	
	if (FSubobjectSnapshotData* const SubobjectData = WorldData.Subobjects.Find(ObjectPathIndex))
	{
		return Internal::SerializeOrCreateSubobjectForEditorWorld(WorldData, *SubobjectData, Cache, ResolvedObject, OriginalObjectPath, LocalisationNamespace, SelectionMap);
	}
	
	// This should not happen after release of 4.27. This is error may happen if certain snapshots in an older data format are applied because the data wasn't yet captured.
	UE_LOGF(LogLevelSnapshots, Warning, "Trying to resolve a subobject but found no associated object data.");
	return ResolvedObject;
}

UObject* UE::LevelSnapshots::Private::ResolveObjectDependencyForClassDefaultObject(FWorldSnapshotData& WorldData, int32 ObjectPathIndex, UObject* CurrentValue)
{
	if (ensure(WorldData.SerializedObjectReferences.IsValidIndex(ObjectPathIndex)))
	{
		const FSoftObjectPath& OriginalObjectPath = WorldData.SerializedObjectReferences[ObjectPathIndex];
		return Internal::ResolveExternalReference(OriginalObjectPath);
	}
	return nullptr;
}

int32 UE::LevelSnapshots::Private::AddObjectDependency(FWorldSnapshotData& WorldData, UObject* ReferenceFromOriginalObject, bool bCheckWhetherSubobject)
{
	// Even if FSnapshotRestorability::IsSubobjectDesirableForCapture later returns false for this object, we want to track it
	const int32 Result = Internal::AddOrFindObjectReference(WorldData, ReferenceFromOriginalObject);

	const AActor* OwningActor = bCheckWhetherSubobject && ReferenceFromOriginalObject ? ReferenceFromOriginalObject->GetTypedOuter<AActor>() : nullptr;
	if (OwningActor
		// This handles the rare case that ReferenceFromOriginalObject is a reference to a subobject of a CDO actor or in a Blueprint.
		// In that case, ReferenceFromOriginalObject is not actually a real world object, hence not a subobject we want to save.
		&& !OwningActor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		Internal::AddSubobjectDependencyInternal(WorldData, ReferenceFromOriginalObject, Result);
	}
	
	return Result;
}

int32 UE::LevelSnapshots::Private::AddCustomSubobjectDependency(FWorldSnapshotData& WorldData, UObject* ReferenceFromOriginalObject)
{
	if (!ensure(ReferenceFromOriginalObject))
	{
		return INDEX_NONE;
	}

	const int32 SubobjectIndex = Internal::AddOrFindObjectReference(WorldData, ReferenceFromOriginalObject);
	if (WorldData.CustomSubobjectSerializationData.Find(SubobjectIndex) == nullptr)
	{
		WorldData.CustomSubobjectSerializationData.Add(SubobjectIndex);
	}
	else
	{
		UE_LOGF(LogLevelSnapshots, Warning, "Object %ls was already added as dependency. Investigate", *ReferenceFromOriginalObject->GetName());
		UE_DEBUG_BREAK();
	}
		
	return SubobjectIndex;
}

FCustomSerializationData* UE::LevelSnapshots::Private::FindCustomSubobjectData(FWorldSnapshotData& WorldData,const FSoftObjectPath& ReferenceFromOriginalObject)
{
	const int32 SubobjectIndex = WorldData.SerializedObjectReferences.Find(ReferenceFromOriginalObject);
	return WorldData.CustomSubobjectSerializationData.Find(SubobjectIndex);
}

const FCustomSerializationData* UE::LevelSnapshots::Private::FindCustomActorOrSubobjectData(const FWorldSnapshotData& WorldData,UObject* OriginalObject)
{
	// Is it an actor?
	if (const FActorSnapshotData* SavedActorData = WorldData.ActorData.Find(OriginalObject))
	{
		return &SavedActorData->CustomActorSerializationData;
	}
	if (Cast<AActor>(OriginalObject))
	{
		// Return immediately to avoid searching the entire array below.
		return nullptr;
	}

	// If not an actor, it is a subobject
	const int32 ObjectReferenceIndex = WorldData.SerializedObjectReferences.Find(OriginalObject);
	if (ObjectReferenceIndex != INDEX_NONE)
	{
		return WorldData.CustomSubobjectSerializationData.Find(ObjectReferenceIndex);
	}

	return nullptr;
}
