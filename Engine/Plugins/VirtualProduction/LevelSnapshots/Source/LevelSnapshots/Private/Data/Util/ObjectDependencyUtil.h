// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Util/ObjectDependencyCallback.h"

class AActor;
class UObject;

struct FCustomSerializationData;
struct FPropertySelectionMap;
struct FSnapshotDataCache;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/** Resolves an object dependency for use in the snapshot world. If the object is a subobject, it gets fully serialized. */
	UObject* ResolveObjectDependencyForSnapshotWorld(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, int32 ObjectPathIndex, const FProcessObjectDependency& ProcessObjectDependency, const FString& LocalisationNamespace); 

	/**
	 * Resolves an object dependency for use in the editor world. If the object is a subobject, it is serialized.
	 * Steps for serializing subobject:
	 *  - If an equivalent object with the saved name and class exists, use that and serialize the properties in SelectionMap into it.
	 *  - If object does not exist, allocate a new object and serialize all properties into it
	 *  - If object exists but has different class, skip. Return null.
	 */
	UObject* ResolveObjectDependencyForEditorWorld(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, int32 ObjectPathIndex, const FString& LocalisationNamespace, const FPropertySelectionMap& SelectionMap);

	/**
	 * Resolves an object depedency when restoring a class default object. Simply resolves without further checks.
	 * @param WorldData Snapshot data
	 * @param ObjectPathIndex Identifier the reference in FWorldSnapshotData::SerializedObjectReferences
	 * @param CurrentValue The value currently saved in the uproperty that we're resolving for
	 */
	UObject* ResolveObjectDependencyForClassDefaultObject(FWorldSnapshotData& WorldData, int32 ObjectPathIndex, UObject* CurrentValue);


	
	/**
	 * Saves the object's path. Intended for external objects, e.g. to UMaterial in the content browser.
	 *
	 * @param ReferenceFromOriginalObject The reference as it was found in a property in the editor world
	 * @param bCheckWhetherSubobject If true, this will check the the object is a subobject of an actor and call AddSubobjectDependency.
	 */
	int32 AddObjectDependency(FWorldSnapshotData& WorldData, UObject* ReferenceFromOriginalObject, bool bCheckWhetherSubobject = true);



	
	/**
	 * Adds a subobject to SerializedObjectReferences and CustomSubobjectSerializationData.
	 * @return A valid index in SerializedObjectReferences and the corresponding subobject data.
	 */
	int32 AddCustomSubobjectDependency(FWorldSnapshotData& WorldData, UObject* ReferenceFromOriginalObject);

	/** Tries to find custom serialisation data. Use this when you already know that ReferenceFromOriginalObject is not an actor. */
	FCustomSerializationData* FindCustomSubobjectData(FWorldSnapshotData& WorldData, const FSoftObjectPath& ReferenceFromOriginalObject);
	
	/** Tries to find custom serialisation data. Use when OriginalObject might be an actor . */
	const FCustomSerializationData* FindCustomActorOrSubobjectData(const FWorldSnapshotData& WorldData, UObject* OriginalObject);
}


