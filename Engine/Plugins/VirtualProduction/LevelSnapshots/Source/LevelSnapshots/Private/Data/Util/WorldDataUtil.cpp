// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Util/WorldDataUtil.h"

#include "Archive/TakeWorldObjectSnapshotArchive.h"
#include "ClassDataUtil.h"
#include "CustomSerialization/CustomObjectSerializationWrapper.h"
#include "Data/Hashing/ActorHashUtil.h"
#include "Data/Util/ObjectDependencyUtil.h"
#include "Data/WorldSnapshotData.h"
#include "Filtering/PropertySelectionMap.h"
#include "Restoration/Actor/ActorUtil.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "SnapshotDataCache.h"
#include "SnapshotConsoleVariables.h"
#include "SnapshotRestorability.h"
#include "Util/SortedScopedLog.h"
#include "Util/SoftObjectPathUtil.h"

#include "Algo/ForEach.h"
#include "Async/ParallelFor.h"
#include "Components/ActorComponent.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "Misc/ScopedSlowTask.h"
#include "Templates/NonNullPointer.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "Internationalization/Internationalization.h"
#include "ScopedTransaction.h"
#include "UnrealEdGlobals.h"
#endif

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

namespace UE::LevelSnapshots::Private::Internal
{
	static TArray<AActor*> GetAllActorsIn(UWorld* World);

	static void CaptureSnapshotData(const TArray<AActor*>& ActorsInWorld, FWorldSnapshotData& SnapshotData);
	static void ComputeActorHashes(const TArray<AActor*>& ActorsInWorld, FWorldSnapshotData& SnapshotData);
}

FWorldSnapshotData UE::LevelSnapshots::Private::SnapshotWorld(UWorld* World)
{
	FScopedSlowTask TakeSnapshotTask(2.f, LOCTEXT("TakeSnapshotKey", "Take snapshot"));
	TakeSnapshotTask.MakeDialogDelayed(1.f);

	const TArray<AActor*> ActorsInWorld = Internal::GetAllActorsIn(World);
	FWorldSnapshotData SnapshotData;
	SnapshotData.SnapshotVersionInfo.Initialize();
	
	TakeSnapshotTask.EnterProgressFrame(1.f);
	Internal::CaptureSnapshotData(ActorsInWorld, SnapshotData);
	
	TakeSnapshotTask.EnterProgressFrame(1.f);
	Internal::ComputeActorHashes(ActorsInWorld, SnapshotData);

	return MoveTemp(SnapshotData);
}

namespace UE::LevelSnapshots::Private::Internal
{
	static TArray<AActor*> GetAllActorsIn(UWorld* World)
	{
		TArray<AActor*> ObjectArray;
		
		int32 NumActors = 0;
		for (ULevel* Level : World->GetLevels())
		{
			if (Level)
			{
				NumActors += Level->Actors.Num();
			}
		}

		ObjectArray.Reserve(NumActors);

		for (ULevel* Level : World->GetLevels())
		{
			if (Level)
			{
				ObjectArray.Append(Level->Actors);
			}
		}

		// Move temp to force optimization on debug / development builds
		return MoveTemp(ObjectArray);
	}
	
	static FActorSnapshotData SnapshotActor(AActor* OriginalActor, FWorldSnapshotData& WorldData);
	static TOptional<FComponentSnapshotData> SnapshotComponent(UActorComponent* OriginalComponent);

	static void CaptureSnapshotData(const TArray<AActor*>& ActorsInWorld, FWorldSnapshotData& SnapshotData)
	{
		FScopedSlowTask CaptureData(ActorsInWorld.Num(), LOCTEXT("CapturingWorldData", "Capturing data"));
		CaptureData.MakeDialogDelayed(1.f);
		
		const bool bShouldLog = ConsoleVariables::CVarLogTimeTakingSnapshots.GetValueOnAnyThread();
		FConditionalSortedScopedLog SortedLog(bShouldLog);
		Algo::ForEach(ActorsInWorld, [&SnapshotData, &CaptureData, &SortedLog](AActor* Actor)
		{
			CaptureData.EnterProgressFrame();
			
			if (Restorability::IsActorDesirableForCapture(Actor))
			{
				FScopedLogItem LogTakeSnapshot = SortedLog.AddScopedLogItem(Actor->GetName());
				SnapshotData.ActorData.Add(Actor, Internal::SnapshotActor(Actor, SnapshotData));
			}
		});
	}
	
	static FActorSnapshotData SnapshotActor(AActor* OriginalActor, FWorldSnapshotData& WorldData)
	{
		const FString NameToSearchFor = ConsoleVariables::CVarBreakOnSnapshotActor.GetValueOnAnyThread();
		if (!NameToSearchFor.IsEmpty() && OriginalActor->GetName().Contains(NameToSearchFor))
		{
			UE_DEBUG_BREAK();
		}
	
		FActorSnapshotData Result;
		Result.ClassIndex = AddClassArchetype(WorldData, OriginalActor);
		FTakeWorldObjectSnapshotArchive::TakeSnapshot(Result.SerializedActorData, WorldData, OriginalActor);
		// If external modules registered for custom serialisation, trigger their callbacks
		TakeSnapshotOfActorCustomSubobjects(OriginalActor, Result.CustomActorSerializationData, WorldData);
	
		TInlineComponentArray<UActorComponent*> Components;
		OriginalActor->GetComponents(Components);
		for (UActorComponent* Comp : Components)
		{
			if (!Restorability::IsComponentDesirableForCapture(Comp))
			{
				continue;
			}
		
			TOptional<FComponentSnapshotData> SerializedComponentData = SnapshotComponent(Comp);
			if (SerializedComponentData)
			{
				const int32 ComponentIndex = AddObjectDependency(WorldData, Comp);
				Result.ComponentData.Add(ComponentIndex, *SerializedComponentData);
				// If external modules registered for custom serialisation, trigger their callbacks
				TakeSnapshotForSubobject(Comp, WorldData);
			}
		}

#if WITH_EDITORONLY_DATA
		Result.ActorLabel = OriginalActor->GetActorLabel();
#endif
		return Result;
	}

	static TOptional<FComponentSnapshotData> SnapshotComponent(UActorComponent* OriginalComponent)
	{
		if (OriginalComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Components created dynamically in the construction script are not supported (%ls). Skipping...", *OriginalComponent->GetPathName());
			return {};
		}
		FComponentSnapshotData Result;
		Result.CreationMethod = OriginalComponent->CreationMethod;
		return Result;
	}

	static void ComputeActorHashes(const TArray<AActor*>& ActorsInWorld, FWorldSnapshotData& SnapshotData)
	{
		FScopedSlowTask ComputeHash(1.f, LOCTEXT("ComputingDataHashes", "Computing hashes"));
		ComputeHash.MakeDialogDelayed(1.f);
		
		// Hashing takes about half of the time
		ParallelFor(ActorsInWorld.Num(), [&SnapshotData, &ActorsInWorld](int32 Index)
		{
			AActor* Actor = ActorsInWorld[Index];
			if (Restorability::IsActorDesirableForCapture(Actor))
			{
				PopulateActorHash(SnapshotData.ActorData[Actor].Hash, Actor);
			}
		});
	}
}

namespace UE::LevelSnapshots::Private::Internal
{
	static void PreloadClassesForRestore(FWorldSnapshotData& WorldData, const FPropertySelectionMap& SelectionMap);
	static void ApplyToWorld_HandleRemovingActors(UWorld* WorldToApplyTo, const FPropertySelectionMap& PropertiesToSerialize);
	static void ApplyToWorld_HandleRecreatingActors(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, TSet<AActor*>& EvaluatedActors, UPackage* LocalisationSnapshotPackage, const FPropertySelectionMap& PropertiesToSerialize);
	static void ApplyToWorld_HandleSerializingMatchingActors(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, TSet<AActor*>& EvaluatedActors, const TArray<FSoftObjectPath>& SelectedPaths, UPackage* LocalisationSnapshotPackage, const FPropertySelectionMap& PropertiesToSerialize);

#if WITH_EDITOR
	class FScopedEditorSelectionClearer
	{
		FSelectionStateOfLevel SelectionStateOfLevel;
	public:
		FScopedEditorSelectionClearer()
		{
			GEditor->GetSelectionStateOfLevel(SelectionStateOfLevel);
			GEditor->SelectNone(true, true, false);
		}
		
		~FScopedEditorSelectionClearer()
		{
			GEditor->SetSelectionStateOfLevel(SelectionStateOfLevel);
		}
	};
#endif
}

void UE::LevelSnapshots::Private::ApplyToWorld(
	FWorldSnapshotData& WorldData,
	FSnapshotDataCache& Cache,
	UWorld* WorldToApplyTo,
	UPackage* LocalisationSnapshotPackage,
	const FPropertySelectionMap& PropertiesToSerialize,
	TFunctionRef<void()> OnPreApply,
	TFunctionRef<void()> OnPostApply)
{
	check(WorldToApplyTo);
	
	const TArray<FSoftObjectPath> SelectedPaths = PropertiesToSerialize.GetKeys();
	const int32 NumActorsToRecreate = PropertiesToSerialize.GetDeletedActorsToRespawn().Num();
	const int32 NumMatchingActors = SelectedPaths.Num();
	FScopedSlowTask ApplyToWorldTask(NumActorsToRecreate + NumMatchingActors, LOCTEXT("ApplyToWorldKey", "Apply to world"));
	ApplyToWorldTask.MakeDialogDelayed(1.f, true);
	
	// Certain custom Blueprint compilers, such as nDisplay, may reset the transaction context. That would cause a crash.
	Internal::PreloadClassesForRestore(WorldData, PropertiesToSerialize);
	
#if WITH_EDITOR
	// Deleting components while they are selected can cause a crash
	Internal::FScopedEditorSelectionClearer RestoreSelection;
	FScopedTransaction Transaction(LOCTEXT("ApplyToWorldKey", "Apply to world"));
#endif

	// It's important to call these events after the transaction has been started
	OnPreApply();
	ON_SCOPE_EXIT { OnPostApply(); };

	// Clear editor world subobject cache from previous ApplyToWorld
	for (auto SubobjectIt = Cache.SubobjectCache.CreateIterator(); SubobjectIt; ++SubobjectIt)
	{
		SubobjectIt->Value.EditorObject.Reset();
	}
	
	Internal::ApplyToWorld_HandleRemovingActors(WorldToApplyTo, PropertiesToSerialize);
	
	TSet<AActor*> EvaluatedActors;
	ApplyToWorldTask.EnterProgressFrame(NumActorsToRecreate);
	Internal::ApplyToWorld_HandleRecreatingActors(WorldData, Cache, EvaluatedActors, LocalisationSnapshotPackage, PropertiesToSerialize);	

	ApplyToWorldTask.EnterProgressFrame(NumMatchingActors);
	Internal::ApplyToWorld_HandleSerializingMatchingActors(WorldData, Cache, EvaluatedActors, SelectedPaths, LocalisationSnapshotPackage, PropertiesToSerialize);
	
	// If we're in the editor then update the gizmos locations as they can get out of sync if any of the serialized actors were selected
#if WITH_EDITOR
	if (GUnrealEd)
	{
		GUnrealEd->UpdatePivotLocationForSelection();
	}
#endif
}

bool UE::LevelSnapshots::Private::HasSavedComponentData(const FWorldSnapshotData& WorldData, const FSoftObjectPath& WorldActorPath, const UActorComponent* EditorOrSnapshotComponent)
{
	const FActorSnapshotData* ActorData = WorldData.ActorData.Find(WorldActorPath);
	if (!ActorData)
	{
		return false;
	}

	const FString ComponentName = EditorOrSnapshotComponent->GetName();
	const int32 ComponentDataIndex = [&WorldData, ActorData, ComponentName]() -> int32
	{
		// For better performance, instead of iterating through all elements in FWorldSnapshotData::SerializedObjectReferences we can use FActorSnapshotData::ComponentData as hint
		for (const TPair<int32, FComponentSnapshotData>& ComponentData : ActorData->ComponentData)
		{
			if (ensure(WorldData.SerializedObjectReferences.IsValidIndex(ComponentData.Key))
				&& ExtractLastSubobjectName(WorldData.SerializedObjectReferences[ComponentData.Key]) == ComponentName)
			{
				return ComponentData.Key;
			}
		}
		return INDEX_NONE;
	}();

	const FSubobjectSnapshotData* ComponentData = WorldData.Subobjects.Find(ComponentDataIndex);
	return ComponentData
		&& ensureMsgf(ComponentDataIndex != INDEX_NONE, TEXT("Did not expect FWorldSnapshotData::Subobjects to contain INDEX_NONE. Investigate."))
		&& ComponentData->SerializedData.Num() > 0;
}

namespace UE::LevelSnapshots::Private::Internal
{
	static void PreloadClassesForRestore(FWorldSnapshotData& WorldData, const FPropertySelectionMap& SelectionMap)
	{
		FScopedSlowTask PreloadClasses(WorldData.ActorData.Num(), LOCTEXT("ApplyToWorld.PreloadClasses", "Preloading classes..."));
		PreloadClasses.MakeDialogDelayed(1.f, false);
		
		// Class required for respawning
		for (const FSoftObjectPath& OriginalRemovedActorPath : SelectionMap.GetDeletedActorsToRespawn())
		{
			PreloadClasses.EnterProgressFrame();
			
			FActorSnapshotData* ActorSnapshot = WorldData.ActorData.Find(OriginalRemovedActorPath);
			if (ensure(ActorSnapshot))
			{
				const FSoftClassPath ClassPath = GetClass(*ActorSnapshot, WorldData);
				UClass* ActorClass = ClassPath.TryLoadClass<AActor>();
				UE_CLOGF(ActorClass == nullptr, LogLevelSnapshots, Warning, "Failed to resolve class '%ls'. Was it removed?", *ClassPath.ToString());
			}
		}

		// Technically we also have to load all component classes... we can skip it for now because the only problematic compiler right now is the nDisplay one.
	}
	
	static void ApplyToWorld_HandleRemovingActors(UWorld* WorldToApplyTo, const FPropertySelectionMap& PropertiesToSerialize)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(ApplyToWorld_RemoveActors)
	
#if WITH_EDITOR
		const TSet<TWeakObjectPtr<AActor>>& ActorsToDespawn = PropertiesToSerialize.GetNewActorsToDespawn();
		const bool bShouldDespawnActors = ActorsToDespawn.Num() > 0;
		if (!bShouldDespawnActors || !ensure(GUnrealEd))
		{
			return;
		}
	
		// Not sure whether needed. "DELETE" command does in UUnrealEdEngine::Exec_Actor ...
		FEditorDelegates::OnDeleteActorsBegin.Broadcast();
		
		FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();
		TArray<AActor*> ActorsToDelete;
		// Actors returning false from CanDeleteSelectedActor is skipped by DeleteActors: just call EditorDelete on them.
		TArray<AActor*> UserManagedActors;
		ActorsToDelete.Reserve(ActorsToDespawn.Num());
		for (const TWeakObjectPtr<AActor>& ActorToDespawn : ActorsToDespawn)
		{
			if (!ActorToDespawn.IsValid())
			{
				continue;
			}

			Module.OnPreRemoveActor({ PropertiesToSerialize, ActorToDespawn.Get() });
			if (FText Dummy; ActorToDespawn->CanDeleteSelectedActor(Dummy))
			{
				ActorsToDelete.Add(ActorToDespawn.Get());
			}
			else
			{
				UserManagedActors.Add(ActorToDespawn.Get());
			}
		}

		// DeleteActors handles advanced clean-up cases, such as proper transaction, group actors, level script, etc.
		constexpr bool bVerifyDeletionCanHappen = true;
		constexpr bool bWarnAboutReferences = false;
		GUnrealEd->DeleteActors(ActorsToDelete, WorldToApplyTo, GUnrealEd->GetSelectedActors()->GetElementSelectionSet(), bVerifyDeletionCanHappen, bWarnAboutReferences, bWarnAboutReferences);
		// EditorDestroyActor for any actors returning false from CanDeleteSelectedActor (these are usually advanced tool specific actors, like level instances, PCG actors, etc.)
		for (AActor* UserManagedActor : UserManagedActors)
		{
			UserManagedActor->GetWorld()->EditorDestroyActor(UserManagedActor, true);
		}
		
		// ... and call the end event like in UUnrealEdEngine
		FEditorDelegates::OnDeleteActorsEnd.Broadcast();
#else
		// In non-editor builds delete the actors like gameplay code would:
		for (const TWeakObjectPtr<AActor>& ActorToDespawn: PropertiesToSerialize.GetNewActorsToDespawn())
		{
			if (ensureMsgf(ActorToDespawn.IsValid(), TEXT("Actor became invalid since selection set was created")))
			{
				ActorToDespawn->Destroy(true, true);
			}
		}
#endif
		
		Module.OnPostRemoveActors({ PropertiesToSerialize });
	}
	
	static bool HandleNameClash(const FSoftObjectPath& OriginalRemovedActorPath);
	static FString ExtractActorName(const FSoftObjectPath& OriginalRemovedActorPath);
	
	static void ApplyToWorld_HandleRecreatingActors(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, TSet<AActor*>& EvaluatedActors, UPackage* LocalisationSnapshotPackage, const FPropertySelectionMap& PropertiesToSerialize)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(ApplyToWorld_RecreateActors);

		FScopedSlowTask RecreateActors(PropertiesToSerialize.GetDeletedActorsToRespawn().Num() * 2, LOCTEXT("ApplyToWorld.RecreateActorsKey", "Re-creating actors"));
		RecreateActors.MakeDialogDelayed(1.f, false);
		
		const bool bShouldLog = ConsoleVariables::CVarLogTimeRecreatingActors.GetValueOnAnyThread();
		FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();
		TMap<FSoftObjectPath, AActor*> RecreatedActors;
		
		// 1st pass: allocate the actors. Serialisation is done in separate step so object references to other deleted actors resolve correctly.
		{
			FConditionalSortedScopedLog AllocationLog(bShouldLog, TEXT("Recreation - allocation"));
			for (const FSoftObjectPath& OriginalRemovedActorPath : PropertiesToSerialize.GetDeletedActorsToRespawn())
			{
				const FString ActorName = ExtractActorName(OriginalRemovedActorPath);
				FScopedLogItem TimedLog = AllocationLog.AddScopedLogItem(ActorName, bShouldLog);
				RecreateActors.EnterProgressFrame(1, FText::FromString(FString::Printf(TEXT("Allocating %s for recreation"), *ActorName)));
				
				FActorSnapshotData* ActorSnapshot = WorldData.ActorData.Find(OriginalRemovedActorPath);
				if (!ensure(ActorSnapshot))
				{
					continue;	
				}

				const FSoftClassPath ClassPath = GetClass(*ActorSnapshot, WorldData);
				UClass* ActorClass = ClassPath.TryLoadClass<AActor>();
				if (!ActorClass)
				{
					UE_LOGF(LogLevelSnapshots, Warning, "Skipping recreation of %ls due to missing class", *OriginalRemovedActorPath.ToString());
					continue;
				}

				if (!HandleNameClash(OriginalRemovedActorPath))
				{
					UE_LOGF(LogLevelSnapshots, Warning, "Skipping recreation of %ls due to name clash", *OriginalRemovedActorPath.ToString());
					continue;
				}
				
				// Example: /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent becomes /Game/MapName.MapName
				const FSoftObjectPath PathToOwningWorldAsset = OriginalRemovedActorPath.GetAssetPathString();
				UObject* UncastWorld = PathToOwningWorldAsset.ResolveObject();
				if (!UncastWorld)
				{
					// Do not TryLoad. If the respective level is loaded, the world must already exist.
					// User has most likely removed the level from the world. We don't want to load that level and modify it by accident. 
					UE_LOGF(LogLevelSnapshots, Error, "Failed to resolve world '%ls'", *PathToOwningWorldAsset.ToString());
					continue;
				}

				// Each Level in UWorld::Levels has a corresponding UWorld associated with it in which we re-create the actor.
				if (UWorld* OwningLevelWorld = ExactCast<UWorld>(UncastWorld))
				{
					ULevel* OverrideLevel = OwningLevelWorld->PersistentLevel;

					const FName ActorFName = *ActorName;
					FActorSpawnParameters SpawnParameters;
					// Cannot be overriden but we want to inform the subscribers
					SpawnParameters.bNoFail = true;
					SpawnParameters.Name = ActorFName;
					SpawnParameters.OverrideLevel = OverrideLevel;
					SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Required_ErrorAndReturnNull;
					SpawnParameters.bDeferConstruction = true;
					// Overriable properties
					SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
					SpawnParameters.ObjectFlags = ActorSnapshot->SerializedActorData.GetObjectFlags()
						// Was wrongly saved for some very old snapshots - if this causes trouble at some point ... consider just deprecating the old snapshots
						// Note that non-transactional actors cannot be removed from the world (so if you recreate and then remove using Level Snapshots it won't work)
						| RF_Transactional;
					
					Module.OnPreCreateActor(OwningLevelWorld, ActorClass, SpawnParameters);
					ensureMsgf(SpawnParameters.bNoFail
						&& SpawnParameters.Name == ActorFName
						&& SpawnParameters.OverrideLevel == OverrideLevel
						&& SpawnParameters.NameMode == FActorSpawnParameters::ESpawnActorNameMode::Required_ErrorAndReturnNull
						&& SpawnParameters.bDeferConstruction, TEXT("You cannot override these properties!"));
					SpawnParameters.bNoFail = true;
					SpawnParameters.Name = ActorFName;
					SpawnParameters.OverrideLevel = OverrideLevel;
					SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Required_ErrorAndReturnNull;
					SpawnParameters.bDeferConstruction = true;
					
					if (AActor* RecreatedActor = OwningLevelWorld->SpawnActor(ActorClass, nullptr, SpawnParameters))
					{
						// The label is set while the actor is still in construction to avoid any side-effects to PostEditChangeProperty.
#if WITH_EDITOR
						// Otherwise actor will show up with internal object name, e.g. actor previously called Cube will be StaticMeshActor1
						if (ActorSnapshot->ActorLabel.IsEmpty()) 
						{
							// This might be an old snapshot in which this data was not yet saved
							RecreatedActor->SetActorLabel(RecreatedActor->GetName());
						}
						else
						{
							RecreatedActor->SetActorLabel(ActorSnapshot->ActorLabel);
						}
#endif
						RecreatedActor->FinishSpawning(RecreatedActor->GetActorTransform());
						Module.OnPostRecreateActor(RecreatedActor);
						RecreatedActors.Add(OriginalRemovedActorPath, RecreatedActor);
					}
				}
			}	
		}
		
		// 2nd pass: serialize.
		{
			FConditionalSortedScopedLog SerializationLog(bShouldLog, TEXT("Recreation - serialization"));
			for (const FSoftObjectPath& OriginalRemovedActorPath : PropertiesToSerialize.GetDeletedActorsToRespawn())
			{
				const FString ActorName = ExtractActorName(OriginalRemovedActorPath);
				FScopedLogItem TimedLog = SerializationLog.AddScopedLogItem(ActorName, bShouldLog);
				RecreateActors.EnterProgressFrame(1, FText::FromString(FString::Printf(TEXT("Serializing recreated %s"), *ActorName)));
			
				if (FActorSnapshotData* ActorSnapshot = WorldData.ActorData.Find(OriginalRemovedActorPath))
				{
					if (AActor** RecreatedActor = RecreatedActors.Find(OriginalRemovedActorPath))
					{
						// Mark it, otherwise we'll serialize it again when we look for world actors matching the snapshot
						EvaluatedActors.Add(*RecreatedActor);
						UE::LevelSnapshots::Private::RestoreIntoRecreatedEditorWorldActor(*RecreatedActor, *ActorSnapshot, WorldData, Cache, LocalisationSnapshotPackage, PropertiesToSerialize);
					}
					else
					{
						UE_LOGF(LogLevelSnapshots, Error, "Failed to recreate actor %ls", *OriginalRemovedActorPath.ToString());
					}
				}
			}
		}
	}

	static bool HandleNameClash(const FSoftObjectPath& OriginalRemovedActorPath)
	{
		UObject* FoundObject = FindObject<UObject>(nullptr, *OriginalRemovedActorPath.ToString());
		if (!FoundObject)
		{
			return true;
		}

		// If it's not an actor nor component then it's possibly an UObjectRedirector
		UActorComponent* AsComponent = Cast<UActorComponent>(FoundObject); // Unlikely
		AActor* AsActor = AsComponent ? AsComponent->GetOwner() : Cast<AActor>(FoundObject);
		if (!AsActor)
		{
			return false;
		}
		
		const bool bShouldRename= AsActor != nullptr;
		const bool bCanDestroy = IsValid(AsActor);
		if (bCanDestroy)
		{
			UE_LOGF(LogLevelSnapshots, Verbose, "Handeling name clash for %ls", *OriginalRemovedActorPath.ToString());
			
#if WITH_EDITOR
			GEditor->SelectActor(AsActor, /*bSelect =*/true, /*bNotifyForActor =*/false, /*bSelectEvenIfHidden =*/true);
			const bool bVerifyDeletionCanHappen = true;
			const bool bWarnAboutReferences = false;
			GEditor->edactDeleteSelected(AsActor->GetWorld(), bVerifyDeletionCanHappen, bWarnAboutReferences, bWarnAboutReferences);
#else
			AsActor->Destroy(true, true);
#endif
		}

		if (bShouldRename)
		{
			const FName NewName = MakeUniqueObjectName(FoundObject->GetOuter(), FoundObject->GetClass());
			return FoundObject->Rename(*NewName.ToString(), nullptr, REN_NonTransactional | REN_DontCreateRedirectors);
		}

		return false;
	}

	static FString ExtractActorName(const FSoftObjectPath& OriginalRemovedActorPath)
	{
		const FString& SubObjectPath = OriginalRemovedActorPath.GetSubPathString();
		const int32 LastDotIndex = SubObjectPath.Find(TEXT("."));
		// Full string: /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42 SubPath: PersistentLevel.StaticMeshActor_42 
		checkf(LastDotIndex != INDEX_NONE, TEXT("There should always be at least one dot after PersistentLevel"));
		const int32 NameLength = SubObjectPath.Len() - LastDotIndex - 1;
		const FString ActorName = SubObjectPath.Right(NameLength);
		return ActorName;
	}
	
	static void ApplyToWorld_HandleSerializingMatchingActors(FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, TSet<AActor*>& EvaluatedActors, const TArray<FSoftObjectPath>& SelectedPaths, UPackage* LocalisationSnapshotPackage, const FPropertySelectionMap& PropertiesToSerialize)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(ApplyToWorld_SerializeMatchedActors);
	
		FScopedSlowTask ExitingActorTask(SelectedPaths.Num(), LOCTEXT("ApplyToWorld.MatchingPropertiesKey", "Writing existing actors"));
		ExitingActorTask.MakeDialogDelayed(1.f, true);
		
		const bool bShouldLog = ConsoleVariables::CVarLogTimeRestoringMatchedActors.GetValueOnAnyThread();
		FConditionalSortedScopedLog SerializationLog(bShouldLog, TEXT("Modified"));
		for (const FSoftObjectPath& SelectedObject : SelectedPaths)
		{
			if (ExitingActorTask.ShouldCancel())
			{
				return;
			}
		
			if (SelectedObject.IsValid())
			{
				AActor* OriginalWorldActor = nullptr;
				UObject* ResolvedObject = SelectedObject.ResolveObject();
				if (AActor* AsActor = Cast<AActor>(ResolvedObject))
				{
					OriginalWorldActor = AsActor;
				}
				else if (ResolvedObject)
				{
					OriginalWorldActor = ResolvedObject->GetTypedOuter<AActor>();
				}

				if (!OriginalWorldActor)
				{
					UE_LOGF(LogLevelSnapshots, Warning, "Selection map requested restoration for %ls but the object could not be resolved", *SelectedObject.ToString());	
				}
				else if (Restorability::IsActorRestorable(OriginalWorldActor) && !EvaluatedActors.Contains(OriginalWorldActor))
				{
					const FString ActorName = ExtractActorName(OriginalWorldActor);
					FScopedLogItem TimedLog = SerializationLog.AddScopedLogItem(ActorName, bShouldLog);
					ExitingActorTask.EnterProgressFrame(1, FText::FromString(FString::Printf(TEXT("Serializing matched %s"), *ActorName)));
					
					EvaluatedActors.Add(OriginalWorldActor);
					FActorSnapshotData* ActorSnapshot = WorldData.ActorData.Find(OriginalWorldActor);
					if (ensure(ActorSnapshot))
					{
						RestoreIntoExistingWorldActor(OriginalWorldActor, *ActorSnapshot, WorldData, Cache, LocalisationSnapshotPackage, PropertiesToSerialize);
					}

					continue;
				}
			}

			ExitingActorTask.EnterProgressFrame(1);
		}
	}
}

#undef LOCTEXT_NAMESPACE
