// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

class AActor;
class UActorComponent;
class UWorld;

struct FPropertySelectionMap;
struct FSnapshotDataCache;
struct FSoftObjectPath;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/** Captures the state of the world */
	FWorldSnapshotData SnapshotWorld(UWorld* World);
	
	/* Applies the saved properties to WorldActor */
	void ApplyToWorld(
		FWorldSnapshotData& WorldData,
		FSnapshotDataCache& Cache,
		UWorld* WorldToApplyTo,
		UPackage* LocalisationSnapshotPackage,
		const FPropertySelectionMap& PropertiesToSerialize,
		TFunctionRef<void()> OnPreApply,
		TFunctionRef<void()> OnPostApply
		);

	/** Whether the component was saved in the world data */
	bool HasSavedComponentData(const FWorldSnapshotData& WorldData, const FSoftObjectPath& WorldActorPath, const UActorComponent* EditorOrSnapshotComponent);
}
