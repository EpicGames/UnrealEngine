// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/WorldSnapshotData.h"

#include "Data/Compression/CompressionUtil.h"
#include "Data/Util/ClassDataUtil.h"
#include "LevelSnapshotsModule.h"
#include "SnapshotCustomVersion.h"
#include "Util/PropertyIterator.h"

#include "Serialization/BufferArchive.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void FWorldSnapshotData::ForEachOriginalActor(TFunctionRef<void(const FSoftObjectPath&, const FActorSnapshotData&)> HandleOriginalActorPath) const
{
	for (auto OriginalPathIt = ActorData.CreateConstIterator(); OriginalPathIt; ++OriginalPathIt)
	{
		HandleOriginalActorPath(OriginalPathIt->Key, OriginalPathIt->Value);
	}	
}

bool FWorldSnapshotData::HasMatchingSavedActor(const FSoftObjectPath& OriginalObjectPath) const
{
	return ActorData.Contains(OriginalObjectPath);
}

namespace UE::LevelSnapshots::Private::Internal
{
	static void CollectActorReferences(FWorldSnapshotData& WorldData, FArchive& Ar)
	{
		// If the archive is modifying our references, e.g. Ar.IsModifyingWeakAndStrongReferences(), a rehash may be needed
		bool bNeedsRehash = false;
		for (auto ActorIt = WorldData.ActorData.CreateIterator(); ActorIt; ++ActorIt)
		{
			const FSoftObjectPath BeforeSerialization = ActorIt->Key;
			Ar << ActorIt->Key;
			bNeedsRehash |= ActorIt->Key != BeforeSerialization;
			FSoftClassPath ClassPath = GetClass(ActorIt->Value, WorldData);
			Ar << ClassPath;
		}

		if (bNeedsRehash)
		{
			const FMapProperty* MapProperty = CastField<FMapProperty>(FWorldSnapshotData::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FWorldSnapshotData, ActorData)));
			FScriptMapHelper Helper(MapProperty, &WorldData.ActorData);
			Helper.Rehash();
		}
	}

	static void CollectClassDefaultReferences(FWorldSnapshotData& WorldData, FArchive& Ar)
	{
		for (auto ClassDefaultIt = WorldData.ClassData.CreateIterator(); ClassDefaultIt; ++ClassDefaultIt)
		{
			Ar << ClassDefaultIt->ClassPath;
		}
	}

	static void CollectEnums(FArchive& Ar)
	{
		TArray<UEnum*> Enums = { StaticEnum<ESnapshotClassFlags>() };
		for (UEnum* Enum : Enums)
		{
			FName ClassFlagsName = Enum->GetFName();
			Ar << ClassFlagsName;
			for (int32 i = 0; i < Enum->NumEnums(); ++i)
			{
				FName ValueName = Enum->GetNameByIndex(i);
				Ar << ValueName;
			}
		}
	}
	
	static void CollectReferencesAndNames(FWorldSnapshotData& WorldData, FArchive& Ar)
	{
		// References
		Ar << WorldData.SerializedObjectReferences;
		CollectActorReferences(WorldData, Ar);
		CollectClassDefaultReferences(WorldData, Ar);

		// Names
		Ar << WorldData.SerializedNames;
		Ar << WorldData.SnapshotVersionInfo.CustomVersions;
		CollectEnums(Ar);

		// Serialize hardcoded property names
		for (int32 i = 0; i < static_cast<int32>(EName::MaxHardcodedNameIndex); ++i)
		{
			FName HardcodedName(static_cast<EName>(i));
			Ar << HardcodedName;
		}

		auto ProcessProperty = [&Ar](const FProperty* Property)
		{
			FName PropertyName = Property->GetFName();
			Ar << PropertyName;
		};
		auto ProcessStruct = [&Ar](UStruct* Struct)
		{
			FName StructName = Struct->GetFName();
			Ar << StructName;
		};
		
		FPropertyIterator(FWorldSnapshotData::StaticStruct(), ProcessProperty, ProcessStruct);
		FPropertyIterator(FSnapshotVersionInfo::StaticStruct(), ProcessProperty, ProcessStruct);
		FPropertyIterator(FActorSnapshotData::StaticStruct(), ProcessProperty, ProcessStruct);
		FPropertyIterator(FSubobjectSnapshotData::StaticStruct(), ProcessProperty, ProcessStruct);
		FPropertyIterator(FCustomSerializationData::StaticStruct(), ProcessProperty, ProcessStruct);
		FPropertyIterator(FClassSnapshotData::StaticStruct(), ProcessProperty, ProcessStruct);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static void MigrateClassDefaultData(FWorldSnapshotData& WorldData)
	{
		TMap<FSoftClassPath, int32> PreventDoubleAddition;
		
		// Actor data
		for (auto ActorIt = WorldData.ActorData.CreateIterator(); ActorIt; ++ActorIt)
		{
			FActorSnapshotData& ActorData = ActorIt->Value;
			const FSoftClassPath ClassPath = ActorData.ActorClass_DEPRECATED;
			ActorData.ActorClass_DEPRECATED.Reset();
			
			if (const int32* Existing = PreventDoubleAddition.Find(ClassPath))
			{
				ActorData.ClassIndex = *Existing;
				continue;
			}
			
			const FClassDefaultObjectSnapshotData* ClassDefaultData = WorldData.ClassDefaults_DEPRECATED.Find(ClassPath);
			if (ClassDefaultData)
			{
				FClassSnapshotData ClassData;
				ClassData.SerializedData = ClassDefaultData->SerializedData;
				ClassData.ObjectFlags = ClassDefaultData->ObjectFlags;
				ClassData.ClassPath = ClassPath;

				const int32 ClassIndex = WorldData.ClassData.Add(ClassData);
				ActorData.ClassIndex = ClassIndex;
				PreventDoubleAddition.Add(ClassPath, ClassIndex);
			}
			else
			{
				checkNoEntry();
			}
		}

		// Subobject data
		for (auto SubobjectIt = WorldData.Subobjects.CreateIterator(); SubobjectIt; ++SubobjectIt)
		{
			FSubobjectSnapshotData& SubobjectData = SubobjectIt->Value;
			if (SubobjectData.bWasSkippedClass)
			{
				continue;
			}
			
			const FSoftClassPath ClassPath = SubobjectData.Class_DEPRECATED;
			SubobjectData.Class_DEPRECATED.Reset();
			
			if (const int32* Existing = PreventDoubleAddition.Find(ClassPath))
			{
				SubobjectData.ClassIndex = *Existing;
				continue;
			}
			
			const FClassDefaultObjectSnapshotData* ClassDefaultData = WorldData.ClassDefaults_DEPRECATED.Find(ClassPath);
			if (ClassDefaultData)
			{
				FClassSnapshotData ClassData;
				ClassData.SerializedData = ClassDefaultData->SerializedData;
				ClassData.ObjectFlags = ClassDefaultData->ObjectFlags;
				ClassData.ClassPath = ClassPath;
			
				const int32 ClassIndex = WorldData.ClassData.Add(ClassData);
				SubobjectData.ClassIndex = ClassIndex;
				PreventDoubleAddition.Add(ClassPath, ClassIndex);
			}
			else
			{
				checkNoEntry();
			}
		}

		WorldData.ClassDefaults_DEPRECATED.Reset();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FWorldSnapshotData::Serialize(FArchive& Ar)
{
	using namespace UE::LevelSnapshots::Private;
	Ar.UsingCustomVersion(UE::LevelSnapshots::FSnapshotCustomVersion::GUID);

	// When this struct is saved, the save algorithm collects references. It's faster if we just give it the info directly.
	if (Ar.IsObjectReferenceCollector())
	{
		UE::LevelSnapshots::Private::Internal::CollectReferencesAndNames(*this, Ar);
		return true;
	}

	if (Ar.IsSaving())
	{
		UE::LevelSnapshots::Compress(Ar, *this);
		return true;
	}

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(UE::LevelSnapshots::FSnapshotCustomVersion::GUID) >= UE::LevelSnapshots::FSnapshotCustomVersion::OoddleCompression)
		{
			UE::LevelSnapshots::Decompress(Ar, *this);
			return true;
		}
		else
		{
			// Use tagged property serialization
			return false;
		}
	}
	
	return false;
}

void FWorldSnapshotData::PostSerialize(const FArchive& Ar)
{
	using namespace UE::LevelSnapshots::Private;
	using namespace UE::LevelSnapshots::Private::Internal;
	
	if (Ar.IsLoading() && !SnapshotVersionInfo.IsInitialized())
	{
		// Assets saved before we added version tracking need to receive versioning info of 4.27.
		// Skip snapshot version info because it did not exist yet at that time (you'll get migration bugs otherwise).
		const bool bWithoutSnapshotVersion = true;
		SnapshotVersionInfo.Initialize(bWithoutSnapshotVersion);
	}
	
	if (Ar.IsLoading()
		&& Ar.CustomVer(UE::LevelSnapshots::FSnapshotCustomVersion::GUID) < UE::LevelSnapshots::FSnapshotCustomVersion::ClassArchetypeRefactor)
	{
		MigrateClassDefaultData(*this);
	}
}

#undef LOCTEXT_NAMESPACE