// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplySnapshotFilter.h"

#include "CustomSerialization/CustomSerializationDataManager.h"
#include "Data/LevelSnapshot.h"
#include "Data/Util/ObjectDependencyUtil.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Filtering/PropertySelection.h"
#include "Interfaces/IPropertyComparer.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "Params/PropertyComparisonParams.h"
#include "Restorability/SnapshotRestorability.h"
#include "SnapshotConsoleVariables.h"
#include "SnapshotCustomVersion.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "Stats/StatsMisc.h"

namespace UE::LevelSnapshots::Private::Internal
{
	static bool AreSubobjectsAllowed(UObject* SnapshotObject, UObject* WorldObject)
	{
		return (WorldObject && Restorability::IsSubobjectDesirableForCapture(WorldObject))
			|| (SnapshotObject && Restorability::IsSubobjectDesirableForCapture(SnapshotObject));
	}

	class FConditionalHeaderAndFooterLog
	{
	public:
		FConditionalHeaderAndFooterLog(AActor* Actor)
		{
			const bool bLog = ConsoleVariables::CVarLogSelectionMap.GetValueOnAnyThread();
			if (bLog)
			{
				UE_LOGF(LogLevel, Log, "=============== %ls ===============", *Actor->GetName());
				UE_LOGF(LogLevel, Log, "Path: %ls", *Actor->GetPathName());
			}
		}
	};
	
	static void ConditionallyLogSelectionSet(const UObject* ForObject, const FPropertySelection& PropertySelection)
	{
		const bool bLog = ConsoleVariables::CVarLogSelectionMap.GetValueOnAnyThread();
		if (bLog)
		{
			UE_LOGF(LogLevel, Log, "\t%ls", *ForObject->GetPathName());
			for (const FLevelSnapshotPropertyChain& PropertyChain : PropertySelection.GetSelectedProperties())
			{
				UE_LOGF(LogLevel, Log, "\t\t%ls", *PropertyChain.ToString());
			}
		}
	}
}

UE::LevelSnapshots::Private::FApplySnapshotFilter UE::LevelSnapshots::Private::FApplySnapshotFilter::Make(ULevelSnapshot* Snapshot, AActor* DeserializedSnapshotActor, AActor* WorldActor, const ULevelSnapshotFilter* Filter)
{
	return FApplySnapshotFilter(Snapshot, DeserializedSnapshotActor, WorldActor, Filter);
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::ApplyFilterToFindSelectedProperties(FPropertySelectionMap& MapToAddTo)
{
	SCOPED_SNAPSHOT_CORE_TRACE(ApplyFilters);
	
	if (EnsureParametersAreValid() && Restorability::IsActorRestorable(WorldActor) && EFilterResult::CanInclude(Filter->IsActorValid({ DeserializedSnapshotActor, WorldActor })))
	{
		const Internal::FConditionalHeaderAndFooterLog ConditionalHeader(WorldActor);
		const FConditionalScopeLogTime LogTime(ConsoleVariables::CVarLogSelectionMap.GetValueOnAnyThread(), TEXT("Total Time"));
		
		FilterActorPair(MapToAddTo);
		AnalyseComponentRestoration(MapToAddTo);
	}
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::FPropertyContainerContext::FPropertyContainerContext(
	FPropertySelectionMap& MapToAddTo, FPropertySelection& SelectionToAddTo, UStruct* ContainerClass, void* SnapshotContainer, void* WorldContainer, UObject* AnalysedSnapshotObject, UObject* AnalysedWorldObject, const TArray<FString>& AuthoredPathInformation, const FLevelSnapshotPropertyChain& PropertyChain, UClass* RootClass)
	:
	MapToAddTo(MapToAddTo),
	SelectionToAddTo(SelectionToAddTo),
	ContainerClass(ContainerClass),
	SnapshotContainer(SnapshotContainer),
	WorldContainer(WorldContainer),
	AnalysedSnapshotObject(AnalysedSnapshotObject),
	AnalysedWorldObject(AnalysedWorldObject),
	AuthoredPathInformation(AuthoredPathInformation),
	PropertyChain(PropertyChain),
	RootClass(RootClass)
{}

UE::LevelSnapshots::Private::FApplySnapshotFilter::FApplySnapshotFilter(ULevelSnapshot* Snapshot, AActor* DeserializedSnapshotActor, AActor* WorldActor, const ULevelSnapshotFilter* Filter)
    :
    Snapshot(Snapshot),
    DeserializedSnapshotActor(DeserializedSnapshotActor),
    WorldActor(WorldActor),
    Filter(Filter)
{
	EnsureParametersAreValid();
}

bool UE::LevelSnapshots::Private::FApplySnapshotFilter::EnsureParametersAreValid() const
{
	if (!ensure(WorldActor && DeserializedSnapshotActor && Filter))
	{
		return false;
	}

	UClass* WorldClass = WorldActor->GetClass();
	UClass* DeserializedClass = DeserializedSnapshotActor->GetClass();
	if (WorldClass != DeserializedClass)
	{
		UE_LOGF(
            LogLevelSnapshots,
            Error,
            "FApplySnapshotFilter::ApplyFilterToFindSelectedProperties: WorldActor class '%ls' differs from DerserializedSnapshoatActor class '%ls'. Differing classes are not supported. (Actor: %ls)",
            *WorldActor->GetClass()->GetName(),
            *DeserializedSnapshotActor->GetClass()->GetName(),
			*WorldActor->GetPathName()
        );
		return false;
	}
	
	return true;
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::AnalyseComponentRestoration(FPropertySelectionMap& MapToAddTo)
{
	FAddedAndRemovedComponentInfo ComponentSelection;

	const bool bShouldAnalyseUnmatchedComponents = Snapshot->GetSerializedData().SnapshotVersionInfo.GetSnapshotCustomVersion() >= FSnapshotCustomVersion::SubobjectSupport;
	IterateRestorableComponents(Snapshot, DeserializedSnapshotActor, WorldActor,
		[this, &MapToAddTo](UActorComponent* SnapshotComp, UActorComponent* WorldComp)
		{
			FilterSubobjectPair(MapToAddTo, SnapshotComp, WorldComp);
		},
		[this, &ComponentSelection, bShouldAnalyseUnmatchedComponents](UActorComponent* UnmatchedSnapshotComp)
		{
			if (const EFilterResult::Type FilterResult = Filter->IsDeletedComponentValid(FIsDeletedComponentValidParams(UnmatchedSnapshotComp, WorldActor));
				bShouldAnalyseUnmatchedComponents && EFilterResult::CanInclude(FilterResult))
			{
				ComponentSelection.SnapshotComponentsToAdd.Add(UnmatchedSnapshotComp);
			}
		},
		[this, &ComponentSelection, bShouldAnalyseUnmatchedComponents](UActorComponent* UnmatchedWorldComp)
		{
			if (const EFilterResult::Type FilterResult = Filter->IsAddedComponentValid(FIsAddedComponentValidParams(UnmatchedWorldComp));
				bShouldAnalyseUnmatchedComponents && EFilterResult::CanInclude(FilterResult))
			{
				ComponentSelection.EditorWorldComponentsToRemove.Add(UnmatchedWorldComp);
			}
		}
	);
	
	const bool bIsEmpty = ComponentSelection.SnapshotComponentsToAdd.Num() == 0 && ComponentSelection.EditorWorldComponentsToRemove.Num() == 0;
	if (!bIsEmpty)
	{
		MapToAddTo.AddComponentSelection(WorldActor, ComponentSelection);
	}
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::FilterActorPair(FPropertySelectionMap& MapToAddTo)
{
	AnalysedSnapshotObjects.Add(DeserializedSnapshotActor);
	
	FPropertySelection ActorSelection;
	FPropertyContainerContext ActorContext(
		MapToAddTo,
		ActorSelection,
        DeserializedSnapshotActor->GetClass(),
        DeserializedSnapshotActor,
        WorldActor,
        DeserializedSnapshotActor,
        WorldActor,
        {},
        FLevelSnapshotPropertyChain(),
        WorldActor->GetClass()
        );

	PreApplyFilter(ActorContext, DeserializedSnapshotActor, WorldActor);
	TrackChangedRootProperties(ActorContext, DeserializedSnapshotActor, WorldActor);
	const EFilterObjectPropertiesResult FilterResult = FindAndFilterCustomSubobjectPairs(MapToAddTo, DeserializedSnapshotActor, WorldActor);
	PostApplyFilter(ActorContext, DeserializedSnapshotActor, WorldActor);
	
	ActorSelection.SetHasCustomSerializedSubobjects(FilterResult == EFilterObjectPropertiesResult::HasCustomSubobjects);
	if (!ActorSelection.IsEmpty())
	{
		MapToAddTo.AddObjectProperties(WorldActor, ActorSelection);
		Internal::ConditionallyLogSelectionSet(WorldActor, ActorSelection);
	}
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult UE::LevelSnapshots::Private::FApplySnapshotFilter::FilterSubobjectPair(FPropertySelectionMap& MapToAddTo, UObject* SnapshotSubobject, UObject* WorldSubobject)
{
	if (const EPropertySearchResult* Result = AnalysedSnapshotObjects.Find(SnapshotSubobject))
	{
		return *Result;
	}
	if (SnapshotSubobject->GetClass() != WorldSubobject->GetClass())
	{
		return EPropertySearchResult::FoundProperties;
	}
	
	FPropertySelection SubobjectSelection;
	FPropertyContainerContext ComponentContext(
		MapToAddTo,
		SubobjectSelection,
        SnapshotSubobject->GetClass(),
        SnapshotSubobject,
        WorldSubobject,
        SnapshotSubobject,
        WorldSubobject,
        { WorldSubobject->GetName() },
        FLevelSnapshotPropertyChain(),
		WorldSubobject->GetClass()
        );

	PreApplyFilter(ComponentContext, SnapshotSubobject, WorldSubobject);
	TrackChangedRootProperties(ComponentContext, SnapshotSubobject, WorldSubobject);
	const EFilterObjectPropertiesResult FilterResult = FindAndFilterCustomSubobjectPairs(MapToAddTo, SnapshotSubobject, WorldSubobject);
	SubobjectSelection.SetHasCustomSerializedSubobjects(FilterResult == EFilterObjectPropertiesResult::HasCustomSubobjects);

	// Non-component subobjects should always be added so when applying to world and resolving a subobject, we can tell whether it is referenced or just a dead object still existing in memory
	const bool bAddedProperties = [this, &MapToAddTo, &ComponentContext, SnapshotSubobject, WorldSubobject, &SubobjectSelection]()
	{
		PostApplyFilter(ComponentContext, SnapshotSubobject, WorldSubobject);
		if (SubobjectSelection.IsEmpty())
		{
			// We don't track components because we can efficiently tell whether they're dead or not by asking the parent actor
			if (!WorldSubobject->IsA<UActorComponent>())
			{
				MapToAddTo.MarkSubobjectForRestoringReferencesButSkipProperties(WorldSubobject);
			}
			return false;
		}
		
		MapToAddTo.AddObjectProperties(WorldSubobject, SubobjectSelection);
		Internal::ConditionallyLogSelectionSet(WorldSubobject, SubobjectSelection);
		return true;
	}();
	EPropertySearchResult Result = bAddedProperties ? EPropertySearchResult::FoundProperties : EPropertySearchResult::NoPropertiesFound;
	AnalysedSnapshotObjects.Add(SnapshotSubobject, Result);
	return Result;
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult UE::LevelSnapshots::Private::FApplySnapshotFilter::FilterStructPair(FPropertyContainerContext& Parent, FStructProperty* StructProperty)
{
	FPropertyContainerContext StructContext(
		Parent.MapToAddTo,
		Parent.SelectionToAddTo,
        StructProperty->Struct,
        StructProperty->ContainerPtrToValuePtr<uint8>(Parent.SnapshotContainer),
        StructProperty->ContainerPtrToValuePtr<uint8>(Parent.WorldContainer),
        Parent.AnalysedSnapshotObject,
        Parent.AnalysedWorldObject,
        Parent.AuthoredPathInformation,
        Parent.PropertyChain.MakeAppended(StructProperty),
        Parent.RootClass
        );
	StructContext.AuthoredPathInformation.Add(StructProperty->GetAuthoredName());
	return TrackChangedPropertiesInStruct(StructContext);
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::EFilterObjectPropertiesResult UE::LevelSnapshots::Private::FApplySnapshotFilter::FindAndFilterCustomSubobjectPairs(FPropertySelectionMap& MapToAddTo, UObject* SnapshotOwner, UObject* WorldOwner)
{
	FLevelSnapshotsModule& LevelSnapshots = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
	TSharedPtr<ICustomObjectSnapshotSerializer> ExternalSerializer = LevelSnapshots.GetCustomSerializerForClass(SnapshotOwner->GetClass());
	if (!ExternalSerializer.IsValid())
	{
		return EFilterObjectPropertiesResult::HasOnlyNormalProperties;
	}

	const FCustomSerializationData* SerializationData = UE::LevelSnapshots::Private::FindCustomActorOrSubobjectData(Snapshot->GetSerializedData(), WorldOwner);
	if (!SerializationData)
	{
		UE_LOGF(LogLevelSnapshots, Warning, "Custom ICustomObjectSnapshotSerializer is registered for class %ls but no data was saved for it.", *WorldOwner->GetClass()->GetName());
		return EFilterObjectPropertiesResult::HasOnlyNormalProperties;
	}

	const FCustomSerializationDataReader SubobjectDataReader(
			FCustomSerializationDataGetter_ReadOnly::CreateLambda([SerializationData]() -> const FCustomSerializationData* { return SerializationData; }),
			Snapshot->GetSerializedData()
			);
	bool bAtLeastOneSubobjectWasAdded = false;
	for (int32 i = 0; i < SubobjectDataReader.GetNumSubobjects(); ++i)
	{	
		TSharedPtr<ISnapshotSubobjectMetaData> SubobjectMetadata = SubobjectDataReader.GetSubobjectMetaData(i);
		UObject* SnapshotSubobject = ExternalSerializer->FindOrRecreateSubobjectInSnapshotWorld(SnapshotOwner, *SubobjectMetadata, SubobjectDataReader);
		UObject* EditorSubobject = ExternalSerializer->FindSubobjectInEditorWorld(WorldOwner, *SubobjectMetadata, SubobjectDataReader);
		
		// Snapshot subobject is missing
		if (!SnapshotSubobject || !ensureAlwaysMsgf(SnapshotSubobject->IsIn(SnapshotOwner), TEXT("Your interface must return subobjects")))
		{
			UE_CLOGF(!SnapshotSubobject && EditorSubobject, LogLevelSnapshots, Warning, "Snapshot subobject was missing. Removing newly added editor world subobjects is not supported.");
			continue;
		}

		// Editor subobject is missing
		if (!EditorSubobject)
		{
			if (ensureAlwaysMsgf(SnapshotSubobject->IsIn(SnapshotOwner), TEXT("Your interface must return subobjects")))
			{
				MapToAddTo.AddCustomEditorSubobjectToRecreate(WorldOwner, SnapshotSubobject);
				bAtLeastOneSubobjectWasAdded = true;
				AnalysedSnapshotObjects.Add(SnapshotSubobject);
			}
			continue;
		}

		// Both subobjects exist
		if (ensureAlwaysMsgf(EditorSubobject->IsIn(WorldOwner), TEXT("Your interface must return subobjects"))
			&& ensureAlwaysMsgf(!AnalysedSnapshotObjects.Contains(SnapshotSubobject), TEXT("You returned an object that was already found (probably by standard Level Snapshot serialisation)")))
		{
			bAtLeastOneSubobjectWasAdded |= FilterSubobjectPair(MapToAddTo, SnapshotSubobject, EditorSubobject) == EPropertySearchResult::FoundProperties;
		}
	}

	return bAtLeastOneSubobjectWasAdded ? EFilterObjectPropertiesResult::HasCustomSubobjects : EFilterObjectPropertiesResult::HasOnlyNormalProperties;
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackChangedRootProperties(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject)
{
	FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();
	const FPropertyComparerArray PropertyComparers = Module.GetPropertyComparerForClass(ContainerContext.RootClass);
	
	for (TFieldIterator<FProperty> FieldIt(ContainerContext.ContainerClass); FieldIt; ++FieldIt)
	{
		// Ask external modules about the property
		const FPropertyComparisonParams Params { Snapshot, ContainerContext.RootClass, *FieldIt, ContainerContext.SnapshotContainer, ContainerContext.WorldContainer, SnapshotObject, WorldObject, DeserializedSnapshotActor, WorldActor} ;
		const IPropertyComparer::EPropertyComparison ComparisonResult = Module.ShouldConsiderPropertyEqual(PropertyComparers, Params);

		bool bSkipEqualityTest = false;
		switch (ComparisonResult)
		{
		case IPropertyComparer::EPropertyComparison::TreatEqual:
			continue;

		case IPropertyComparer::EPropertyComparison::TreatUnequal:
			bSkipEqualityTest = true;

		default:
			break;
		}
		
		TrackPropertyIfChangedRecursive(ContainerContext, *FieldIt, bSkipEqualityTest);
	}
}

TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::HandlePossibleStructProperty(FPropertyContainerContext& ContainerContext, FProperty* PropertyToHandle)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyToHandle))
	{
		return FilterStructPair(ContainerContext, StructProperty);
	}
	return {};
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackChangedPropertiesInStruct(FPropertyContainerContext& ContainerContext)
{
	bool bFoundAtLeastOneProperty = false;
	for (TFieldIterator<FProperty> FieldIt(ContainerContext.ContainerClass); FieldIt; ++FieldIt)
	{
		bFoundAtLeastOneProperty |= TrackPropertyIfChangedRecursive(ContainerContext, *FieldIt) == EPropertySearchResult::FoundProperties;
	}
	return bFoundAtLeastOneProperty ? EPropertySearchResult::FoundProperties : EPropertySearchResult::NoPropertiesFound;
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::PreApplyFilter(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject)
{
	FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();
	Module.PreApplyFilters({ SnapshotObject, WorldObject, Snapshot });
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::PostApplyFilter(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject)
{
	ExtendAnalysedProperties(ContainerContext, SnapshotObject, WorldObject);
}

void UE::LevelSnapshots::Private::FApplySnapshotFilter::ExtendAnalysedProperties(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject)
{
	FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();

	const FPostApplyFiltersResult FilterExtension = Module.PostApplyFilters({ ContainerContext.SelectionToAddTo, SnapshotObject, WorldObject });
	for (const FLevelSnapshotPropertyChain& ToFilter : FilterExtension.AdditionalPropertiesToFilter)
	{
		TArray<FString> Path;
		for (int32 i = 0; i < ToFilter.GetNumProperties(); ++i)
		{
			Path.Emplace(ToFilter.GetPropertyFromRoot(i)->GetAuthoredName());
		}
		
		const bool bIsPropertyValid = EFilterResult::CanInclude(Filter->IsPropertyValid(
			{ DeserializedSnapshotActor, WorldActor, ContainerContext.SnapshotContainer, ContainerContext.WorldContainer, ToFilter.GetPropertyFromRoot(ToFilter.GetNumProperties() - 1), Path }
		));
		if (bIsPropertyValid)
		{
			ContainerContext.SelectionToAddTo.AddProperty(ToFilter);
		}
	}

	for (const FLevelSnapshotPropertyChain& ForceShow : FilterExtension.AdditionalPropertiesToForcefullyShow)
	{
		ContainerContext.SelectionToAddTo.AddProperty(ForceShow);
	}
}

TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::HandlePossibleSubobjectProperties(FPropertyContainerContext& ContainerContext, FProperty* PropertyToHandle)
{
	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyToHandle))
	{
		return TrackPossibleArraySubobjectProperties(ContainerContext, ArrayProperty);
	}
	if (FSetProperty* SetProperty = CastField<FSetProperty>(PropertyToHandle))
	{
		return TrackPossibleSetSubobjectProperties(ContainerContext, SetProperty);
	}
	if (FMapProperty* MapProperty = CastField<FMapProperty>(PropertyToHandle))
	{
		return TrackPossibleMapSubobjectProperties(ContainerContext, MapProperty);
	}
	
	if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyToHandle))
	{
		void* SnapshotValuePtr = PropertyToHandle->ContainerPtrToValuePtr<void>(ContainerContext.SnapshotContainer, 0);
		void* WorldValuePtr = PropertyToHandle->ContainerPtrToValuePtr<void>(ContainerContext.WorldContainer, 0);
		return TrackPossibleSubobjectProperties(ContainerContext, ObjectProperty, SnapshotValuePtr, WorldValuePtr);
	}
	
	return {};
}

TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackPossibleArraySubobjectProperties(FPropertyContainerContext& ContainerContext, FArrayProperty* PropertyToHandle)
{
	struct FArrayData
	{
		FScriptArrayHelper SnapshotArray;
		FScriptArrayHelper WorldArray;
		const int32 SnapshotNum;
		const int32 WorldNum;

		FArrayData(FPropertyContainerContext& ContainerContext, FArrayProperty* ArrayProperty)
			:
			SnapshotArray(FScriptArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ContainerContext.SnapshotContainer, 0))),
			WorldArray(FScriptArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ContainerContext.WorldContainer, 0))),
			SnapshotNum(SnapshotArray.Num()),
			WorldNum(WorldArray.Num())
		{}

		void* GetSnapshotValuePtrFromIndex(int32 WorldIndex)
		{
			return SnapshotArray.GetRawPtr(WorldIndex);
		}

		void* GetWorldValuePtrFromIndex(int32 WorldIndex)
		{
			return WorldArray.GetRawPtr(WorldIndex);
		}
	};

	FObjectPropertyBase* InnerAsObjectProperty = CastField<FObjectPropertyBase>(PropertyToHandle->Inner);
	if (InnerAsObjectProperty && !CastField<FSoftObjectProperty>(PropertyToHandle->Inner))
	{
		FArrayData ArrayData(ContainerContext, PropertyToHandle);
		return TrackPossibleSubobjectsInCollection(ContainerContext, InnerAsObjectProperty, ArrayData);
	}
	return {};
}

TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackPossibleSetSubobjectProperties(FPropertyContainerContext& ContainerContext, FSetProperty* PropertyToHandle)
{
	struct FSetData
	{
		FScriptSetHelper SnapshotSet;
		FScriptSetHelper WorldSet;
		const int32 SnapshotNum;
		const int32 WorldNum;

		TMap<FName, void*> ObjectNameToSnapshotValuePtr;
		const FObjectPropertyBase* ObjectProperty;  

		FSetData(FPropertyContainerContext& ContainerContext, FSetProperty* SetProperty)
			:
			SnapshotSet(FScriptSetHelper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(ContainerContext.SnapshotContainer, 0))),
			WorldSet(FScriptSetHelper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(ContainerContext.WorldContainer, 0))),
			SnapshotNum(SnapshotSet.Num()),
			WorldNum(WorldSet.Num()),
			ObjectProperty(CastField<FObjectPropertyBase>(SetProperty->ElementProp))
		{
			for (int32 SnapshotIndex = 0; SnapshotIndex < SnapshotSet.Num(); ++SnapshotIndex)
			{
				void* SnapshotValuePtr = SnapshotSet.GetElementPtr(SnapshotIndex);
				if (const UObject* SnapshotObject = ObjectProperty->GetObjectPropertyValue(SnapshotValuePtr))
				{
					ObjectNameToSnapshotValuePtr.Add(SnapshotObject->GetFName(), SnapshotValuePtr);
				}
			}
		}

		void* GetSnapshotValuePtrFromIndex(int32 LogicalWorldIndex)
		{
			// Note: we only look for subobjects with the same name because it is faster; otherwise we'd need to compare all properties (n^2 performance).
			// This works most of the time but not when some subobject has changed name (unlikely but possible).
			// In the cases it does not work, we just end up with an additional diff in the results view; a low price to pay.
			const void* WorldValuePtr = GetWorldValuePtrFromIndex(LogicalWorldIndex);
			if (!WorldValuePtr)
			{
				return nullptr;
			}
			
			if (UObject* WorldObject = ObjectProperty->GetObjectPropertyValue(WorldValuePtr))
			{
				void** PossibleSnapshotObject = ObjectNameToSnapshotValuePtr.Find(WorldObject->GetFName());
				return PossibleSnapshotObject ? *PossibleSnapshotObject : nullptr;
			}
			return nullptr;
		}

		void* GetWorldValuePtrFromIndex(int32 LogicalWorldIndex)
		{
			const int32 InternalIndex = WorldSet.FindInternalIndex(LogicalWorldIndex);
			return WorldSet.IsValidIndex(InternalIndex) ? WorldSet.GetElementPtr(InternalIndex) : nullptr;
		}
	};

	FObjectPropertyBase* InnerAsObjectProperty = CastField<FObjectPropertyBase>(PropertyToHandle->ElementProp);
	if (InnerAsObjectProperty && !CastField<FSoftObjectProperty>(PropertyToHandle->ElementProp))
	{
		FSetData SetData(ContainerContext, PropertyToHandle);
		return TrackPossibleSubobjectsInCollection(ContainerContext, InnerAsObjectProperty, SetData);
	}
	return {};
}

TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackPossibleMapSubobjectProperties(FPropertyContainerContext& ContainerContext, FMapProperty* PropertyToHandle)
{
	struct FMapData
	{
		FScriptMapHelper SnapshotMap;
		FScriptMapHelper WorldMap;
		const int32 SnapshotNum;
		const int32 WorldNum;

		FMapData(FPropertyContainerContext& ContainerContext, FMapProperty* MapProperty)
			:
			SnapshotMap(FScriptMapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(ContainerContext.SnapshotContainer, 0))),
			WorldMap(FScriptMapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(ContainerContext.WorldContainer, 0))),
			SnapshotNum(SnapshotMap.Num()),
			WorldNum(WorldMap.Num())
		{}

		void* GetSnapshotValuePtrFromIndex(int32 LogicalWorldIndex)
		{
			const int32 InternalIndex = WorldMap.FindInternalIndex(LogicalWorldIndex);
			if (SnapshotMap.IsValidIndex(InternalIndex))
			{
				const void* WorldKey = WorldMap.GetKeyPtr(InternalIndex);
				return SnapshotMap.FindValueFromHash(WorldKey);
			}
			return nullptr;
		}

		void* GetWorldValuePtrFromIndex(int32 LogicalWorldIndex)
		{
			const int32 InternalIndex = WorldMap.FindInternalIndex(LogicalWorldIndex);
			return WorldMap.IsValidIndex(InternalIndex) ? WorldMap.GetValuePtr(InternalIndex) : nullptr;
		}
	};

	FObjectPropertyBase* InnerAsObjectProperty = CastField<FObjectPropertyBase>(PropertyToHandle->ValueProp);
	if (InnerAsObjectProperty && !CastField<FSoftObjectProperty>(PropertyToHandle->ValueProp))
	{
		FMapData MapData(ContainerContext, PropertyToHandle);
		return TrackPossibleSubobjectsInCollection(ContainerContext, InnerAsObjectProperty, MapData);
	}
	return {};
}

template<typename TCollectionData>
TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackPossibleSubobjectsInCollection(FPropertyContainerContext& ContainerContext, FObjectPropertyBase* ObjectProperty, TCollectionData& Detail)
{
	TArray<UObject*> UnchangedSubobjects;
	bool bFoundChangedProperties = false;
	bool bHadUnmatchedObjects = Detail.SnapshotNum != Detail.WorldNum;
	const int32 MaxSharedSize = FMath::Min(Detail.SnapshotNum, Detail.WorldNum);
	for (int32 WorldIndex = 0; WorldIndex < MaxSharedSize; ++WorldIndex)
	{
		void* SnapshotValuePtr = Detail.GetSnapshotValuePtrFromIndex(WorldIndex);
		void* WorldValuePtr = SnapshotValuePtr ? Detail.GetWorldValuePtrFromIndex(WorldIndex) : nullptr;
		if (!SnapshotValuePtr || !WorldValuePtr)
		{
			bHadUnmatchedObjects = true;
			continue;
		}
		
		const TOptional<EPropertySearchResult> SubobjectPairResult = TrackPossibleSubobjectProperties(ContainerContext, ObjectProperty, SnapshotValuePtr, WorldValuePtr);
		if (!SubobjectPairResult.IsSet())
		{
			return {};
		}
		
		const bool bFoundChangesOnSubobject = SubobjectPairResult == EPropertySearchResult::FoundProperties;
		bFoundChangedProperties |= bFoundChangesOnSubobject;
		UObject* WorldObject = ObjectProperty->GetObjectPropertyValue(WorldValuePtr);
		if (!bFoundChangesOnSubobject && WorldObject)
		{
			UnchangedSubobjects.Add(WorldObject);
		}
	}

	const bool bHasCollectionChanged = bFoundChangedProperties || bHadUnmatchedObjects;
	if (bHasCollectionChanged)
	{
		// The collection has changed. When restoring, SnapshotUtil::ResolveObjectDependencyForEditorWorld is called for each object
		// We tell it to not write any data into the subobjects by adding an empty set.
		for (UObject* UnchangedSubobject : UnchangedSubobjects)
		{
			ContainerContext.MapToAddTo.MarkSubobjectForRestoringReferencesButSkipProperties(UnchangedSubobject);
		}
	}
	
	return bHasCollectionChanged ? EPropertySearchResult::FoundProperties : EPropertySearchResult::NoPropertiesFound;
}

TOptional<UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult> UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackPossibleSubobjectProperties(FPropertyContainerContext& ContainerContext, FObjectPropertyBase* PropertyToHandle, void* SnapshotValuePtr, void* WorldValuePtr)
{
	// Soft object properties do not work with subobjects and are not expected to: the path will remain pointing at the real world object...
	if (FSoftObjectProperty* SoftObjectPath = CastField<FSoftObjectProperty>(PropertyToHandle))
	{
		// ... with soft paths we care about the string change only so compare it normally
		return {};
	}
	
	const bool bPropertyMarkedAsSubobject = PropertyToHandle->HasAnyPropertyFlags(CPF_PersistentInstance | CPF_InstancedReference | CPF_ContainsInstancedReference);
	UObject* SnapshotObject = PropertyToHandle->GetObjectPropertyValue(SnapshotValuePtr);
	UObject* WorldObject = PropertyToHandle->GetObjectPropertyValue(WorldValuePtr);

	if (SnapshotObject == nullptr || WorldObject == nullptr)
	{
		// Are they subobjects?
		const bool bSnapshotObjectIsSubobject = bPropertyMarkedAsSubobject || (SnapshotObject && SnapshotObject->IsIn(DeserializedSnapshotActor));
		const bool bWorldObjectIsSubobject = bPropertyMarkedAsSubobject || (WorldObject && WorldObject->IsIn(WorldActor));

		// Skipped?
		const bool bIsSnapshotSupportedSubobject = SnapshotObject && bSnapshotObjectIsSubobject && !Restorability::IsSubobjectDesirableForCapture(SnapshotObject);
		const bool bIsWorldSupportedSubobject = WorldObject && bWorldObjectIsSubobject && !Restorability::IsSubobjectDesirableForCapture(WorldObject);

		// Migration: We did not save subobject data. In this case snapshot version resolves to nullptr.
		const bool bIsOldSnapshot = Snapshot->GetSerializedData().SnapshotVersionInfo.GetSnapshotCustomVersion() < FSnapshotCustomVersion::SubobjectSupport;
		const bool bOldSnapshotDataDidNotCaptureSubobjects = bIsOldSnapshot && bWorldObjectIsSubobject && SnapshotObject == nullptr;
		
		return SnapshotObject == WorldObject || bOldSnapshotDataDidNotCaptureSubobjects|| bIsSnapshotSupportedSubobject || bIsWorldSupportedSubobject
			? 
			EPropertySearchResult::NoPropertiesFound 
			: 
			bWorldObjectIsSubobject|| bSnapshotObjectIsSubobject ? EPropertySearchResult::FoundProperties : TOptional<FApplySnapshotFilter::EPropertySearchResult>{};
	}

	// Components are special: Only check whether they point to the same objects. 
	const bool bAreComponents = SnapshotObject->IsA<UActorComponent>() || WorldObject->IsA<UActorComponent>();
	if (bAreComponents)
	{
		return UE::LevelSnapshots::Private::AreObjectPropertiesEquivalent(Snapshot, PropertyToHandle, SnapshotValuePtr, WorldValuePtr, DeserializedSnapshotActor, WorldActor)
			? EPropertySearchResult::NoPropertiesFound : EPropertySearchResult::FoundProperties;
	}
		
	const bool bIsSubobject = bPropertyMarkedAsSubobject || SnapshotObject->IsIn(DeserializedSnapshotActor);
	if (bIsSubobject)
	{
		return Internal::AreSubobjectsAllowed(SnapshotObject, WorldObject) 
				? FilterSubobjectPair(ContainerContext.MapToAddTo, SnapshotObject, WorldObject) : EPropertySearchResult::NoPropertiesFound;
	}

	return {};
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackPropertyIfChangedRecursive(FPropertyContainerContext& ContainerContext, FProperty* PropertyInCommon, bool bSkipEqualityTest)
{
	if (!Restorability::IsRestorableProperty(PropertyInCommon))
	{
		return EPropertySearchResult::NoPropertiesFound;
	}

	check(ContainerContext.WorldContainer);
	check(ContainerContext.SnapshotContainer); 

	TArray<FString> PropertyPath = ContainerContext.AuthoredPathInformation;
	PropertyPath.Add(PropertyInCommon->GetAuthoredName());
	const bool bIsPropertyValid = EFilterResult::CanInclude(Filter->IsPropertyValid(
        { DeserializedSnapshotActor, WorldActor, ContainerContext.SnapshotContainer, ContainerContext.WorldContainer, PropertyInCommon, PropertyPath }
        ));

	if (bIsPropertyValid
		&& (bSkipEqualityTest
			|| TrackChangedProperties(ContainerContext, PropertyInCommon) == EPropertySearchResult::FoundProperties))
	{
		ContainerContext.SelectionToAddTo.AddProperty(ContainerContext.PropertyChain.MakeAppended(PropertyInCommon));
		return EPropertySearchResult::FoundProperties;
	}
	return EPropertySearchResult::NoPropertiesFound;
}

UE::LevelSnapshots::Private::FApplySnapshotFilter::EPropertySearchResult UE::LevelSnapshots::Private::FApplySnapshotFilter::TrackChangedProperties(FPropertyContainerContext& ContainerContext, FProperty* PropertyInCommon)
{
	const TOptional<EPropertySearchResult> StructResult = HandlePossibleStructProperty(ContainerContext, PropertyInCommon);
	if (StructResult.IsSet())
	{
		return *StructResult;
	}
	
	const TOptional<EPropertySearchResult> SubobjectResult = HandlePossibleSubobjectProperties(ContainerContext, PropertyInCommon);
	if (SubobjectResult.IsSet())
	{
		return *SubobjectResult;
	}

	if (!AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, PropertyInCommon, ContainerContext.SnapshotContainer, ContainerContext.WorldContainer, DeserializedSnapshotActor, WorldActor))
	{
		return EPropertySearchResult::FoundProperties;
	}
	
	return EPropertySearchResult::NoPropertiesFound;
}
