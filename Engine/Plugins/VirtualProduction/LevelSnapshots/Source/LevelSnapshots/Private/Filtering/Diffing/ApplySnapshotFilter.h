// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filtering/PropertySelectionMap.h"

class AActor;
class UActorComponent;
class ULevelSnapshot;
class ULevelSnapshotFilter;

namespace UE::LevelSnapshots::Private
{
	class FApplySnapshotFilter
	{
	public:

		static FApplySnapshotFilter Make(ULevelSnapshot* Snapshot, AActor* DeserializedSnapshotActor, AActor* WorldActor, const ULevelSnapshotFilter* Filter);
		
		void ApplyFilterToFindSelectedProperties(FPropertySelectionMap& MapToAddTo);

	private:

		enum class EFilterObjectPropertiesResult
		{
			HasCustomSubobjects,
			HasOnlyNormalProperties
		};
		
		struct FPropertyContainerContext
		{
			FPropertySelectionMap& MapToAddTo;
			FPropertySelection& SelectionToAddTo;
		
			UStruct* ContainerClass;
			void* SnapshotContainer;
			void* WorldContainer;
			UObject* AnalysedSnapshotObject;
			UObject* AnalysedWorldObject;
				
			/* Information passed to blueprints. Property name is appended to this.
			 * Example: [FooComponent] [BarStructPropertyName]...
			 */
			TArray<FString> AuthoredPathInformation;

			/* Keeps track of the structs leading to this container */
			FLevelSnapshotPropertyChain PropertyChain;
			/* Class that PropertyChain begins from. */
			UClass* RootClass;

			FPropertyContainerContext(FPropertySelectionMap& MapToAddTo, FPropertySelection& SelectionToAddTo, UStruct* ContainerClass, void* SnapshotContainer, void* WorldContainer, UObject* AnalysedSnapshotObject, UObject* AnalysedWorldObject, const TArray<FString>& AuthoredPathInformation, const FLevelSnapshotPropertyChain& PropertyChain, UClass* RootClass);
		};

		enum class EPropertySearchResult
		{
			FoundProperties,
			NoPropertiesFound
		};
		
		FApplySnapshotFilter(ULevelSnapshot* Snapshot, AActor* DeserializedSnapshotActor, AActor* WorldActor, const ULevelSnapshotFilter* Filter);
		bool EnsureParametersAreValid() const;
		
		void AnalyseComponentRestoration(FPropertySelectionMap& MapToAddTo);

		// Container types: actors, subobjects, structs, custom subobjects 
		void FilterActorPair(FPropertySelectionMap& MapToAddTo);
		EPropertySearchResult FilterSubobjectPair(FPropertySelectionMap& MapToAddTo, UObject* SnapshotSubobject, UObject* WorldSubobject);
		EPropertySearchResult FilterStructPair(FPropertyContainerContext& Parent, FStructProperty* StructProperty);
		EFilterObjectPropertiesResult FindAndFilterCustomSubobjectPairs(FPropertySelectionMap& MapToAddTo, UObject* SnapshotOwner, UObject* WorldOwner);

		// Start property recursion
		void TrackChangedRootProperties(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject);
		TOptional<EPropertySearchResult> HandlePossibleStructProperty(FPropertyContainerContext& ContainerContext, FProperty* PropertyToHandle);
		EPropertySearchResult TrackChangedPropertiesInStruct(FPropertyContainerContext& ContainerContext);

		// Registered callbacks for ILevelSnapshotsModule
		void PreApplyFilter(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject);
		void PostApplyFilter(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject);
		void ExtendAnalysedProperties(FPropertyContainerContext& ContainerContext, UObject* SnapshotObject, UObject* WorldObject);

		// Subobject analysis
		TOptional<EPropertySearchResult> HandlePossibleSubobjectProperties(FPropertyContainerContext& ContainerContext, FProperty* PropertyToHandle);
		TOptional<EPropertySearchResult> TrackPossibleArraySubobjectProperties(FPropertyContainerContext& ContainerContext, FArrayProperty* PropertyToHandle);
		TOptional<EPropertySearchResult> TrackPossibleSetSubobjectProperties(FPropertyContainerContext& ContainerContext, FSetProperty* PropertyToHandle);
		TOptional<EPropertySearchResult> TrackPossibleMapSubobjectProperties(FPropertyContainerContext& ContainerContext, FMapProperty* PropertyToHandle);
		template<typename TCollectionData>
		TOptional<EPropertySearchResult> TrackPossibleSubobjectsInCollection(FPropertyContainerContext& ContainerContext, FObjectPropertyBase* ObjectProperty, TCollectionData& Detail);
		TOptional<EPropertySearchResult> TrackPossibleSubobjectProperties(FPropertyContainerContext& ContainerContext, FObjectPropertyBase* PropertyToHandle, void* SnapshotValuePtr, void* WorldValuePtr);

		// Single property analysis
		EPropertySearchResult TrackPropertyIfChangedRecursive(FPropertyContainerContext& ContainerContext, FProperty* PropertyInCommon, bool bSkipEqualityTest = false);
		EPropertySearchResult TrackChangedProperties(FPropertyContainerContext& ContainerContext, FProperty* PropertyInCommon);
		
		ULevelSnapshot* Snapshot;
		AActor* DeserializedSnapshotActor;
		AActor* WorldActor;
		const ULevelSnapshotFilter* Filter;
		/** Every time an object is analysed, it is added here, so we do not analyse more than once. */
		TMap<UObject*, EPropertySearchResult> AnalysedSnapshotObjects;
	};
}

