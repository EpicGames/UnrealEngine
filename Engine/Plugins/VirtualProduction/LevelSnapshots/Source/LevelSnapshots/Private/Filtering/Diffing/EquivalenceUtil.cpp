// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/Diffing/EquivalenceUtil.h"

#include "Archive/ApplySnapshotToEditorArchive.h"
#include "Data/CustomSerialization/CustomObjectSerializationWrapper.h"
#include "Data/SnapshotCustomVersion.h"
#include "Data/Util/WorldDataUtil.h"
#include "Data/WorldSnapshotData.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "Restoration/Actor/ActorUtil.h"
#include "Params/PropertyComparisonParams.h"
#include "PropertyInfoHelpers.h"
#include "SnapshotRestorability.h"
#include "Util/SoftObjectPathUtil.h"

#include "Components/ActorComponent.h"
#include "EngineUtils.h"
#include "LevelSnapshot.h"
#include "GameFramework/Actor.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

namespace UE::LevelSnapshots::Private::Internal
{
	/** Iterates properties of SnapshotObject and WorldObject, owned by SnapshotActor and WorldActor, and returns whether at least one property value was different */
	static bool HaveDifferentPropertyValues(ULevelSnapshot* Snapshot, UObject* SnapshotObject, UObject* WorldObject, AActor* SnapshotActor, AActor* WorldActor)
	{
		UClass* ClassToIterate = SnapshotObject->GetClass();
		
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		const FPropertyComparerArray PropertyComparers = Module.GetPropertyComparerForClass(ClassToIterate);
		for (TFieldIterator<FProperty> FieldIt(ClassToIterate); FieldIt; ++FieldIt)
		{
			// Ask external modules about the property
			const FPropertyComparisonParams Params { Snapshot, ClassToIterate, *FieldIt, SnapshotObject, WorldObject, SnapshotObject, WorldObject, SnapshotActor, WorldActor} ;
			const IPropertyComparer::EPropertyComparison ComparisonResult = Module.ShouldConsiderPropertyEqual(PropertyComparers, Params);
			
			switch (ComparisonResult)
			{
			case IPropertyComparer::EPropertyComparison::TreatEqual:
				continue;
			case IPropertyComparer::EPropertyComparison::TreatUnequal:
				return true;
			default:
				if (!UE::LevelSnapshots::Private::AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, *FieldIt, SnapshotObject, WorldObject, SnapshotActor, WorldActor))
				{
					return true;
				}
			}
		}

		return false;
	}

	static bool EnqueueMatchingCustomSubobjects(TInlineComponentArray<TPair<UObject*, UObject*>>& SnapshotOriginalPairsToProcess, const FWorldSnapshotData& WorldData, UObject* SnapshotObject, UObject* WorldObject)
	{
		bool bFailedToMatchAllObjects = false;
		ForEachMatchingCustomSubobjectPair(WorldData, SnapshotObject, WorldObject,
			[&SnapshotOriginalPairsToProcess, &WorldData, &bFailedToMatchAllObjects](UObject* SnapshotSubobject, UObject* WorldSubobject)
			{
				SnapshotOriginalPairsToProcess.Add(TPair<UObject*, UObject*>(SnapshotSubobject, WorldSubobject));
				if (!bFailedToMatchAllObjects)
				{
					bFailedToMatchAllObjects |= EnqueueMatchingCustomSubobjects(SnapshotOriginalPairsToProcess, WorldData, SnapshotSubobject, WorldSubobject);
				}
			},
			[&bFailedToMatchAllObjects](UObject*)
			{
				bFailedToMatchAllObjects = true;
			}
		);

		return bFailedToMatchAllObjects;
	}

	static UActorComponent* TryFindMatchingComponent(AActor* ActorToSearchOn, UActorComponent* CounterpartComponentToMatch)
	{
		UActorComponent* MatchedComponent = FindMatchingComponent(ActorToSearchOn, CounterpartComponentToMatch);
		if (MatchedComponent && MatchedComponent->GetClass() != CounterpartComponentToMatch->GetClass())
		{
			UE_LOGF(LogLevelSnapshots, Error, "Components named %ls were matched to each other but had differing classes (%ls and %ls).",
				*CounterpartComponentToMatch->GetName(),
				*MatchedComponent->GetClass()->GetName(),
				*CounterpartComponentToMatch->GetClass()->GetName()
			);
			return nullptr;
		}

		return MatchedComponent;
	}

	/** @return False if one component could not be matched */
	static bool EnqueueMatchingComponents(ULevelSnapshot* Snapshot, TInlineComponentArray<TPair<UObject*, UObject*>>& SnapshotOriginalPairsToProcess, const FWorldSnapshotData& WorldData, AActor* SnapshotActor, AActor* WorldActor)
	{
		bool bFoundUnmatchedObjects = false;
		UE::LevelSnapshots::Private::IterateRestorableComponents(Snapshot, SnapshotActor, WorldActor,
			[&SnapshotOriginalPairsToProcess, &WorldData, &bFoundUnmatchedObjects](UActorComponent* SnapshotComp, UActorComponent* WorldComp)
			{
				bFoundUnmatchedObjects |= EnqueueMatchingCustomSubobjects(SnapshotOriginalPairsToProcess, WorldData, SnapshotComp, WorldComp);
				SnapshotOriginalPairsToProcess.Add(TPair<UObject*, UObject*>(SnapshotComp, WorldComp));
			},
			[&bFoundUnmatchedObjects](UActorComponent* SnapshotComp)
			{
				bFoundUnmatchedObjects = true;
			},
			[&bFoundUnmatchedObjects](UActorComponent* WorldComp)
			{
				bFoundUnmatchedObjects = true;
			});

		return bFoundUnmatchedObjects;
	}
}

void UE::LevelSnapshots::Private::IterateRestorableComponents(ULevelSnapshot* Snapshot, AActor* SnapshotActor, AActor* WorldActor, FHandleMatchedActorComponent OnComponentsMatched, FHandleUnmatchedActorComponent OnSnapshotComponentUnmatched, FHandleUnmatchedActorComponent OnWorldComponentUnmatched)
{
	const FSoftObjectPath WorldActorPath = WorldActor;
	for (UActorComponent* WorldComp : WorldActor->GetComponents())
	{
		if (!Restorability::IsComponentDesirableForCapture(WorldComp))
		{
			continue;
		}

		UActorComponent* SnapshotMatchedComp = Internal::TryFindMatchingComponent(SnapshotActor, WorldComp);
		if (SnapshotMatchedComp && HasSavedComponentData(Snapshot->GetSerializedData(), WorldActorPath, WorldComp))
		{
			OnComponentsMatched(SnapshotMatchedComp, WorldComp);
		}
		else if (!SnapshotMatchedComp)
		{
			OnWorldComponentUnmatched(WorldComp);
		}
	}

	for (UActorComponent* SnapshotComp : SnapshotActor->GetComponents())
	{
		if (Restorability::IsComponentDesirableForCapture(SnapshotComp)
			&& HasSavedComponentData(Snapshot->GetSerializedData(), WorldActorPath, SnapshotComp)
			&& Internal::TryFindMatchingComponent(WorldActor, SnapshotComp) == nullptr)
		{
			OnSnapshotComponentUnmatched(SnapshotComp);
		}
	}
}

namespace UE::LevelSnapshots::Private::Internal
{
	static FString ExtractRootToLeafComponentPath(const FSoftObjectPath& ComponentPath)
	{
		const TOptional<int32> FirstDotAfterActorName = UE::LevelSnapshots::Private::FindDotAfterActorName(ComponentPath);
		// PersistentLevel.SomeActor.SomeParentComp.SomeChildComp becomes SomeParentComp.SomeChildComp
		return ensure(FirstDotAfterActorName) ? ComponentPath.GetSubPathString().RightChop(*FirstDotAfterActorName) : FString();
	}
}

UActorComponent* UE::LevelSnapshots::Private::FindMatchingComponent(AActor* ActorToSearchOn, const FSoftObjectPath& ComponentPath)
{
	const FString RootToLeafPath = Internal::ExtractRootToLeafComponentPath(ComponentPath);
	if (!ensure(!RootToLeafPath.IsEmpty()))
	{
		return nullptr;
	}

	for (UActorComponent* Component : ActorToSearchOn->GetComponents())
	{
		const FString OtherRootToLeaf = Internal::ExtractRootToLeafComponentPath(Component);
		if (RootToLeafPath.Equals(OtherRootToLeaf))
		{
			return Component;
		}
	}

	return nullptr;
}

bool UE::LevelSnapshots::Private::HasOriginalChangedPropertiesSinceSnapshotWasTaken(ULevelSnapshot* Snapshot, AActor* SnapshotActor, AActor* WorldActor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(HasOriginalChangedProperties);
	
	if (!IsValid(SnapshotActor) || !IsValid(WorldActor))
	{
		return SnapshotActor != WorldActor;
	}
		
	UClass* SnapshotClass = SnapshotActor->GetClass();
	UClass* WorldClass = WorldActor->GetClass();
	if (SnapshotClass != WorldClass)
	{
		return true;
	}

	TInlineComponentArray<TPair<UObject*, UObject*>> SnapshotOriginalPairsToProcess;
	SnapshotOriginalPairsToProcess.Add(TPair<UObject*, UObject*>(SnapshotActor, WorldActor));
	
	const bool bFailedToMatchAllComponentObjects = Internal::EnqueueMatchingComponents(Snapshot, SnapshotOriginalPairsToProcess, Snapshot->GetSerializedData(), SnapshotActor, WorldActor);
	if (bFailedToMatchAllComponentObjects)
	{
		return true;
	}
	
	const bool bFailedToMatchAllSubobjects = Internal::EnqueueMatchingCustomSubobjects(SnapshotOriginalPairsToProcess, Snapshot->GetSerializedData(), SnapshotActor, WorldActor);
	if (bFailedToMatchAllSubobjects)
	{
		return true;
	}
	
	for (const TPair<UObject*, UObject*>& NextPair : SnapshotOriginalPairsToProcess)
	{
		UObject* const SnapshotObject = NextPair.Key;
		UObject* const WorldObject = NextPair.Value;
		if (Internal::HaveDifferentPropertyValues(Snapshot, SnapshotObject, WorldObject, SnapshotActor, WorldActor))
		{
			return true;
		}
	}
	return false;
}

namespace UE::LevelSnapshots::Private
{
	bool AreMapPropertiesEquivalent(ULevelSnapshot* Snapshot, const FMapProperty* MapProperty, void* SnapshotValuePtr, void* WorldValuePtr, AActor* SnapshotActor, AActor* WorldActor);
	bool AreSetPropertiesEquivalent(ULevelSnapshot* Snapshot, const FSetProperty* SetProperty, void* SnapshotValuePtr, void* WorldValuePtr, AActor* SnapshotActor, AActor* WorldActor);
}

bool UE::LevelSnapshots::Private::AreSnapshotAndOriginalPropertiesEquivalent(ULevelSnapshot* Snapshot, const FProperty* LeafProperty, void* SnapshotContainer, void* WorldContainer, AActor* SnapshotActor, AActor* WorldActor)
{
	// Ensure that property's flags are allowed. Skip check collection properties, e.g. FArrayProperty::Inner, etc.: inner properties do not have the same flags.
	const bool bIsInnnerCollectionProperty = IsPropertyInCollection(LeafProperty);
	if (!bIsInnnerCollectionProperty && !Restorability::IsRestorableProperty(LeafProperty))
	{
		return true;
	}
	
	for (int32 i = 0; i < LeafProperty->ArrayDim; ++i)
	{
		void* SnapshotValuePtr = LeafProperty->ContainerPtrToValuePtr<void>(SnapshotContainer, i);
		void* WorldValuePtr = LeafProperty->ContainerPtrToValuePtr<void>(WorldContainer, i);

		// Check whether float is nearly equal instead of exactly equal
		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(LeafProperty))
		{
			return AreNumericPropertiesNearlyEqual(NumericProperty, SnapshotValuePtr, WorldValuePtr);
		}
		
		// Use our custom equality function for struct properties
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(LeafProperty))
		{
			for (TFieldIterator<FProperty> FieldIt(StructProperty->Struct); FieldIt; ++FieldIt)
			{
				if (!AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, *FieldIt, SnapshotValuePtr, WorldValuePtr, SnapshotActor, WorldActor))
				{
					return false;
				}
			}
			return true;
		}

		// Check whether property value points to a subobject
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(LeafProperty))
		{
			return AreObjectPropertiesEquivalent(Snapshot, ObjectProperty, SnapshotValuePtr, WorldValuePtr, SnapshotActor, WorldActor);
		}


		// Use our custom equality function for array, set, and map inner properties
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(LeafProperty))
		{
			FScriptArrayHelper SnapshotArray(ArrayProperty, SnapshotValuePtr);
			FScriptArrayHelper WorldArray(ArrayProperty, WorldValuePtr);
			if (SnapshotArray.Num() != WorldArray.Num())
			{
				return false;
			}

			for (int32 j = 0; j < SnapshotArray.Num(); ++j)
			{
				void* SnapshotElementValuePtr = SnapshotArray.GetRawPtr(j);
				void* WorldElementValuePtr = WorldArray.GetRawPtr(j);

				if (!AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, ArrayProperty->Inner, SnapshotElementValuePtr, WorldElementValuePtr, SnapshotActor, WorldActor))
				{
					return false;
				}
			}
			return true;
		}
		
		if (const FMapProperty* MapProperty = CastField<FMapProperty>(LeafProperty))
		{
			return AreMapPropertiesEquivalent(Snapshot, MapProperty, SnapshotValuePtr, WorldValuePtr, SnapshotActor, WorldActor);
		}

		if (const FSetProperty* SetProperty = CastField<FSetProperty>(LeafProperty))
		{
			return AreSetPropertiesEquivalent(Snapshot, SetProperty, SnapshotValuePtr, WorldValuePtr, SnapshotActor, WorldActor);
		}

		if (const FTextProperty* TextProperty = CastField<FTextProperty>(LeafProperty))
		{
			const FText& SnapshotText = TextProperty->GetPropertyValue_InContainer(SnapshotContainer);
			const FText& WorldText = TextProperty->GetPropertyValue_InContainer(WorldContainer);
			return SnapshotText.IdenticalTo(WorldText, ETextIdenticalModeFlags::None) || SnapshotText.ToString().Equals(WorldText.ToString());
		}
		
		// Use normal property comparison for all other properties
		if (!LeafProperty->Identical_InContainer(SnapshotContainer, WorldContainer, i, PPF_DeepComparison | PPF_DeepCompareDSOsOnly))
		{
			return false;
		}
	}
	return true;
}

bool UE::LevelSnapshots::Private::AreMapPropertiesEquivalent(ULevelSnapshot* Snapshot, const FMapProperty* MapProperty, void* SnapshotValuePtr, void* WorldValuePtr, AActor* SnapshotActor, AActor* WorldActor)
{
	// Technically we need to check permutations like done in UE4MapProperty_Private::IsPermutation... 
	FScriptMapHelper SnapshotMap(MapProperty, SnapshotValuePtr);
	FScriptMapHelper WorldMap(MapProperty, WorldValuePtr);
	if (SnapshotMap.Num() != WorldMap.Num())
	{
		return false;
	}
	
	FScriptMapHelper::FIterator SnapshotMapIterator(SnapshotMap);
	FScriptMapHelper::FIterator WorldMapIterator(WorldMap);

	for (; SnapshotMapIterator && WorldMapIterator; ++SnapshotMapIterator, ++WorldMapIterator)
	{
		void* const SnapshotPairPtr = SnapshotMap.GetPairPtr(SnapshotMapIterator);
		void* const WorldPairPtr = WorldMap.GetPairPtr(WorldMapIterator);

		const bool bAreKeysEquivalent = AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, SnapshotMap.KeyProp, SnapshotPairPtr, WorldPairPtr, SnapshotActor, WorldActor);
		const bool bAreValuesEquivalent = AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, SnapshotMap.ValueProp, SnapshotPairPtr, WorldPairPtr, SnapshotActor, WorldActor);
		if (!bAreKeysEquivalent || !bAreValuesEquivalent)
		{
			return false;
		}
	}
	return true;
}

bool UE::LevelSnapshots::Private::AreSetPropertiesEquivalent(ULevelSnapshot* Snapshot, const FSetProperty* SetProperty, void* SnapshotValuePtr, void* WorldValuePtr, AActor* SnapshotActor, AActor* WorldActor)
{
	// Technically we need to check permutations like done in UE4SetProperty_Private::IsPermutation... 
	FScriptSetHelper SnapshotMap(SetProperty, SnapshotValuePtr);
	FScriptSetHelper WorldMap(SetProperty, WorldValuePtr);
	if (SnapshotMap.Num() != WorldMap.Num())
	{
		return false;
	}

	for (int32 j = 0; j < SnapshotMap.Num(); ++j)
	{
		void* const SnapshotElemValuePtr = SnapshotMap.GetElementPtr(j);
		void* const WorldElemValuePtr = WorldMap.GetElementPtr(j);

		if (!AreSnapshotAndOriginalPropertiesEquivalent(Snapshot, SetProperty->ElementProp, SnapshotElemValuePtr, WorldElemValuePtr, SnapshotActor, WorldActor))
		{
			return false;
		}
	}
	return true;
}

bool UE::LevelSnapshots::Private::AreObjectPropertiesEquivalent(ULevelSnapshot* Snapshot, const FObjectPropertyBase* ObjectProperty, void* SnapshotValuePtr, void* WorldValuePtr, AActor* SnapshotActor, AActor* WorldActor)
{
	// Native identity check handles:
	// - external references, e.g. UMaterial in content browser
	// - soft object paths: if SnapshotValuePtr is a TSoftObjectPtr<AActor> property, it retains the object path to the editor object.
	if (ObjectProperty->Identical(SnapshotValuePtr, WorldValuePtr, 0))
	{
		return true;
	}
		
	UObject* SnapshotObject = ObjectProperty->GetObjectPropertyValue(SnapshotValuePtr);
	UObject* WorldObject = ObjectProperty->GetObjectPropertyValue(WorldValuePtr);
	return AreReferencesEquivalent(Snapshot, SnapshotObject, WorldObject, SnapshotActor, WorldActor);
}

namespace UE::LevelSnapshots::Private::Internal
{
	/** Checks whether the two subobject object properties should be considered equivalent */
	static bool HaveSameNames(UObject* SnapshotPropertyValue, UObject* OriginalPropertyValue, const FSnapshotDataCache& Cache)
	{
		AActor* SnapshotOwningActor = SnapshotPropertyValue->GetTypedOuter<AActor>();
		AActor* OriginalOwningActor = OriginalPropertyValue->GetTypedOuter<AActor>();
		if (!ensureMsgf(SnapshotOwningActor && OriginalOwningActor, TEXT("This is weird: the objects are part of a world and not actors, so they should be subobjects of actors, like components. Investigate")))
		{
			return false;
		}

		// Are the two subobjects owned by equivalent actors
		const AActor* EquivalentSnapshotActor = GetRawPointerOrNull(UE::LevelSnapshots::Private::GetPreallocatedIfCached(OriginalOwningActor, Cache));
		const bool bAreOwnedByEquivalentActors = EquivalentSnapshotActor == SnapshotOwningActor; 
		if (!bAreOwnedByEquivalentActors)
		{
			return false;
		}

		// Check that chain of outers correspond to each other.
		UObject* CurrentSnapshotOuter = SnapshotPropertyValue;
		UObject* CurrentOriginalOuter = OriginalPropertyValue;
		for (; CurrentSnapshotOuter != SnapshotOwningActor && CurrentOriginalOuter != OriginalOwningActor; CurrentSnapshotOuter = CurrentSnapshotOuter->GetOuter(), CurrentOriginalOuter = CurrentOriginalOuter->GetOuter())
		{
			const bool bHaveSameName = CurrentSnapshotOuter->GetFName().IsEqual(CurrentOriginalOuter->GetFName());
			// I thought of also checking whether the two outers have the same class but I see no reason to atm
			if (!bHaveSameName)
			{
				return false;
			}
		}

		return true;
	}

	static bool AreEquivalentSubobjects(ULevelSnapshot* Snapshot, UObject* SnapshotPropertyValue, UObject* OriginalPropertyValue, AActor* SnapshotActor, AActor* OriginalActor)
	{
		AActor* OwningSnapshotActor = SnapshotPropertyValue->GetTypedOuter<AActor>();
		AActor* OwningWorldActor = OriginalPropertyValue->GetTypedOuter<AActor>();
		if (ensure(OwningSnapshotActor && OwningWorldActor))
		{
			return AreActorsEquivalent(OwningSnapshotActor, OwningWorldActor, Snapshot->GetSerializedData(), Snapshot->GetCache()) && !HaveDifferentPropertyValues(Snapshot, SnapshotPropertyValue, OriginalPropertyValue, SnapshotActor, OriginalActor);
		}
		return false;
	}
}

bool UE::LevelSnapshots::Private::AreReferencesEquivalent(ULevelSnapshot* Snapshot, UObject* SnapshotPropertyValue, UObject* OriginalPropertyValue, AActor* SnapshotActor, AActor* OriginalActor)
{
	if (SnapshotPropertyValue == nullptr || OriginalPropertyValue == nullptr)
	{
		// Migration: We did not save subobject data. In this case snapshot version resolves to nullptr.
		const bool bWorldObjectIsSubobject = OriginalPropertyValue && !OriginalPropertyValue->IsA<AActor>() && OriginalPropertyValue->IsInA(AActor::StaticClass());
		const bool bIsOldSnapshot = Snapshot->GetSerializedData().SnapshotVersionInfo.GetSnapshotCustomVersion() < FSnapshotCustomVersion::SubobjectSupport;
		const bool bOldSnapshotDataDidNotCaptureSubobjects = bIsOldSnapshot && bWorldObjectIsSubobject && SnapshotPropertyValue == nullptr;
		
		return bOldSnapshotDataDidNotCaptureSubobjects || SnapshotPropertyValue == OriginalPropertyValue;
	}
	if (SnapshotPropertyValue->GetClass() != OriginalPropertyValue->GetClass())
	{
		return false;
	}

	if (AActor* OriginalActorReference = Cast<AActor>(OriginalPropertyValue))
	{
		return AreActorsEquivalent(SnapshotPropertyValue, OriginalActorReference, Snapshot->GetSerializedData(), Snapshot->GetCache());
	}

	// Fast path for components and other subobjects of actors.
	const bool bIsWorldObject = SnapshotPropertyValue->IsInA(UWorld::StaticClass()) && OriginalPropertyValue->IsInA(UWorld::StaticClass());
	if (bIsWorldObject)
	{
		// Note: Only components are required to have same names. Other subobjects only need to have equal properties.
		const bool bAreComponents = SnapshotPropertyValue->IsA<UActorComponent>() && OriginalPropertyValue->IsA<UActorComponent>();
		return (bAreComponents && Internal::HaveSameNames(SnapshotPropertyValue, OriginalPropertyValue, Snapshot->GetCache()))
			|| !Restorability::IsSubobjectDesirableForCapture(OriginalPropertyValue)
			// Iterates through all properties -> possibly slow
			|| (!bAreComponents && Internal::AreEquivalentSubobjects(Snapshot, SnapshotPropertyValue, OriginalPropertyValue, SnapshotActor, OriginalActor));
	}

	// Example: two materials in the GetTransientPackage()
	const bool bIsSnapshotObjectTransient = (SnapshotPropertyValue->GetPackage()->GetFlags() & RF_Transient) != 0;
	const bool bIsWorldObjectTransient = (OriginalPropertyValue->GetPackage()->GetFlags() & RF_Transient) != 0;
	const bool bAreTransient = bIsSnapshotObjectTransient || bIsWorldObjectTransient;
	if (bAreTransient)
	{
		return !Restorability::IsSubobjectDesirableForCapture(OriginalPropertyValue)
			// Iterates through all properties -> possibly slow
			|| !Internal::HaveDifferentPropertyValues(Snapshot, SnapshotPropertyValue, OriginalPropertyValue, SnapshotActor, OriginalActor);
	}

	// Pointing to assets in asset registry, i.e. no subobjects?
	return SnapshotPropertyValue == OriginalPropertyValue;
}

bool UE::LevelSnapshots::Private::AreActorsEquivalent(UObject* SnapshotPropertyValue, AActor* OriginalActorReference, const FWorldSnapshotData& WorldData, const FSnapshotDataCache& Cache)
{
	// Compare actors
	const FActorSnapshotData* SavedData = WorldData.ActorData.Find(OriginalActorReference);
	if (SavedData == nullptr)
	{
		return false;
	}

	// The snapshot actor was already allocated, if some other snapshot actor is referencing it
	const TOptional<TNonNullPtr<AActor>> PreallocatedSnapshotVersion = UE::LevelSnapshots::Private::GetPreallocatedIfCached(OriginalActorReference, Cache);
	return GetRawPointerOrNull(PreallocatedSnapshotVersion) == SnapshotPropertyValue;
}