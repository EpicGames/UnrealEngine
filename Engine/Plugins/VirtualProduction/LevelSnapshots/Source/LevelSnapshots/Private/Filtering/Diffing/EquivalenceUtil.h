// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

class AActor;
class UActorComponent;
class ULevelSnapshot;
class UObject;

struct FSnapshotDataCache;
struct FSoftObjectPath;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	typedef TFunctionRef<void(UActorComponent* SnapshotComponent, UActorComponent* WorldComponent)> FHandleMatchedActorComponent;
	typedef TFunctionRef<void(UActorComponent*)> FHandleUnmatchedActorComponent;
	
	/**
	 * Iterates through both actors' component lists and calls the appropriate callback.
	 *
	 * @param SnapshotActor Actor stored in ULevelSnapshot's internal world
	 * @param WorldActor Actor stored in level viewport world
	 * @param OnComponentsMatched Called when component exists on both actors
	 * @param OnSnapshotComponentUnmatched Called when component exists on snapshot but not on editor world
	 * @param OnWorldComponentUnmatched Called when component exists in editor world but not in snapshot world
	 */
	void IterateRestorableComponents(ULevelSnapshot* Snapshot, AActor* SnapshotActor, AActor* WorldActor, FHandleMatchedActorComponent OnComponentsMatched, FHandleUnmatchedActorComponent OnSnapshotComponentUnmatched, FHandleUnmatchedActorComponent OnWorldComponentUnmatched);

	/** Tries to find an actor component by following its full outer path, e.g. /Game/Map.Map:PersistentLevel.SomeActor.SomeParentComp.SomeChildComp will find SomeChildComp with an outer SomeParentComp. */
	UActorComponent* FindMatchingComponent(AActor* ActorToSearchOn, const FSoftObjectPath& ComponentPath);

	
	
	/** Checks whether the original actor has any properties that changed since the snapshot was taken.  */
	bool HasOriginalChangedPropertiesSinceSnapshotWasTaken(ULevelSnapshot* Snapshot, AActor* SnapshotActor, AActor* WorldActor);

	/**
	 * Checks whether the snapshot and original property value should be considered equal.
	 * Primitive properties are trivial. Special support is needed for object references.
	 *
	 * @param IgnoredProperties Properties to ignore
	 */
	bool AreSnapshotAndOriginalPropertiesEquivalent(ULevelSnapshot* Snapshot, const FProperty* LeafProperty, void* SnapshotContainer, void* WorldContainer, AActor* SnapshotActor, AActor* WorldActor);

	
	/**
	 * Two object properties are equivalent if they are
	 *  - both null
	 *  - asset references and point to the same asset
	 *  - actor or component references
	 *  - subobject references where 1. the subobject have the same name, 2. have the same classes, and 3. the properties are equal
	 */
	bool AreObjectPropertiesEquivalent(ULevelSnapshot* Snapshot, const FObjectPropertyBase* ObjectProperty, void* SnapshotValuePtr, void* WorldValuePtr, AActor* SnapshotActor, AActor* WorldActor);
	
	/** Checks whether two pointers point to "equivalent" objects. */
	bool AreReferencesEquivalent(ULevelSnapshot* Snapshot, UObject* SnapshotPropertyValue, UObject* OriginalPropertyValue, AActor* SnapshotActor, AActor* OriginalActor);

	/** Checks whether the two actors are equivalent */
	bool AreActorsEquivalent(UObject* SnapshotPropertyValue, AActor* OriginalActorReference, const FWorldSnapshotData& WorldData, const FSnapshotDataCache& Cache);
}
