// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/PropertySelection.h"

#include "PropertyInfoHelpers.h"

namespace
{
	FArchiveSerializedPropertyChain TakeFirstElements(const FArchiveSerializedPropertyChain* Chain, int32 ElementsToTake)
	{
		if (!ensure(ElementsToTake <= Chain->GetNumProperties()))
		{
			return *Chain;
		}

		FArchiveSerializedPropertyChain Result;
		for (int32 i = 0; i < ElementsToTake; ++i)
		{
			Result.PushProperty(Chain->GetPropertyFromStack(i), false);
		}
		return Result;
	}
}

FLevelSnapshotPropertyChain::FLevelSnapshotPropertyChain(const FProperty* RootProperty)
{
	AppendInline(RootProperty);
}

FLevelSnapshotPropertyChain::FLevelSnapshotPropertyChain(const FArchiveSerializedPropertyChain* ChainToLeaf, const FProperty* LeafProperty)
{
	if (ChainToLeaf)
	{
		for (int32 i = 0; i < ChainToLeaf->GetNumProperties(); ++i)
		{
			AppendInline(ChainToLeaf->GetPropertyFromRoot(i));
		}
	}
	AppendInline(LeafProperty);
}

FLevelSnapshotPropertyChain FLevelSnapshotPropertyChain::MakeAppended(const FProperty* Property) const
{
	FLevelSnapshotPropertyChain Result = *this;
	Result.AppendInline(Property);
	return Result;
}

void FLevelSnapshotPropertyChain::AppendInline(const FProperty* Property)
{
	if (ensure(Property))
	{
		// Sadly const_cast is required because parent class FArchiveSerializedPropertyChain requires it. However, we'll never actually modify the property.
		PushProperty(const_cast<FProperty*>(Property), false);
	}
}

TOptional<FLevelSnapshotPropertyChain> FLevelSnapshotPropertyChain::FindPathToProperty(
	const FProperty* InLeafProperty, const UStruct* InStructToSearch, const bool bIncludeLeafPropertyInChain)
{
	struct Local
	{
		static bool MakePropertyChainFromLeafProperty(
			const FProperty* InLeafProperty, const UStruct* InStructToSearch, FLevelSnapshotPropertyChain& OutPropertyChain, const bool bIncludeLeafPropertyInChain)
		{
			for (TFieldIterator<FProperty> It(InStructToSearch); It; ++It)
			{
				const FProperty* Property = *It;

				if (Property == InLeafProperty)
				{
					if (bIncludeLeafPropertyInChain)
					{
						OutPropertyChain.AppendInline(InLeafProperty);
					}

					return true;
				}
				
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					FLevelSnapshotPropertyChain LocalChain = OutPropertyChain;
					LocalChain.AppendInline(StructProperty);

					if (MakePropertyChainFromLeafProperty(InLeafProperty, StructProperty->Struct, LocalChain, bIncludeLeafPropertyInChain))
					{
						OutPropertyChain = LocalChain;

						return true;
					}
				}
			}

			return false;
		}
	};

	bool bSuccess = false;
	
	FLevelSnapshotPropertyChain Result;
	
	if (UE::LevelSnapshots::IsPropertyInCollection(InLeafProperty))
	{
		if (const FProperty* ParentProperty = UE::LevelSnapshots::GetParentProperty(InLeafProperty))
		{
			bSuccess = Local::MakePropertyChainFromLeafProperty(ParentProperty, InStructToSearch, Result, true);
			if (bIncludeLeafPropertyInChain)
			{
				Result.AppendInline(InLeafProperty);
			}
		}
	}
	else
	{
		bSuccess = Local::MakePropertyChainFromLeafProperty(InLeafProperty, InStructToSearch, Result, bIncludeLeafPropertyInChain);
	}
	
	return bSuccess ? Result : TOptional<FLevelSnapshotPropertyChain>();
}

FString FLevelSnapshotPropertyChain::ToString() const
{
	FString Result;
	for (int32 i = 0; i < GetNumProperties(); ++i)
	{
		const FProperty* CurrentProperty = GetPropertyFromRoot(i);
		Result.Append(CurrentProperty->GetName());
		if (i != GetNumProperties() - 1)
		{
			Result.Append(TEXT("."));
		}
	}
	return Result;
}

bool FLevelSnapshotPropertyChain::EqualsSerializedProperty(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const
{
	check(GetNumProperties() > 0);
	
	const bool bHaveSameLeaf = LeafProperty == GetPropertyFromStack(0);
	if (!ContainerChain)
	{
		return bHaveSameLeaf;
	}
	
	const bool bHaveSameChainLength = GetNumProperties() == ContainerChain->GetNumProperties() + 1;
	if (!bHaveSameLeaf
        || !bHaveSameChainLength)
	{
		return false;
	}
	
	// Walk up the chain and compare every element
	for (int32 i = 0; i < ContainerChain->GetNumProperties(); ++i)
	{
		if (ContainerChain->GetPropertyFromRoot(i) != GetPropertyFromRoot(i))
		{
			return false;
		}
	}

	return true;
}

bool FLevelSnapshotPropertyChain::IsEmpty() const
{
	return GetNumProperties() == 0;
}

bool FLevelSnapshotPropertyChain::operator==(const FLevelSnapshotPropertyChain& InPropertyChain) const
{
	const int32 NumberOfProperties = GetNumProperties();
	
	if (NumberOfProperties != InPropertyChain.GetNumProperties())
	{
		return false;
	}

	for (int32 PropertyItr = 0; PropertyItr < NumberOfProperties; PropertyItr++)
	{
		FProperty* PropertyA = GetPropertyFromStack(PropertyItr);
		FProperty* PropertyB = InPropertyChain.GetPropertyFromStack(PropertyItr);

		if (PropertyA != PropertyB)
		{
			return false;
		}
	}
	
	return true;
}

FPropertySelection::FPropertySelection(const FProperty* SingleProperty)
{
	AddProperty({ SingleProperty });
}

FPropertySelection::FPropertySelection(const TSet<const FProperty*>& Properties)
{
	for (const FProperty* Property : Properties)
	{
		AddProperty({ Property  });
	}
}

bool FPropertySelection::ShouldSerializeProperty(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const
{
	if (IsPropertySelected(ContainerChain, LeafProperty))
	{
		return true;
	}

	// If it's a root property, it's not in any collection nor struct
	const bool bIsRootProperty = !ContainerChain || ContainerChain->GetNumProperties() == 0; 
	if (bIsRootProperty)
	{
		return false;
	}
	
	for (int32 i = 0; i < ContainerChain->GetNumProperties(); ++i)
	{
		const FProperty* ParentProperty = ContainerChain->GetPropertyFromStack(i);
		
		// Edge case: structs can implement custom Serialize() implementations:
		// Example: Suppose we're serializing AFooActor::MyStruct where MyStruct is of type FStructType
		// struct FStructType
		// {
		//		FVector SomeVar;
		//		Serialize(FArchive& Ar) { Ar << SomeVar;  }
		// }
		// LeafProperty will be FVector::X, which we need to allow.
		const FStructProperty* StructProperty = CastField<FStructProperty>(ParentProperty); 
		if (StructProperty && StructProperty->Struct->UseNativeSerialization())
		{
			const FArchiveSerializedPropertyChain ChainMinusOne = TakeFirstElements(ContainerChain, i - 1); // Yes, it can be -1 when i = 0, this is valid.
			// ... however the property is only like SomeVar if the struct does not show up in our selected properties
			return !IsPropertySelected(&ChainMinusOne, StructProperty);
		}
		
		// Always serialize all properties inside of collections
		if (UE::LevelSnapshots::IsPropertyCollection(ParentProperty))
		{
			// We assume this function is called by FArchive::ShouldSkipProperty,
			// i.e. ShouldSerializeProperty would return true the previous elements
			return true;
		}
	}
	
	return false;
}

bool FPropertySelection::IsPropertySelected(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const
{
	return FindPropertyChain(ContainerChain, LeafProperty) != INDEX_NONE;
}

bool FPropertySelection::IsEmpty() const
{
	return SelectedProperties.Num() == 0 && !HasCustomSerializedSubobjects();
}

void FPropertySelection::AddProperty(const FLevelSnapshotPropertyChain& SelectedProperty)
{
	if (ensure(SelectedProperty.GetNumProperties() != 0))
	{
		SelectedLeafProperties.Add(SelectedProperty.GetPropertyFromStack(0));
	
		SelectedProperties.Add(SelectedProperty);
	}
}
void FPropertySelection::RemoveProperty(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty)
{
	// We won't modify the property. TFieldPath interface is a bit cumbersome here.
	SelectedLeafProperties.Remove(const_cast<FProperty*>(LeafProperty));
	
	const int32 Index = FindPropertyChain(ContainerChain, LeafProperty);
	if (Index != INDEX_NONE)
	{
		SelectedProperties.RemoveAtSwap(Index);
	}
}

void FPropertySelection::RemoveProperty(FArchiveSerializedPropertyChain* ContainerChain)
{
	FArchiveSerializedPropertyChain CopiedContainerChain = *ContainerChain;
	FProperty* LeafProperty = CopiedContainerChain.GetPropertyFromStack(0);
	CopiedContainerChain.PopProperty(LeafProperty, LeafProperty->IsEditorOnlyProperty());
	
	RemoveProperty(&CopiedContainerChain, LeafProperty);
}

const TArray<TFieldPath<FProperty>>& FPropertySelection::GetSelectedLeafProperties() const
{
	return SelectedLeafProperties;
}

const TArray<FLevelSnapshotPropertyChain>& FPropertySelection::GetSelectedProperties() const
{
	return SelectedProperties;
}

int32 FPropertySelection::FindPropertyChain(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const
{
	if (!ensure(LeafProperty))
	{
		return false;
	}

	for (int32 i = 0; i < SelectedProperties.Num(); ++i)
	{
		const FLevelSnapshotPropertyChain& SelectedChain = SelectedProperties[i];
		if (SelectedChain.EqualsSerializedProperty(ContainerChain, LeafProperty))
		{
			return i;
		}
	}
	
	return INDEX_NONE;
}
