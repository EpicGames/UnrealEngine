// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/PropertySelectionMap.h"

#include "Filtering/RestorableObjectSelection.h"
#include "LevelSnapshotsLog.h"

#include "Algo/ForEach.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectHash.h"

void FPropertySelectionMap::AddDeletedActorToRespawn(const FSoftObjectPath& Original)
{
	DeletedActorsToRespawn.Add(Original);
}

void FPropertySelectionMap::RemoveDeletedActorToRespawn(const FSoftObjectPath& Original)
{
	DeletedActorsToRespawn.Remove(Original);
}

void FPropertySelectionMap::AddNewActorToDespawn(AActor* WorldActor)
{
	NewActorsToDespawn.Add(WorldActor);
}

void FPropertySelectionMap::RemoveNewActorToDespawn(AActor* WorldActor)
{
	NewActorsToDespawn.Remove(WorldActor);
}

bool FPropertySelectionMap::AddObjectProperties(UObject* WorldObject, const FPropertySelection& SelectedProperties)
{
	if (ensure(WorldObject)
		// Expliticly warn about empty sets for subobjects. See documentation of MarkSubobjectForRestoringReferencesButSkipProperties.
		&& ensureMsgf(!SelectedProperties.IsEmpty(), TEXT("Maybe you meant to call MarkSubobjectForRestoringReferencesButSkipProperties?"))
		&& ensureMsgf(!SubobjectsMarkedForReferencesRestorationOnly.Contains(WorldObject), TEXT("Object was already added via MarkSubobjectForRestoringReferencesButSkipProperties because it had no changed properties. Now we're told that is does have selected properties...")))
	{		
		EditorWorldObjectToSelectedProperties.FindOrAdd(WorldObject) = SelectedProperties;
		return true;
	}
	return false;
}

void FPropertySelectionMap::MarkSubobjectForRestoringReferencesButSkipProperties(UObject* WorldSubbject)
{
	if (ensure(WorldSubbject)
		&& ensureMsgf(!EditorWorldObjectToSelectedProperties.Contains(WorldSubbject), TEXT("Object was already added via AddObjectProperties with changed properties. Now we're told that is has no changed properties...")))
	{
		if (UActorComponent* Component = Cast<UActorComponent>(WorldSubbject))
		{
			ensureAlwaysMsgf(Component->CreationMethod == EComponentCreationMethod::Native || Component->CreationMethod == EComponentCreationMethod::UserConstructionScript,
				TEXT("Native components are expected to wind-up here if they're not referenced by uproperties (see FComponentEditorUtils::CanEditComponentInstance). Instanced and SimpleUserConstructionScriptComponent should never show up here."));
			UE_LOGF(LogLevelSnapshots, Warning, "Reference to component %ls will not be restored. Note: native component are only restored if they're referenced by an EditAnywhere property. See FComponentEditorUtils::CanEditComponentInstance.", *WorldSubbject->GetPathName());
			return;
		}
		
		SubobjectsMarkedForReferencesRestorationOnly.Add(WorldSubbject);
	}
}

bool FPropertySelectionMap::IsSubobjectMarkedForReferenceRestorationOnly(const FSoftObjectPath& WorldSubobjectPath) const
{
	return SubobjectsMarkedForReferencesRestorationOnly.Contains(WorldSubobjectPath);
}

void FPropertySelectionMap::RemoveObjectPropertiesFromMap(UObject* WorldObject)
{
	EditorWorldObjectToSelectedProperties.Remove(WorldObject);
}

void FPropertySelectionMap::AddComponentSelection(AActor* EditorWorldActor, const UE::LevelSnapshots::FAddedAndRemovedComponentInfo& ComponentSelection)
{
	const bool bIsEmpty = ComponentSelection.SnapshotComponentsToAdd.Num() == 0 && ComponentSelection.EditorWorldComponentsToRemove.Num() == 0;
	if (ensure(EditorWorldActor && !bIsEmpty))
	{
		EditorActorToComponentSelection.Add(EditorWorldActor, ComponentSelection);
	}
}

void FPropertySelectionMap::RemoveComponentSelection(AActor* EditorWorldActor)
{
	EditorActorToComponentSelection.Remove(EditorWorldActor);
}

void FPropertySelectionMap::AddCustomEditorSubobjectToRecreate(UObject* EditorWorldOwner, UObject* SnapshotSubobject)
{
	EditorWorldObjectToCustomSubobjectSelection.FindOrAdd(EditorWorldOwner).CustomSnapshotSubobjectsToRestore.Add(SnapshotSubobject);
}

void FPropertySelectionMap::RemoveCustomEditorSubobjectToRecreate(UObject* EditorWorldOwner, UObject* SnapshotSubobject)
{
	if (UE::LevelSnapshots::FCustomSubobjectRestorationInfo* RestorationInfo = EditorWorldObjectToCustomSubobjectSelection.Find(EditorWorldOwner))
	{
		if (RestorationInfo->CustomSnapshotSubobjectsToRestore.Num() == 1)
		{
			EditorWorldObjectToCustomSubobjectSelection.Remove(EditorWorldOwner);
		}
		else
		{
			RestorationInfo->CustomSnapshotSubobjectsToRestore.Remove(SnapshotSubobject);
		}
	}
}

UE::LevelSnapshots::FRestorableObjectSelection FPropertySelectionMap::GetObjectSelection(const FSoftObjectPath& EditorWorldObject) const
{
	return UE::LevelSnapshots::FRestorableObjectSelection(EditorWorldObject, *this);
}

TArray<FSoftObjectPath> FPropertySelectionMap::GetKeys() const
{
	TArray<FSoftObjectPath> Result;
	EditorWorldObjectToSelectedProperties.GenerateKeyArray(Result);

	TArray<FSoftObjectPath> ComponentPaths;
	EditorActorToComponentSelection.GenerateKeyArray(ComponentPaths);
	for (const FSoftObjectPath& Path : ComponentPaths)
	{
		Result.Add(Path);
	}

	return Result;
}

void FPropertySelectionMap::ForEachModifiedObject(TFunctionRef<void(const FSoftObjectPath&)> Callback) const
{
	Algo::ForEach(EditorWorldObjectToSelectedProperties, [Callback](const TPair<FSoftObjectPath, FPropertySelection>& Pair)
	{
		Callback(Pair.Key);
	});
	Algo::ForEach(EditorActorToComponentSelection, [this, Callback](const TPair<FSoftObjectPath, UE::LevelSnapshots::FAddedAndRemovedComponentInfo>& Pair)
	{
		if (!EditorWorldObjectToSelectedProperties.Contains(Pair.Key))
		{
			Callback(Pair.Key);
		}
	});
}

bool FPropertySelectionMap::HasAnyModifiedActors() const
{
	return EditorWorldObjectToSelectedProperties.Num() > 0 || EditorActorToComponentSelection.Num() > 0;
}

bool FPropertySelectionMap::HasChanges(AActor* EditorWorldActor) const
{
	using namespace UE::LevelSnapshots;
	
	const FRestorableObjectSelection Selection = GetObjectSelection(EditorWorldActor);
	
	const FPropertySelection* PropertySelection = Selection.GetPropertySelection();
	const bool bHasSelectedProperties = PropertySelection && !PropertySelection->IsEmpty();
	
	const FAddedAndRemovedComponentInfo* ComponentSelection = Selection.GetComponentSelection();
	const bool bHasAddedOrRemovedComps = ComponentSelection
		&& (Selection.GetComponentSelection()->SnapshotComponentsToAdd.Num() != 0 || Selection.GetComponentSelection()->EditorWorldComponentsToRemove.Num() != 0);
	
	const FCustomSubobjectRestorationInfo* CustomSubobjectInfo = Selection.GetCustomSubobjectSelection();
	const bool bHasCustomSubobjects = CustomSubobjectInfo != nullptr;
	if (bHasSelectedProperties || bHasAddedOrRemovedComps || bHasCustomSubobjects)
	{
		return true;
	}
		
	for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(EditorWorldActor))
	{
		if (HasChanges(Component))
		{
			return true;
		}
	}

	return false;
}

bool FPropertySelectionMap::HasChanges(UActorComponent* EditorWorldComponent) const
{
	using namespace UE::LevelSnapshots;
	
	const FRestorableObjectSelection Selection = GetObjectSelection(EditorWorldComponent);
	const FPropertySelection* PropertySelection = Selection.GetPropertySelection();
	const FCustomSubobjectRestorationInfo* CustomSubobjectInfo = Selection.GetCustomSubobjectSelection();
	return (PropertySelection && !PropertySelection->IsEmpty()) || CustomSubobjectInfo;
}

bool FPropertySelectionMap::HasChanges(UObject* EditorWorldObject) const
{
	using namespace UE::LevelSnapshots;
	if (AActor* Actor = Cast<AActor>(EditorWorldObject))
	{
		return HasChanges(Actor);
	}
	
	const FRestorableObjectSelection Selection = GetObjectSelection(EditorWorldObject);
	const FPropertySelection* PropertySelection = Selection.GetPropertySelection();
	const FCustomSubobjectRestorationInfo* CustomSubobjectInfo = Selection.GetCustomSubobjectSelection();
	return (PropertySelection && !PropertySelection->IsEmpty()) || CustomSubobjectInfo;
}

void FPropertySelectionMap::Empty(bool bCanShrink)
{
	EditorWorldObjectToSelectedProperties.Empty(bCanShrink ? EditorWorldObjectToSelectedProperties.Num() : 0);
	EditorActorToComponentSelection.Empty(bCanShrink ? EditorActorToComponentSelection.Num() : 0);
	DeletedActorsToRespawn.Empty(bCanShrink ? DeletedActorsToRespawn.Num() : 0);
	NewActorsToDespawn.Empty(bCanShrink ? NewActorsToDespawn.Num() : 0);
}

TArray<UObject*> FPropertySelectionMap::GetDirectSubobjectsWithProperties(UObject* Root) const
{
	SCOPED_SNAPSHOT_CORE_TRACE(GetDirectSubobjectsWithProperties);
	
	TArray<UObject*> Subobjects;
	GetObjectsWithOuter(Root, Subobjects, EGetObjectsFlags::IncludeNestedObjects);

	for (int32 i = Subobjects.Num() - 1; i > 0; --i)
	{
		const bool bHasSelectedProperties = GetObjectSelection(Subobjects[i]).GetPropertySelection() != nullptr;
		if (!bHasSelectedProperties)
		{
			Subobjects.RemoveAt(i);
		}
	}

	return Subobjects;
}
