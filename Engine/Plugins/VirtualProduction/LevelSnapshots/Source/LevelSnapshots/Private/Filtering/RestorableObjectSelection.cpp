// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/RestorableObjectSelection.h"

#include "Filtering/AddedAndRemovedComponentInfo.h"
#include "Filtering/PropertySelectionMap.h"
#include "Filtering/PropertySelection.h"

const FPropertySelection* UE::LevelSnapshots::FRestorableObjectSelection::GetPropertySelection() const
{
	return Owner.EditorWorldObjectToSelectedProperties.Find(ObjectPath);
}

const UE::LevelSnapshots::FAddedAndRemovedComponentInfo* UE::LevelSnapshots::FRestorableObjectSelection::GetComponentSelection() const
{
	return Owner.EditorActorToComponentSelection.Find(ObjectPath);
}

const UE::LevelSnapshots::FCustomSubobjectRestorationInfo* UE::LevelSnapshots::FRestorableObjectSelection::GetCustomSubobjectSelection() const
{
	return Owner.EditorWorldObjectToCustomSubobjectSelection.Find(ObjectPath);
}
