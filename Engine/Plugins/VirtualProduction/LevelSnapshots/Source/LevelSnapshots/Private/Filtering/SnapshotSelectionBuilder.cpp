// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotSelectionBuilder.h"

#include "ConstantFilter.h"
#include "Filtering/PropertySelectionMap.h"
#include "LevelSnapshot.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotsFilteringLibrary.h"

#include "Algo/ForEach.h"
#include "Async/ParallelTransformReduce.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "Templates/NonNullPointer.h"

namespace UE::LevelSnapshots::Private
{
	FPropertySelectionMap FSnapshotSelectionBuilder::DiffAndFilterSnapshot(UWorld* InWorld, ULevelSnapshot* InSnapshot, const ULevelSnapshotFilter* Filter)
	{
		FSnapshotSelectionBuilder Helper(InWorld, InSnapshot, Filter);
		return Helper.DiffAndFilterSnapshot();
	}

	FSnapshotSelectionBuilder::FSnapshotSelectionBuilder(UWorld* InWorld, ULevelSnapshot* InSnapshot, const ULevelSnapshotFilter* Filter)
		: World(InWorld)
		, Snapshot(InSnapshot)
		, Filter(Filter ? Filter : GetMutableDefault<UConstantFilter>())
	{}

	FPropertySelectionMap FSnapshotSelectionBuilder::DiffAndFilterSnapshot() const
	{
		TArray<AActor*> ModifiedActors;
		TArray<FSoftObjectPath> RemovedActors;
		TArray<AActor*> AddedActors;
		Snapshot->DiffWorld(
			World,
			ULevelSnapshot::FActorConsumer::CreateLambda([this, &ModifiedActors](AActor* MatchedActor)
			{
				if (Snapshot->HasChangedSinceSnapshotWasTaken(MatchedActor))
				{
					ModifiedActors.Add(MatchedActor);
				}
			}),
			ULevelSnapshot::FActorPathConsumer::CreateLambda([&RemovedActors](const FSoftObjectPath& RemovedActor){ RemovedActors.Add(RemovedActor); }),
			ULevelSnapshot::FActorConsumer::CreateLambda([&AddedActors](AActor* AddedActor){ AddedActors.Add(AddedActor); })
			);
		
		FPropertySelectionMap Result;
		FilterModified(ModifiedActors, Result);
		FilterRemoved(RemovedActors, Result);
		FilterAdded(AddedActors, Result);
		return Result;
	}

	void FSnapshotSelectionBuilder::FilterModified(const TArray<AActor*>& ModifiedActors, FPropertySelectionMap& Result) const
	{
		for (auto ModifiedIt = ModifiedActors.CreateConstIterator(); ModifiedIt; ++ModifiedIt)
		{
			AActor* WorldActor = *ModifiedIt;
			if (Snapshot->HasChangedSinceSnapshotWasTaken(WorldActor))
			{
				const FSoftObjectPath ActorPath = WorldActor;
				const TOptional<TNonNullPtr<AActor>> DeserializedSnapshotActor = Snapshot->GetDeserializedActor(ActorPath);
				if (!ensureAlwaysMsgf(GetRawPointerOrNull(DeserializedSnapshotActor), TEXT("Failed to get TMap value for key %s. Is the snapshot corrupted?"), *ActorPath.ToString()))
				{
					continue;
				}
		
				const EFilterResult::Type ActorInclusionResult = Filter->IsActorValid(FIsActorValidParams(DeserializedSnapshotActor->Get(), WorldActor));
				if (EFilterResult::CanInclude(ActorInclusionResult))
				{
					ULevelSnapshotsFilteringLibrary::ApplyFilterToFindSelectedProperties(
						Snapshot,
						Result,
						WorldActor,
						DeserializedSnapshotActor->Get(),
						Filter
					);
				}
			}
		}
	}
	
	void FSnapshotSelectionBuilder::FilterRemoved(const TArray<FSoftObjectPath>& RemovedActors, FPropertySelectionMap& Result) const
	{
		for (auto RemovedActorsIt = RemovedActors.CreateConstIterator(); RemovedActorsIt; ++RemovedActorsIt)
		{
			const EFilterResult::Type FilterResult = Filter->IsDeletedActorValid(
				FIsDeletedActorValidParams(

					// SavedActorPath
					*RemovedActorsIt,

					// HelperForDeserialization
					[this](const FSoftObjectPath& ObjectPath)
					{
						return GetRawPointerOrNull(Snapshot->GetDeserializedActor(ObjectPath));
					},

					// ClassGetter
					[this](const FSoftObjectPath& ObjectPath)
					{
						return GetRawPointerOrNull(Snapshot->GetActorClass(ObjectPath));
					}
				)
			);
			
			if (EFilterResult::CanInclude(FilterResult))
			{
				Result.AddDeletedActorToRespawn(*RemovedActorsIt);
			}
		}
	}
	
	void FSnapshotSelectionBuilder::FilterAdded(const TArray<AActor*>& AddedActors, FPropertySelectionMap& Result) const
	{
		for (auto AddedActorsIt = AddedActors.CreateConstIterator(); AddedActorsIt; ++AddedActorsIt)
		{
			const EFilterResult::Type FilterResult = Filter->IsAddedActorValid(FIsAddedActorValidParams(*AddedActorsIt)); 
			if (EFilterResult::CanInclude(FilterResult))
			{
				Result.AddNewActorToDespawn(*AddedActorsIt);
			}
		}
	};
}
