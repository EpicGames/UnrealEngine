// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class ULevelSnapshotFilter;
class ULevelSnapshot;
class UWorld;
struct FPropertySelectionMap;

namespace UE::LevelSnapshots::Private
{
	/** Algorithm for diffing a snapshot */
	class FSnapshotSelectionBuilder
	{
	public:

		static FPropertySelectionMap DiffAndFilterSnapshot(UWorld* InWorld, ULevelSnapshot* InSnapshot, const ULevelSnapshotFilter* Filter);
		
	private:

		UWorld* World;
		ULevelSnapshot* Snapshot;
		const ULevelSnapshotFilter* Filter;

		FSnapshotSelectionBuilder(UWorld* InWorld, ULevelSnapshot* InSnapshot, const ULevelSnapshotFilter* Filter);

		FPropertySelectionMap DiffAndFilterSnapshot() const;
		
		void FilterModified(const TArray<AActor*>& ModifiedActors, FPropertySelectionMap& Result) const;
		void FilterRemoved(const TArray<FSoftObjectPath>& RemovedActors, FPropertySelectionMap& Result) const;
		void FilterAdded(const TArray<AActor*>& AddedActors, FPropertySelectionMap& Result) const;
	};
}

