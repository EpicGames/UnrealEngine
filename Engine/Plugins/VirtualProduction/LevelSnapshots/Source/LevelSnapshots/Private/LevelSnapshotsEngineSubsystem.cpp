// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEngineSubsystem.h"

#include "ILevelSnapshotsModule.h"

namespace UE::LevelSnapshots::Private
{
	class FSubsystemRestorationListenerAdapter : public IRestorationListener
	{
		/** Lifetime is tracked by ULevelSnapshotsEngineSubsystem::Deinitialize */
		ULevelSnapshotsEngineSubsystem& OwningSubsystem;
	public:

		FSubsystemRestorationListenerAdapter(ULevelSnapshotsEngineSubsystem& OwningSubsystem)
			: OwningSubsystem(OwningSubsystem)
		{}
		
		virtual void PreApplySnapshot(const FPreApplySnapshotParams& Params) override
		{
			OwningSubsystem.OnPreApplySnapshot.Broadcast({ &Params.LevelSnapshot });
		}
		
		virtual void PostApplySnapshot(const FPostApplySnapshotParams& Params) override
		{
			OwningSubsystem.OnPostApplySnapshot.Broadcast({ &Params.LevelSnapshot });
		}
	};
}

void ULevelSnapshotsEngineSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	using namespace UE::LevelSnapshots;
	ILevelSnapshotsModule& Module = ILevelSnapshotsModule::Get();
	Module.OnPreTakeSnapshot().AddUObject(this, &ULevelSnapshotsEngineSubsystem::HandleOnPreTakeSnapshot);
	Module.OnPostTakeSnapshot().AddUObject(this, &ULevelSnapshotsEngineSubsystem::HandleOnPostTakeSnapshot);

	RestorationListener = MakeShared<Private::FSubsystemRestorationListenerAdapter>(*this);
	Module.RegisterRestorationListener(RestorationListener.ToSharedRef());
}

void ULevelSnapshotsEngineSubsystem::Deinitialize()
{
	Super::Deinitialize();

	using namespace UE::LevelSnapshots;
	if (ILevelSnapshotsModule::IsAvailable())
	{
		ILevelSnapshotsModule& Module = ILevelSnapshotsModule::Get();
		Module.OnPreTakeSnapshot().RemoveAll(this);
		Module.OnPostTakeSnapshot().RemoveAll(this);
		Module.UnregisterRestorationListener(RestorationListener.ToSharedRef());
	}

	RestorationListener.Reset();
}

void ULevelSnapshotsEngineSubsystem::HandleOnPreTakeSnapshot(const UE::LevelSnapshots::FPreTakeSnapshotEventData& EventData)
{
	OnPreTakeSnapshot.Broadcast({ EventData.Snapshot });
}

void ULevelSnapshotsEngineSubsystem::HandleOnPostTakeSnapshot(const UE::LevelSnapshots::FPostTakeSnapshotEventData& EventData)
{
	OnPostTakeSnapshot.Broadcast({ EventData.Snapshot });
}
