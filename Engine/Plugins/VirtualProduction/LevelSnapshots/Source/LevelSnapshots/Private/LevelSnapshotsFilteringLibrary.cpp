// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsFilteringLibrary.h"

#include "ConstantFilter.h"
#include "Filtering/Diffing/ApplySnapshotFilter.h"
#include "Filtering/SnapshotSelectionBuilder.h"
#include "LevelSnapshot.h"
#include "LevelSnapshotFilters.h"

#include "EngineUtils.h"
#include "GameFramework/Actor.h"

FPropertySelectionMap ULevelSnapshotsFilteringLibrary::DiffAndFilterSnapshot(UWorld* InWorld, ULevelSnapshot* InSnapshot, const ULevelSnapshotFilter* Filter)
{
	return UE::LevelSnapshots::Private::FSnapshotSelectionBuilder::DiffAndFilterSnapshot(InWorld, InSnapshot, Filter);
}

void ULevelSnapshotsFilteringLibrary::ApplyFilterToFindSelectedProperties(ULevelSnapshot* Snapshot, FPropertySelectionMap& MapToAddTo, AActor* WorldActor, AActor* DeserializedSnapshotActor, const ULevelSnapshotFilter* Filter)
{
	if (Filter == nullptr)
	{
		Filter = GetMutableDefault<UConstantFilter>();
	}
	
	UE::LevelSnapshots::Private::FApplySnapshotFilter::Make(Snapshot, DeserializedSnapshotActor, WorldActor, Filter)
		.ApplyFilterToFindSelectedProperties(MapToAddTo);
}
