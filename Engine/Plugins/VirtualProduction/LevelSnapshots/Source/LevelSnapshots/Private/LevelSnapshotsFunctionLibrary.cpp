// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsFunctionLibrary.h"

#include "Filtering/Diffing/ApplySnapshotFilter.h"
#include "LevelSnapshot.h"
#include "LevelSnapshotFilters.h"

#include "EngineUtils.h"
#include "LevelSnapshotsFilteringLibrary.h"
#include "CustomSerialization/CustomObjectSerializationWrapper.h"
#include "UObject/Package.h"

ULevelSnapshot* ULevelSnapshotsFunctionLibrary::TakeLevelSnapshot(const UObject* WorldContextObject, const FName NewSnapshotName, const FString Description)
{
	return TakeLevelSnapshot_Internal(WorldContextObject, NewSnapshotName, nullptr, Description);
}

ULevelSnapshot* ULevelSnapshotsFunctionLibrary::TakeLevelSnapshot_Internal(const UObject* WorldContextObject, const FName NewSnapshotName, UPackage* InPackage, const FString Description)
{
	UWorld* TargetWorld = nullptr;
	if (WorldContextObject)
	{
		TargetWorld = WorldContextObject->GetWorld();
	}

	if (!ensure(TargetWorld))
	{
		return nullptr;
	}
	
	ULevelSnapshot* NewSnapshot = NewObject<ULevelSnapshot>(InPackage ? InPackage : GetTransientPackage(), NewSnapshotName, RF_NoFlags);
	NewSnapshot->SetSnapshotName(NewSnapshotName);
	NewSnapshot->SetSnapshotDescription(Description);
	NewSnapshot->SnapshotWorld(TargetWorld);
	return NewSnapshot;
}

void ULevelSnapshotsFunctionLibrary::ApplySnapshotToWorld(const UObject* WorldContextObject, ULevelSnapshot* Snapshot, ULevelSnapshotFilter* OptionalFilter)
{
	UWorld* TargetWorld = WorldContextObject ? WorldContextObject->GetWorld() : nullptr;
	if (ensure(TargetWorld && Snapshot))
	{
		const FPropertySelectionMap SelectionMap = ULevelSnapshotsFilteringLibrary::DiffAndFilterSnapshot(TargetWorld, Snapshot, OptionalFilter);
		Snapshot->ApplySnapshotToWorld(TargetWorld, SelectionMap);
	}
}

void ULevelSnapshotsFunctionLibrary::ApplyFilterToFindSelectedProperties(
	ULevelSnapshot* Snapshot,
	FPropertySelectionMap& MapToAddTo, 
	AActor* WorldActor,
	AActor* DeserializedSnapshotActor,
	const ULevelSnapshotFilter* Filter,
	bool bAllowUnchangedProperties,
    bool bAllowNonEditableProperties)
{
	ULevelSnapshotsFilteringLibrary::ApplyFilterToFindSelectedProperties(Snapshot, MapToAddTo, WorldActor, DeserializedSnapshotActor, Filter);
}

void ULevelSnapshotsFunctionLibrary::ForEachMatchingCustomSubobjectPair(
	ULevelSnapshot* Snapshot,
	UObject* SnapshotRootObject,
	UObject* WorldRootObject,
	TFunctionRef<void(UObject* SnapshotSubobject, UObject* EditorWorldSubobject)> HandleCustomSubobjectPair,
	TFunctionRef<void(UObject* UnmatchedSnapshotSubobject)> HandleUnmatchedSnapshotSubobject)
{
	UE::LevelSnapshots::Private::ForEachMatchingCustomSubobjectPair(Snapshot->GetSerializedData(), SnapshotRootObject, WorldRootObject, HandleCustomSubobjectPair, HandleUnmatchedSnapshotSubobject);
}
