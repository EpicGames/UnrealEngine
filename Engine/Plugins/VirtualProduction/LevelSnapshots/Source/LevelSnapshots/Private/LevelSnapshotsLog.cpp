// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsLog.h"

UE_TRACE_CHANNEL_DEFINE(LevelSnapshotsChannel, "Traces Level Snapshots operations for level state management including snapshot creation, comparison, restoration operations, and level state change tracking.");