// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsModule.h"

#include "ClassRestorationSkipper.h"
#include "LevelSnapshotsSettings.h"
#include "LevelSnapshotsLog.h"
#include "Params/PropertyComparisonParams.h"
#include "SpecialSupport/EngineTypesRestorationFence.h"

#include "Algo/AllOf.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

namespace UE::LevelSnapshots::Private
{
	/** @param TPredicate signature IActorSnapshotFilter::FFilterResultData(IActorSnapshotFilter&) */
	template<typename TPredicate>
	static bool CanRestoreActor(const TArray<TSharedRef<IActorSnapshotFilter>>& GlobalFilters, FText* ExclusionReason, TPredicate Predicate)
	{
		for (const TSharedRef<IActorSnapshotFilter>& Filter : GlobalFilters)
		{
			const IActorSnapshotFilter::FFilterResultData Result = Predicate(Filter.Get());
			switch (Result.InclusionResult)
			{
			case IActorSnapshotFilter::EFilterResult::Allow: return true;
			case IActorSnapshotFilter::EFilterResult::DoNotCare: continue;
			case IActorSnapshotFilter::EFilterResult::Disallow: 
				if (ExclusionReason && !Result.ExclusionReason.IsEmpty())
				{
					*ExclusionReason = Result.ExclusionReason;
				}
				return false;
			default:
				checkNoEntry();
			}
		};
		return true;
	}
}

UE::LevelSnapshots::Private::FLevelSnapshotsModule& UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance()
{
	static UE::LevelSnapshots::Private::FLevelSnapshotsModule& ModuleInstance = *[]() -> UE::LevelSnapshots::Private::FLevelSnapshotsModule*
	{
		UE_CLOGF(!FModuleManager::Get().IsModuleLoaded("LevelSnapshots"), LogLevelSnapshots, Fatal, "You called GetInternalModuleInstance before the module was initialised.");
		return &FModuleManager::GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
	}();
	return ModuleInstance;
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::StartupModule()
{
	// Hook up project settings
	const TSharedRef<FClassRestorationSkipper> ClassSkipper = MakeShared<FClassRestorationSkipper>(
		FClassRestorationSkipper::FGetSkippedClassList::CreateLambda([]() -> const FSkippedClassList&
		{
			ULevelSnapshotsSettings* Settings = GetMutableDefault<ULevelSnapshotsSettings>();
			return Settings->SkippedClasses;
		})
	);
	RegisterRestorabilityOverrider(ClassSkipper);
	
	// Add support for engine features that require specialized case handling.
	// Since we may depend on dynamic class look-ups (e.g. FPCGRestoration), this is done in OnPostEngineInit.
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
	{
		EngineTypesRestorationFence::RegisterSpecialEngineTypeSupport(*this);
	});

	OnPreTakeSnapshot().AddRaw(this, &FLevelSnapshotsModule::PreTakeSnapshot);
	OnPostTakeSnapshot().AddRaw(this, &FLevelSnapshotsModule::PostTakeSnapshot);

#if WITH_EDITOR
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	{
		// User Project Settings
		SettingsModule.RegisterSettings("Project", "Plugins", "Level Snapshots",
			NSLOCTEXT("LevelSnapshots", "LevelSnapshotsSettingsCategoryDisplayName", "Level Snapshots"),
			NSLOCTEXT("LevelSnapshots", "LevelSnapshotsSettingsDescription", "Configure the Level Snapshots user settings"),
			GetMutableDefault<ULevelSnapshotsSettings>());
	}
#endif
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::ShutdownModule()
{
	Overrides.Reset();
	PropertyComparers.Reset();
	CustomSerializers.Reset();
	RestorationListeners.Reset();
	
#if WITH_EDITOR
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	{
		SettingsModule.UnregisterSettings("Project", "Plugins", "Level Snapshots");
	}
#endif
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterRestorabilityOverrider(TSharedRef<ISnapshotRestorabilityOverrider> Overrider)
{
	Overrides.AddUnique(Overrider);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterRestorabilityOverrider(const TSharedRef<ISnapshotRestorabilityOverrider>& Overrider)
{
	Overrides.RemoveSwap(Overrider);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::AddSkippedSubobjectClasses(const TSet<UClass*>& Classes)
{
	for (UClass* Class : Classes)
	{
		check(Class);

		if (!Class
			|| !ensureAlwaysMsgf(!Class->IsChildOf(AActor::StaticClass()), TEXT("Invalid function input: Actors can never be subobjects. Check your code."))
			|| !ensureAlwaysMsgf(!Class->IsChildOf(UActorComponent::StaticClass()), TEXT("Invalid function input: Disallow components using RegisterRestorabilityOverrider and implementing ISnapshotRestorabilityOverrider::IsComponentDesirableForCapture instead.")))
		{
			continue;
		}

		SkippedSubobjectClasses.Add(Class);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RemoveSkippedSubobjectClasses(const TSet<UClass*>& Classes)
{
	for (UClass* Class : Classes)
	{
		SkippedSubobjectClasses.Remove(Class);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterPropertyComparer(UClass* Class, TSharedRef<IPropertyComparer> Comparer)
{
	PropertyComparers.FindOrAdd(Class).AddUnique(Comparer);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterPropertyComparer(UClass* Class, const TSharedRef<IPropertyComparer>& Comparer)
{
	TArray<TSharedRef<IPropertyComparer>>* Comparers = PropertyComparers.Find(Class);
	if (!Comparers)
	{
		return;
	}
	Comparers->RemoveSwap(Comparer);

	if (Comparers->Num() == 0)
	{
		PropertyComparers.Remove(Class);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterCustomObjectSerializer(UClass* Class, TSharedRef<ICustomObjectSnapshotSerializer> CustomSerializer, bool bIncludeBlueprintChildClasses)
{
	if (!ensureAlways(Class))
	{
		return;
	}
	
	const bool bIsBlueprint = Class->IsInBlueprint();
	if (!ensureAlwaysMsgf(!bIsBlueprint, TEXT("Registering to Blueprint classes is unsupported because they can be reinstanced at any time")))
	{
		return;
	}

	FCustomSerializer* ExistingSerializer = CustomSerializers.Find(Class);
	if (!ensureAlwaysMsgf(!ExistingSerializer, TEXT("Class already registered")))
	{
		return;
	}

	CustomSerializers.Add(Class, { CustomSerializer, bIncludeBlueprintChildClasses });
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterCustomObjectSerializer(UClass* Class)
{
	CustomSerializers.Remove(Class);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterGlobalActorFilter(TSharedRef<IActorSnapshotFilter> Filter)
{
	GlobalFilters.AddUnique(MoveTemp(Filter));
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterGlobalActorFilter(const TSharedRef<IActorSnapshotFilter>& Filter)
{
	GlobalFilters.RemoveSingle(Filter);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterSnapshotLoader(TSharedRef<ISnapshotLoader> Loader)
{
	SnapshotLoaders.AddUnique(MoveTemp(Loader));
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterSnapshotLoader(const TSharedRef<ISnapshotLoader>& Loader)
{
	SnapshotLoaders.RemoveSingle(Loader);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterRestorationListener(TSharedRef<IRestorationListener> Listener)
{
	RestorationListeners.AddUnique(MoveTemp(Listener));
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterRestorationListener(const TSharedRef<IRestorationListener>& Listener)
{
	RestorationListeners.RemoveSingle(Listener);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterTakeSnapshotListener(TSharedRef<ITakeSnapshotListener> Extender)
{
	TakeSnapshotListenersListeners.AddUnique(MoveTemp(Extender));
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterTakeSnapshotListener(const TSharedRef<ITakeSnapshotListener>& Listener)
{
	TakeSnapshotListenersListeners.RemoveSingle(Listener);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RegisterSnapshotFilterExtender(TSharedRef<ISnapshotFilterExtender> Extender)
{
	FilterExtenders.AddUnique(MoveTemp(Extender));
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::UnregisterSnapshotFilterExtender(const TSharedRef<ISnapshotFilterExtender>& Listener)
{
	FilterExtenders.RemoveSingle(Listener);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::AddExplicitilySupportedProperties(const TSet<const FProperty*>& Properties)
{
	for (const FProperty* Property : Properties)
	{
		SupportedProperties.Add(Property);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RemoveAdditionallySupportedProperties(const TSet<const FProperty*>& Properties)
{
	for (const FProperty* Property : Properties)
	{
		SupportedProperties.Remove(Property);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::AddExplicitlyUnsupportedProperties(const TSet<const FProperty*>& Properties)
{
	for (const FProperty* Property : Properties)
	{
		UnsupportedProperties.Add(Property);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RemoveExplicitlyUnsupportedProperties(const TSet<const FProperty*>& Properties)
{
	for (const FProperty* Property : Properties)
	{
		UnsupportedProperties.Remove(Property);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::AddSkippedClassDefault(const UClass* Class)
{
	SkippedCDOs.Add(Class);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RemoveSkippedClassDefault(const UClass* Class)
{
	SkippedCDOs.Remove(Class);
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::ShouldSkipClassDefaultSerialization(const UClass* Class) const
{
	for (const UClass* CurrentClass = Class; CurrentClass; CurrentClass = CurrentClass->GetSuperClass())
	{
		if (SkippedCDOs.Contains(CurrentClass))
		{
			return true;
		}
	}

	return false;
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::ShouldSkipSubobjectClass(const UClass* Class) const
{
	if (Class->IsChildOf(UActorComponent::StaticClass()))
	{
		return false;
	}

	bool bFoundSkippedClass = false;
	for (const UClass* CurrentClass = Class; !bFoundSkippedClass && CurrentClass; CurrentClass = CurrentClass->GetSuperClass())
	{
		bFoundSkippedClass = SkippedSubobjectClasses.Contains(CurrentClass);
	}

	return bFoundSkippedClass;
}

const TArray<TSharedRef<UE::LevelSnapshots::ISnapshotRestorabilityOverrider>>& UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetOverrides() const
{
	return Overrides;
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::IsPropertyExplicitlySupported(const FProperty* Property) const
{
	return SupportedProperties.Contains(Property);
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::IsPropertyExplicitlyUnsupported(const FProperty* Property) const
{
	return UnsupportedProperties.Contains(Property);
}

UE::LevelSnapshots::Private::FPropertyComparerArray UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetPropertyComparerForClass(UClass* Class) const
{
	FPropertyComparerArray Result;
	for (UClass* CurrentClass = Class; CurrentClass; CurrentClass = CurrentClass->GetSuperClass())
	{
		const TArray<TSharedRef<IPropertyComparer>>* Comparers = PropertyComparers.Find(CurrentClass);
		if (Comparers)
		{
			Result.Append(*Comparers);
		}
	}
	return Result;
}

UE::LevelSnapshots::IPropertyComparer::EPropertyComparison UE::LevelSnapshots::Private::FLevelSnapshotsModule::ShouldConsiderPropertyEqual(const FPropertyComparerArray& Comparers, const FPropertyComparisonParams& Params) const
{
	for (const TSharedRef<IPropertyComparer>& Comparer : Comparers)
	{
		const IPropertyComparer::EPropertyComparison Result = Comparer->ShouldConsiderPropertyEqual(Params);
		if (Result != IPropertyComparer::EPropertyComparison::CheckNormally)
		{
			return Result;
		}
	}
	return IPropertyComparer::EPropertyComparison::CheckNormally;
}

TSharedPtr<UE::LevelSnapshots::ICustomObjectSnapshotSerializer> UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetCustomSerializerForClass(UClass* Class) const
{
    SCOPED_SNAPSHOT_CORE_TRACE(GetCustomSerializerForClass);
    
    const bool bWasInBlueprint = Class->IsInBlueprint();
	while (Class)
    {
        if (const FCustomSerializer* Result = CustomSerializers.Find(Class); Result && (!bWasInBlueprint || Result->bIncludeBlueprintChildren))
        {
            return  Result->Serializer;
        }
		
        Class = Class->GetSuperClass();
    }

	return nullptr;
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::CanModifyMatchedActor(const FCanModifyMatchedActorParams& Params, FText* ExclusionReason) const
{
	return CanRestoreActor(
		GlobalFilters,
		ExclusionReason,
		[&Params](IActorSnapshotFilter& Filter){ return Filter.CanModifyMatchedActor(Params); }
		);
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::CanRecreateActor(const FCanRecreateActorParams& Params, FText* ExclusionReason) const
{
	return CanRestoreActor(
		GlobalFilters,
		ExclusionReason,
		[&Params](IActorSnapshotFilter& Filter){ return Filter.CanRecreateMissingActor(Params); }
		);
}

bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::CanDeleteActor(const AActor* EditorActor, FText* ExclusionReason) const
{
	return CanRestoreActor(
		GlobalFilters,
		ExclusionReason,
		[EditorActor](IActorSnapshotFilter& Filter){ return Filter.CanDeleteNewActor(EditorActor); }
		);
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::AddCanTakeSnapshotDelegate(FName DelegateName, FCanTakeSnapshot Delegate)
{
	CanTakeSnapshotDelegates.FindOrAdd(DelegateName) = Delegate;
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::RemoveCanTakeSnapshotDelegate(FName DelegateName)
{
	CanTakeSnapshotDelegates.Remove(DelegateName);
}


bool UE::LevelSnapshots::Private::FLevelSnapshotsModule::CanTakeSnapshot(const FPreTakeSnapshotEventData& Event) const
{
	return Algo::AllOf(CanTakeSnapshotDelegates, [&Event](const TTuple<FName,FCanTakeSnapshot>& Pair)
		{
			if (Pair.Get<1>().IsBound())
			{
				return Pair.Get<1>().Execute(Event);
			}
			return true;
		});
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostLoadSnapshotObject(const FPostLoadSnapshotObjectParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(SnapshotLoaders);
	
	for (const TSharedRef<ISnapshotLoader>& Loader : SnapshotLoaders)
	{
		Loader->PostLoadSnapshotObject(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreApplySnapshot(const FPreApplySnapshotParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PreApplySnapshot(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostApplySnapshot(const FPostApplySnapshotParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostApplySnapshot(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PreApplySnapshotProperties(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostApplySnapshotProperties(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreApplySnapshotToActor(const FApplySnapshotToActorParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PreApplySnapshotToActor(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostApplySnapshotToActor(const FApplySnapshotToActorParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostApplySnapshotToActor(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreCreateActor(UWorld* World, TSubclassOf<AActor> ActorClass, FActorSpawnParameters& InOutSpawnParams)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PreRecreateActor(World, ActorClass, InOutSpawnParams);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostRecreateActor(AActor* Actor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostRecreateActor(Actor);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreRemoveActor(const FPreRemoveActorParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Listener->PreRemoveActor(Params.ActorToRemove);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Listener->PreRemoveActor(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostRemoveActors(const FPostRemoveActorsParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostRemoveActors(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreRecreateComponent(const FPreRecreateComponentParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PreRecreateComponent(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostRecreateComponent(UActorComponent* RecreatedComponent)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);
	
	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostRecreateComponent(RecreatedComponent);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreRemoveComponent(UActorComponent* ComponentToRemove)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);

	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PreRemoveComponent(ComponentToRemove);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostRemoveComponent(const FPostRemoveComponentParams& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(RestorationListeners);

	for (const TSharedRef<IRestorationListener>& Listener : RestorationListeners)
	{
		Listener->PostRemoveComponent(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPreTakeObjectSnapshot(const FPreTakeObjectSnapshotEvent& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(TakeSnapshotListeners);

	for (const TSharedRef<ITakeSnapshotListener>& Listener : TakeSnapshotListenersListeners)
	{
		Listener->PreTakeObjectSnapshot(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::OnPostTakeObjectSnapshot(const FPostTakeObjectSnapshotEvent& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(TakeSnapshotListeners);

	for (const TSharedRef<ITakeSnapshotListener>& Listener : TakeSnapshotListenersListeners)
	{
		Listener->PostTakeObjectSnapshot(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::PreTakeSnapshot(const FPreTakeSnapshotEventData& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(TakeSnapshotListeners);

	for (const TSharedRef<ITakeSnapshotListener>& Listener : TakeSnapshotListenersListeners)
	{
		Listener->PreTakeSnapshot(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::PostTakeSnapshot(const FPostTakeSnapshotEventData& Params)
{
	SCOPED_SNAPSHOT_CORE_TRACE(TakeSnapshotListeners);

	for (const TSharedRef<ITakeSnapshotListener>& Listener : TakeSnapshotListenersListeners)
	{
		Listener->PostTakeSnapshot(Params);
	}
}

void UE::LevelSnapshots::Private::FLevelSnapshotsModule::PreApplyFilters(const FPreApplyFiltersParams& Params)
{
	for (const TSharedRef<ISnapshotFilterExtender>& Extender : FilterExtenders)
	{
		Extender->PreApplyFilters(Params);
	}
}

UE::LevelSnapshots::FPostApplyFiltersResult UE::LevelSnapshots::Private::FLevelSnapshotsModule::PostApplyFilters(const FPostApplyFiltersParams& Params)
{
	FPostApplyFiltersResult Result;
	for (const TSharedRef<ISnapshotFilterExtender>& Extender : FilterExtenders)
	{
		Result.Combine(Extender->PostApplyFilters(Params));
	}
	return Result;
}

IMPLEMENT_MODULE(UE::LevelSnapshots::Private::FLevelSnapshotsModule, LevelSnapshots)
