// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILevelSnapshotsModule.h"
#include "UObject/SoftObjectPath.h"

namespace UE::LevelSnapshots
{
	struct FPropertyComparisonParams;
}

namespace UE::LevelSnapshots::Private
{
	// The array is used very often in many loops: optimize heap allocation
	using FPropertyComparerArray = TArray<TSharedRef<IPropertyComparer>, TInlineAllocator<4>>;

	class LEVELSNAPSHOTS_API FLevelSnapshotsModule : public ILevelSnapshotsModule
	{
	public:

		static FLevelSnapshotsModule& GetInternalModuleInstance();
		
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

		//~ Begin ILevelSnapshotsModule Interface
		virtual void RegisterRestorabilityOverrider(TSharedRef<ISnapshotRestorabilityOverrider> Overrider) override;
		virtual void UnregisterRestorabilityOverrider(const TSharedRef<ISnapshotRestorabilityOverrider>& Overrider) override;
		virtual void AddSkippedSubobjectClasses(const TSet<UClass*>& Classes) override;
		virtual void RemoveSkippedSubobjectClasses(const TSet<UClass*>& Classes) override;
		virtual void RegisterPropertyComparer(UClass* Class, TSharedRef<IPropertyComparer> Comparer) override;
		virtual void UnregisterPropertyComparer(UClass* Class, const TSharedRef<IPropertyComparer>& Comparer) override;
		virtual void RegisterCustomObjectSerializer(UClass* Class, TSharedRef<ICustomObjectSnapshotSerializer> CustomSerializer, bool bIncludeBlueprintChildClasses = true);
		virtual void UnregisterCustomObjectSerializer(UClass* Class) override;
		virtual void RegisterGlobalActorFilter(TSharedRef<IActorSnapshotFilter> Filter) override;
		virtual void UnregisterGlobalActorFilter(const TSharedRef<IActorSnapshotFilter>& Filter) override;
		virtual void RegisterSnapshotLoader(TSharedRef<ISnapshotLoader> Loader) override;
		virtual void UnregisterSnapshotLoader(const TSharedRef<ISnapshotLoader>& Loader) override;
		virtual void RegisterRestorationListener(TSharedRef<IRestorationListener> Listener) override;
		virtual void UnregisterRestorationListener(const TSharedRef<IRestorationListener>& Listener) override;
		virtual void RegisterTakeSnapshotListener(TSharedRef<ITakeSnapshotListener> Extender) override;
		virtual void UnregisterTakeSnapshotListener(const TSharedRef<ITakeSnapshotListener>& Listener) override;
		virtual void RegisterSnapshotFilterExtender(TSharedRef<ISnapshotFilterExtender> Extender) override;
		virtual void UnregisterSnapshotFilterExtender(const TSharedRef<ISnapshotFilterExtender>& Listener) override;
		virtual void AddExplicitilySupportedProperties(const TSet<const FProperty*>& Properties) override;
		virtual void RemoveAdditionallySupportedProperties(const TSet<const FProperty*>& Properties) override;
		virtual void AddExplicitlyUnsupportedProperties(const TSet<const FProperty*>& Properties) override;
		virtual void RemoveExplicitlyUnsupportedProperties(const TSet<const FProperty*>& Properties) override;
		virtual void AddSkippedClassDefault(const UClass* Class) override;
		virtual void RemoveSkippedClassDefault(const UClass* Class) override;
		virtual bool ShouldSkipClassDefaultSerialization(const UClass* Class) const override;
		//~ Begin ILevelSnapshotsModule Interface

		bool ShouldSkipSubobjectClass(const UClass* Class) const;
		
		const TArray<TSharedRef<ISnapshotRestorabilityOverrider>>& GetOverrides() const;
		bool IsPropertyExplicitlySupported(const FProperty* Property) const;
		bool IsPropertyExplicitlyUnsupported(const FProperty* Property) const;

		FPropertyComparerArray GetPropertyComparerForClass(UClass* Class) const;
		IPropertyComparer::EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparerArray& Comparers, const FPropertyComparisonParams& Params) const;

		TSharedPtr<ICustomObjectSnapshotSerializer> GetCustomSerializerForClass(UClass* Class) const;

		bool CanModifyMatchedActor(const FCanModifyMatchedActorParams& Params, FText* ExclusionReason = nullptr) const;
		bool CanRecreateActor(const FCanRecreateActorParams& Params, FText* ExclusionReason = nullptr) const;
		bool CanDeleteActor(const AActor* EditorActor, FText* ExclusionReason = nullptr) const;

		virtual void AddCanTakeSnapshotDelegate(FName DelegateName, FCanTakeSnapshot Delegate) override;
		virtual void RemoveCanTakeSnapshotDelegate(FName DelegateName) override;
		virtual bool CanTakeSnapshot(const FPreTakeSnapshotEventData& Event) const override;

		void OnPostLoadSnapshotObject(const FPostLoadSnapshotObjectParams& Params);

		void OnPreApplySnapshot(const FPreApplySnapshotParams& Params);
		void OnPostApplySnapshot(const FPostApplySnapshotParams& Params);
		
		void OnPreApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params);
		void OnPostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params);

		void OnPreApplySnapshotToActor(const FApplySnapshotToActorParams& Params);
		void OnPostApplySnapshotToActor(const FApplySnapshotToActorParams& Params);

		void OnPreCreateActor(UWorld* World, TSubclassOf<AActor> ActorClass, FActorSpawnParameters& InOutSpawnParams);
		void OnPostRecreateActor(AActor* Actor);
		
		void OnPreRemoveActor(const FPreRemoveActorParams& Params);
		void OnPostRemoveActors(const FPostRemoveActorsParams& Params);
		
		void OnPreRecreateComponent(const FPreRecreateComponentParams& Params);
		void OnPostRecreateComponent(UActorComponent* RecreatedComponent);

		void OnPreRemoveComponent(UActorComponent* ComponentToRemove);
		void OnPostRemoveComponent(const FPostRemoveComponentParams& Params);

		void OnPreTakeObjectSnapshot(const FPreTakeObjectSnapshotEvent& Params);
		void OnPostTakeObjectSnapshot(const FPostTakeObjectSnapshotEvent& Params);

		void PreApplyFilters(const FPreApplyFiltersParams& Params);
		FPostApplyFiltersResult PostApplyFilters(const FPostApplyFiltersParams& Params);
		
	private:

		struct FCustomSerializer
		{
			TSharedRef<ICustomObjectSnapshotSerializer> Serializer;
			bool bIncludeBlueprintChildren;
		};
		
		/* Allows external modules to override what objects and properties are considered by the snapshot system. */
		TArray<TSharedRef<ISnapshotRestorabilityOverrider>> Overrides;

		/** Subobject classes we do not capture nor restore */
		TSet<UClass*> SkippedSubobjectClasses;
		
		TMap<FSoftClassPath, TArray<TSharedRef<IPropertyComparer>>> PropertyComparers;
		TMap<FSoftClassPath, FCustomSerializer> CustomSerializers;

		TArray<TSharedRef<IActorSnapshotFilter>> GlobalFilters;
		TArray<TSharedRef<ISnapshotLoader>> SnapshotLoaders;
		TArray<TSharedRef<IRestorationListener>> RestorationListeners;
		TArray<TSharedRef<ITakeSnapshotListener>> TakeSnapshotListenersListeners;
		TArray<TSharedRef<ISnapshotFilterExtender>> FilterExtenders;

		/* Allows these properties even when the default behaviour would exclude them. */
		TSet<const FProperty*> SupportedProperties;
		/* Forbid these properties even when the default behaviour would include them. */
		TSet<const FProperty*> UnsupportedProperties;

		/** Classes for which to not serialize class default */
		TSet<FSoftClassPath> SkippedCDOs;

		/** Map of named delegates for confirming that a level snapshot is possible. */
		TMap<FName, FCanTakeSnapshot> CanTakeSnapshotDelegates;
		
		void PreTakeSnapshot(const FPreTakeSnapshotEventData& Params);
		void PostTakeSnapshot(const FPostTakeSnapshotEventData& Params);
	};
}

