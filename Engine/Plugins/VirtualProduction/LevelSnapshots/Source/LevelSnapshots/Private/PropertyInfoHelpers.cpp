// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyInfoHelpers.h"

#include "UObject/UnrealType.h"

float UE::LevelSnapshots::GFloatComparisonPrecision = 1e-03f;
double UE::LevelSnapshots::GDoubleComparisonPrecision = 1e-03;

FProperty* UE::LevelSnapshots::GetParentProperty(const FProperty* Property)
{
	return Property->GetOwner<FProperty>();
}

bool UE::LevelSnapshots::IsPropertyContainer(const FProperty* Property)
{
	return ensure(Property) && (Property->IsA(FStructProperty::StaticClass()) || Property->IsA(FArrayProperty::StaticClass()) ||
			Property->IsA(FMapProperty::StaticClass()) || Property->IsA(FSetProperty::StaticClass()));
}

bool UE::LevelSnapshots::IsPropertyCollection(const FProperty* Property)
{
	return ensure(Property) && (Property->IsA(FArrayProperty::StaticClass()) || Property->IsA(FMapProperty::StaticClass()) || Property->IsA(FSetProperty::StaticClass()));
}

bool UE::LevelSnapshots::IsPropertyInContainer(const FProperty* Property)
{
	if (ensure(Property))
	{
		return IsPropertyInCollection(Property) || IsPropertyInStruct(Property);
	}
	return false;
}

bool UE::LevelSnapshots::IsPropertyInCollection(const FProperty* Property)
{
	if (ensure(Property))
	{
		const FProperty* ParentProperty = GetParentProperty(Property);
		return ParentProperty && IsPropertyCollection(ParentProperty);
	}
	return false;
}

bool UE::LevelSnapshots::IsPropertyInStruct(const FProperty* Property)
{
	if (!ensure(Property))
	{
		return false;
	}

	// Parent struct could be FProperty or UScriptStruct

	if (FProperty* ParentProperty = GetParentProperty(Property))
	{
		return ParentProperty->IsA(FStructProperty::StaticClass());
	}

	return IsValid(Property->GetOwner<UScriptStruct>());
}

bool UE::LevelSnapshots::IsPropertyInMap(const FProperty* Property)
{
	if (!ensure(Property))
	{
		return false;
	}

	FProperty* ParentProperty = GetParentProperty(Property);

	return ParentProperty && ParentProperty->IsA(FMapProperty::StaticClass());
}

bool UE::LevelSnapshots::IsPropertyComponentOrSubobject(const FProperty* Property)
{
	const bool bIsComponentProp = !!(Property->PropertyFlags & (CPF_InstancedReference | CPF_ContainsInstancedReference));
	return bIsComponentProp;
}

void UE::LevelSnapshots::UpdateDecimalComparisionPrecision(float FloatPrecision, double DoublePrecision)
{
	GFloatComparisonPrecision = FloatPrecision;
	GDoubleComparisonPrecision = DoublePrecision;
}

bool UE::LevelSnapshots::AreNumericPropertiesNearlyEqual(const FNumericProperty* NumericProperty, const void* ValuePtrA, const void* ValuePtrB)
{
	check(NumericProperty);
	
	if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(NumericProperty))
	{
		const float ValueA = FloatProperty->GetFloatingPointPropertyValue(ValuePtrA); 
		const float ValueB = FloatProperty->GetFloatingPointPropertyValue(ValuePtrB);
		return FMath::IsNearlyEqual(ValueA, ValueB, GFloatComparisonPrecision);
	}
	
	if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(NumericProperty))
	{
		const double ValueA = DoubleProperty->GetFloatingPointPropertyValue(ValuePtrA);
		const double ValueB = DoubleProperty->GetFloatingPointPropertyValue(ValuePtrB);
		return FMath::IsNearlyEqual(ValueA, ValueB, GDoubleComparisonPrecision);
	}
	
	// Not a float or double? Then some kind of integer (byte, int8, int16, int32, int64, uint8, uint16 ...). Enums are bytes.
	const int64 ValueA = NumericProperty->GetSignedIntPropertyValue(ValuePtrA);
	const int64 ValueB = NumericProperty->GetSignedIntPropertyValue(ValuePtrB);
	return ValueA == ValueB;
}
