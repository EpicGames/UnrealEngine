// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Templates/NonNullPointer.h"

class UPackage;
class UWorld;

struct FActorSnapshotData;
struct FPropertySelectionMap;
struct FSnapshotDataCache;
struct FSoftObjectPath;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/** Allocates the actor, if no yet, and initialises it in the snapshot world. */
	TOptional<TNonNullPtr<AActor>> GetDeserializedActor(const FSoftObjectPath& OriginalObjectPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* LocalisationSnapshotPackage, bool bWarnIfNotFound = true);

	/** Gets the actor class from the original actor soft path */
	TOptional<TNonNullPtr<UClass>> GetActorClass(const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, bool bWarnIfNotFound = true);

	TOptional<TNonNullPtr<AActor>> GetPreallocatedIfCached(const FSoftObjectPath& OriginalActorPath, const FSnapshotDataCache& Cache);

	/** Gets the actor allocated in the snapshot world or allocates it and all of its subobjects. */
	TOptional<TNonNullPtr<AActor>> GetPreallocated(const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, bool bWarnIfNotFound = true);

	/** Restores selected properties into the actor and its subobjects: the actor existed before the snapshot was applied. */
	void RestoreIntoExistingWorldActor(AActor* OriginalActor, FActorSnapshotData& ActorData, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage, const FPropertySelectionMap& SelectedProperties);

	/** Restores all properties into the actor and its subobjects: the actor was recreated as part of the restoration process. */
	void RestoreIntoRecreatedEditorWorldActor(AActor* OriginalActor, FActorSnapshotData& ActorData, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage, const FPropertySelectionMap& SelectedProperties);
}
