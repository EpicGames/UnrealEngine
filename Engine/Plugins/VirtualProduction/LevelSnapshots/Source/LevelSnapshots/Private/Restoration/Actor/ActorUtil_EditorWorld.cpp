// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplySnapshotToEditorArchive.h"

#include "CustomSerialization/CustomObjectSerializationWrapper.h"
#include "Data/ActorSnapshotData.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Filtering/PropertySelectionMap.h"
#include "Restoration/Actor/ActorUtil.h"
#include "Restoration/Component/SnapshotComponentUtil.h"
#include "Restoration/Scope/ApplySnapshotToActorScope.h"
#include "Data/WorldSnapshotData.h"
#include "LevelSnapshotsLog.h"

#include "Components/ActorComponent.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "Templates/NonNullPointer.h"
#include "UObject/Package.h"

#if USE_STABLE_LOCALIZATION_KEYS
#include "Internationalization/TextPackageNamespaceUtil.h"
#endif

namespace UE::LevelSnapshots::Private::Internal::Restore
{
	using FSerializeActor = TFunctionRef<void(AActor* OriginalActor, AActor* DerserializedActor)>;
	using FSerializeComponent = TFunctionRef<void(FSubobjectSnapshotData& SerializedCompData, UActorComponent* Original, UActorComponent* Deserialized)>;
	using FHandleFoundComponent = TFunctionRef<void(FSubobjectSnapshotData& SerializedCompData, UActorComponent* ActorComp)>;

	static void DeserializeComponents(
		AActor* IntoActor,
		const FActorSnapshotData& ActorData,
		FWorldSnapshotData& WorldData,
		FHandleFoundComponent HandleComponent
		)
	{
		for (auto CompIt = ActorData.ComponentData.CreateConstIterator(); CompIt; ++CompIt)
		{
			const EComponentCreationMethod CreationMethod = CompIt->Value.CreationMethod;
			if (CreationMethod == EComponentCreationMethod::UserConstructionScript)	// Construction script components are not supported 
				{
				continue;
				}

			const int32 SubobjectIndex = CompIt->Key;
			const FSoftObjectPath& OriginalComponentPath = WorldData.SerializedObjectReferences[SubobjectIndex];
			if (UActorComponent* ComponentToRestore = UE::LevelSnapshots::Private::FindMatchingComponent(IntoActor, OriginalComponentPath))
			{
				FSubobjectSnapshotData& SnapshotData = WorldData.Subobjects[SubobjectIndex];
				HandleComponent(SnapshotData, ComponentToRestore);
			}
		}
	}

	static void DeserializeIntoEditorWorldActor(AActor* OriginalActor, FActorSnapshotData& ActorData, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage, FSerializeActor SerializeActor, FSerializeComponent SerializeComponent)
	{
		const TOptional<TNonNullPtr<AActor>> Deserialized = UE::LevelSnapshots::Private::GetDeserializedActor(OriginalActor, WorldData, Cache, InLocalisationSnapshotPackage);
		if (!Deserialized)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Failed to serialize into actor %ls. Skipping...", *OriginalActor->GetName());
			return;
		}

		const AActor* AttachParentBeforeRestore = OriginalActor->GetAttachParentActor();
		SerializeActor(OriginalActor, Deserialized.GetValue());
#if WITH_EDITOR
		UE_LOGF(LogLevelSnapshots, Verbose, "ActorLabel is \"%ls\" for \"%ls\"", *OriginalActor->GetActorLabel(), *OriginalActor->GetPathName());
#endif

		TInlineComponentArray<UActorComponent*> DeserializedComponents(Deserialized.GetValue());
		Internal::Restore::DeserializeComponents(OriginalActor, ActorData, WorldData,
			[&SerializeComponent, &DeserializedComponents](
				FSubobjectSnapshotData& SerializedCompData,
				UActorComponent* Comp
				)
		    {
		        const FName OriginalCompName = Comp->GetFName();
		        UActorComponent** DeserializedCompCounterpart = DeserializedComponents.FindByPredicate([OriginalCompName](UActorComponent* Other)
		        {
		            return Other->GetFName() == OriginalCompName;
		        });
				
				UE_CLOGF(!DeserializedCompCounterpart, LogLevelSnapshots, Warning, "Failed to find component called %ls on temp deserialized snapshot actor. Skipping component...", *OriginalCompName.ToString())
		        if (DeserializedCompCounterpart)
		        {
	        		SerializeComponent(SerializedCompData, Comp, *DeserializedCompCounterpart);
		        }
		    }
		);

		// Handle case in which AttachParentAfterRestore is restored after OriginalActor otherwise ReregisterAllComponents will detach OriginalActor again
		const AActor* AttachParentAfterRestore = OriginalActor->GetAttachParentActor();
		if (AttachParentAfterRestore && AttachParentAfterRestore->IsAttachedTo(OriginalActor))
		{
			AttachParentAfterRestore->GetRootComponent()->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		}

		// Update component state, e.g. render state if intensity for lights was changed. Also avoids us having to call PostEditChangeProperty on every property.
		// ReregisterAllComponents must be called before the UserConstructionScript because it processes calls we made to SetupAttachment
		OriginalActor->ReregisterAllComponents();
		{
			// GAllowActorScriptExecutionInEditor must be temporarily true so call to UserConstructionScript isn't skipped
			FEditorScriptExecutionGuard AllowConstructionScript;
			OriginalActor->UserConstructionScript();
		}

#if WITH_EDITOR
		// Update World Outliner. Usually called by USceneComponent::AttachToComponent.
		const bool bAttachParentChanged = AttachParentBeforeRestore != AttachParentAfterRestore;
		if (bAttachParentChanged)
		{
			GEngine->BroadcastLevelActorAttached(OriginalActor, AttachParentAfterRestore);
		}
#endif
	}

	static void PreventAttachParentInfiniteRecursion(UActorComponent* Original, const FPropertySelection& PropertySelection)
	{
		static const FProperty* AttachParent = USceneComponent::StaticClass()->FindPropertyByName(FName("AttachParent"));

		// Suppose snapshot contains Root > Child and now the hierarchy is Child > Root.
		// Root's AttachChildren still contains Child after we apply snapshot since that property is transient.
		// Solution: Detach now, then serialize AttachParent, and OnRegister will automatically call AttachToComponent and update everything.
		USceneComponent* SceneComponent = Cast<USceneComponent>(Original);
		if (Original && PropertySelection.IsPropertySelected(nullptr, AttachParent))
		{
			SceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		}
	}

	static void UpdateAttachParentAttachChildren(UActorComponent* Original)
	{
		// Original->AttachParent had its value serialized but new attach parent's AttachChildren must be updated, too
		USceneComponent* Component = Cast<USceneComponent>(Original);
		const bool bNeedsToUpdateParentAttachChildren = Component && Component->GetAttachParent()
			&& !Component->GetAttachParent()->GetAttachChildren().Contains(Component);
		if (bNeedsToUpdateParentAttachChildren)
		{
			// Hacky way of updating AttachChildren since there is no direct way...
			Component->UnregisterComponent();
			Component->SetupAttachment(Component->GetAttachParent(), Component->GetAttachSocketName());
			// ReregisterAllComponents will be called after the actor is done restoring which will fix up everything
		}
	}
}

void UE::LevelSnapshots::Private::RestoreIntoExistingWorldActor(AActor* OriginalActor, FActorSnapshotData& ActorData, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage, const FPropertySelectionMap& SelectedProperties)
{
	UE_LOGF(LogLevelSnapshots, Verbose, "========== Apply existing %ls ==========", *OriginalActor->GetPathName());
	const bool bWasRecreated = false;
	const FApplySnapshotToActorScope NotifyExternalListeners({ OriginalActor, SelectedProperties, bWasRecreated });
	
	UE::LevelSnapshots::Private::AddAndRemoveComponentsSelectedForRestore(OriginalActor, WorldData, Cache, SelectedProperties, InLocalisationSnapshotPackage);

	auto DeserializeActor = [&ActorData, &WorldData, &Cache, InLocalisationSnapshotPackage, &SelectedProperties](AActor* OriginalActor, AActor* DeserializedActor)
	{
		const FRestoreObjectScope FinishRestore = PreActorRestore_EditorWorld(OriginalActor, ActorData.CustomActorSerializationData, WorldData, Cache, SelectedProperties, InLocalisationSnapshotPackage);
		if (SelectedProperties.GetObjectSelection(OriginalActor).GetPropertySelection() != nullptr)
		{
			FApplySnapshotToEditorArchive::ApplyToExistingEditorWorldObject(ActorData.SerializedActorData, WorldData, Cache, OriginalActor, DeserializedActor, SelectedProperties, ActorData.ClassIndex);
		}
	};
	auto DeserializeComponent = [&WorldData, &Cache, &SelectedProperties, InLocalisationSnapshotPackage](FSubobjectSnapshotData& SerializedCompData, UActorComponent* Original, UActorComponent* Deserialized)
	{
		const FRestoreObjectScope FinishRestore = PreSubobjectRestore_EditorWorld(Deserialized, Original, WorldData, Cache, SelectedProperties, InLocalisationSnapshotPackage);
		const FPropertySelection* ComponentSelectedProperties = SelectedProperties.GetObjectSelection(Original).GetPropertySelection();
		if (ComponentSelectedProperties)
		{
			Internal::Restore::PreventAttachParentInfiniteRecursion(Original, *ComponentSelectedProperties);
			FApplySnapshotToEditorArchive::ApplyToExistingEditorWorldObject(SerializedCompData, WorldData, Cache, Original, Deserialized, SelectedProperties, SerializedCompData.ClassIndex);
			Internal::Restore::UpdateAttachParentAttachChildren(Original);
		};
	};
	Internal::Restore::DeserializeIntoEditorWorldActor(OriginalActor, ActorData, WorldData, Cache, InLocalisationSnapshotPackage, DeserializeActor, DeserializeComponent);
}

void UE::LevelSnapshots::Private::RestoreIntoRecreatedEditorWorldActor(AActor* OriginalActor, FActorSnapshotData& ActorData, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage, const FPropertySelectionMap& SelectedProperties)
{
	UE_LOGF(LogLevelSnapshots, Verbose, "========== Apply recreated %ls ==========", *OriginalActor->GetPathName());
	const bool bWasRecreated = true;
	const FApplySnapshotToActorScope NotifyExternalListeners({ OriginalActor, SelectedProperties, bWasRecreated });
	
	AllocateMissingComponentsForRecreatedActor(OriginalActor, WorldData, Cache);

	auto DeserializeActor = [&ActorData, &WorldData, &Cache, InLocalisationSnapshotPackage, &SelectedProperties](AActor* OriginalActor, AActor* DeserializedActor)
	{
		const FRestoreObjectScope FinishRestore = PreActorRestore_EditorWorld(OriginalActor, ActorData.CustomActorSerializationData, WorldData, Cache, SelectedProperties, InLocalisationSnapshotPackage);
		FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithArchetype(ActorData.SerializedActorData, WorldData, Cache, OriginalActor, SelectedProperties);
	};
	auto DeserializeComponent = [&WorldData, &Cache, &SelectedProperties, InLocalisationSnapshotPackage](FSubobjectSnapshotData& SerializedCompData, UActorComponent* Original, UActorComponent* Deserialized)
	{
		const FRestoreObjectScope FinishRestore = PreSubobjectRestore_EditorWorld(Deserialized, Original, WorldData, Cache, SelectedProperties, InLocalisationSnapshotPackage);
		// Components are not recreated with any archetype -> they must go through the full serialization process 
		FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithoutArchetype(SerializedCompData, WorldData, Cache, Original, SelectedProperties, SerializedCompData.ClassIndex);
		Internal::Restore::UpdateAttachParentAttachChildren(Original);
	};
	Internal::Restore::DeserializeIntoEditorWorldActor(OriginalActor, ActorData, WorldData, Cache, InLocalisationSnapshotPackage, DeserializeActor, DeserializeComponent);

#if WITH_EDITOR
	// Recreated actors have invalid lightning cache... e.g. recreated point lights will show error image (S_LightError)
	OriginalActor->InvalidateLightingCacheDetailed(false);
#endif
}
