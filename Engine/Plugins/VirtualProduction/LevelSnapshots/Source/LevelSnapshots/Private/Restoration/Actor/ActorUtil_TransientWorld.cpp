// Copyright Epic Games, Inc. All Rights Reserved.

#include "Restoration/Actor/ActorUtil.h"

#include "CustomSerialization/CustomObjectSerializationWrapper.h"
#include "Data/Util/ClassDataUtil.h"
#include "Data/ActorSnapshotData.h"
#include "Data/WorldSnapshotData.h"
#include "Data/SnapshotCustomVersion.h"
#include "Data/Util/ObjectDependencyUtil.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Filtering/PropertySelectionMap.h"
#include "Engine/Level.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "LoadSnapshotObjectArchive.h"
#include "Restoration/Component/SnapshotComponentUtil.h"
#include "SnapshotDataCache.h"
#include "Util/SoftObjectPathUtil.h"

#include "Components/ActorComponent.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Templates/NonNullPointer.h"
#include "UObject/Package.h"
#include "UObject/Script.h"

#if USE_STABLE_LOCALIZATION_KEYS
#include "Internationalization/TextPackageNamespaceUtil.h"
#endif

namespace UE::LevelSnapshots::Private::Internal
{
	static void PostSerializeSnapshotActor(AActor* SnapshotActor, const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage)
	{
		struct Local
		{
			static void RecreateRootComponentIfInstanced(AActor* OriginalActor, AActor* SnapshotActor)
			{
				USceneComponent* OriginalRoot = OriginalActor->GetRootComponent();
				
				const bool bSnapshotIsMissingRoot = SnapshotActor->GetRootComponent() == nullptr;
				// Some actors, e.g. exact instances of AActor, have no components and the editor creates an instanced root component
				const bool bSnapshotWasSavedWithoutRootComponentData = OriginalRoot && OriginalRoot->CreationMethod == EComponentCreationMethod::Instance;
				if (bSnapshotIsMissingRoot && bSnapshotWasSavedWithoutRootComponentData)
				{
					USceneComponent* SnapshotRoot = DuplicateObject<USceneComponent>(OriginalRoot, SnapshotActor, OriginalRoot->GetFName());
					UE_CLOGF(SnapshotRoot == nullptr, LogLevelSnapshots, Warning, "Failed to recreate instanced root component for %ls's snapshot counterpart", *OriginalActor->GetPathName());
					if (SnapshotRoot)
					{
						SnapshotActor->SetRootComponent(
							SnapshotRoot
						);
					}
				}
			}
			
			static void HandleAttachParentNotSaved(AActor* OriginalActor, AActor* SnapshotActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage)
			{
				USceneComponent* RootComponent = OriginalActor->GetRootComponent();
				// WorldSettings does not have a root component...
				if (!RootComponent)
				{
					return;
				}
				
				USceneComponent* WorldAttachParent = RootComponent->GetAttachParent();
				if (!WorldAttachParent)
				{
					return;
				}

				// We're covering the case of when the attach parent was not saved... if the current attach parent was not saved... we'll create our own warning
				constexpr bool bWarnIfNotFound = false;
				const TOptional<TNonNullPtr<AActor>> SnapshotAttachParent = GetDeserializedActor(WorldAttachParent->GetOwner(), WorldData, Cache, InLocalisationSnapshotPackage, bWarnIfNotFound);
				if (!SnapshotAttachParent)
				{
					UE_LOGF(LogLevelSnapshots, Warning,
						"Actor %ls had no attach parent saved. Its current attach parent %ls was added after the snapshot was added so we when this actor is restored, we'll fallback to unparenting it.",
						*OriginalActor->GetPathName(),
						*WorldAttachParent->GetOwner()->GetPathName());
					return;
				}

				const TInlineComponentArray<USceneComponent*> ComponentArray(SnapshotAttachParent.GetValue());
				USceneComponent* const* SnapshotAttachComponent = ComponentArray.FindByPredicate([WorldAttachParent](USceneComponent* Comp)
				{
					return Comp->GetFName() == WorldAttachParent->GetFName();
				});

				if (SnapshotAttachComponent)
				{
					SnapshotActor->AttachToComponent(
						*SnapshotAttachComponent,
						// Do not change property values: anything other than KeepRelativeTransform will recompute
						FAttachmentTransformRules::KeepRelativeTransform,
						RootComponent->GetAttachSocketName()
						);
				}
				else
				{
					UE_LOGF(LogLevelSnapshots, Warning, "Failed to fix up AttachParent for %ls's snapshot counterpart", *OriginalActor->GetPathName());
				}
			}
		};
		
		const int32 SavedVersion = WorldData.SnapshotVersionInfo.GetSnapshotCustomVersion();
		AActor* OriginalActor = Cast<AActor>(OriginalActorPath.ResolveObject());

		// USceneComponent::AttachParent was not yet saved.
		if (SavedVersion < FSnapshotCustomVersion::SubobjectSupport
			&& OriginalActor)
		{
			Local::RecreateRootComponentIfInstanced(OriginalActor, SnapshotActor);
			// To avoid lots of actors being unparented by accident, we set the snapshot actor's attach parent to the equivalent of the editor world
			Local::HandleAttachParentNotSaved(OriginalActor, SnapshotActor, WorldData, Cache, InLocalisationSnapshotPackage);
		}
	}

	using FHandleFoundComponent = TFunctionRef<void(FSubobjectSnapshotData& SerializedCompData, UActorComponent* ActorComp, const FSoftObjectPath& OriginalComponentPath, FWorldSnapshotData& SharedData)>;

	static void DeserializeComponents(
		const FActorSnapshotData& ActorData,
		AActor* IntoActor,
		FWorldSnapshotData& WorldData,
		FHandleFoundComponent HandleComponent
	)
	{
		for (auto CompIt = ActorData.ComponentData.CreateConstIterator(); CompIt; ++CompIt)
		{
			const EComponentCreationMethod CreationMethod = CompIt->Value.CreationMethod;
			if (CreationMethod == EComponentCreationMethod::UserConstructionScript)	// Construction script components are not supported 
			{
				continue;
			}

			const int32 SubobjectIndex = CompIt->Key;
			const FSoftObjectPath& OriginalComponentPath = WorldData.SerializedObjectReferences[SubobjectIndex];
			if (UActorComponent* ComponentToRestore = FindMatchingComponent(IntoActor, OriginalComponentPath))
			{
				FSubobjectSnapshotData& SnapshotData = WorldData.Subobjects[SubobjectIndex];
				HandleComponent(SnapshotData, ComponentToRestore, OriginalComponentPath, WorldData);
			}
		}
	}

	static void DeserializeSubobjectsForSnapshotActor(const FActorSnapshotData& ActorData, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, const FProcessObjectDependency& ProcessObjectDependency, UPackage* InLocalisationSnapshotPackage)
	{
		for (const int32 SubobjectIndex : ActorData.OwnedSubobjects)
		{
			check(!ActorData.ComponentData.Contains(SubobjectIndex));

			FString LocalisationNamespace;
#if USE_STABLE_LOCALIZATION_KEYS
			LocalisationNamespace = TextNamespaceUtil::EnsurePackageNamespace(InLocalisationSnapshotPackage);
#endif

			// Ensures the object is allocated and serialized into. Return value not needed.
			ResolveObjectDependencyForSnapshotWorld(
				WorldData,
				Cache,
				SubobjectIndex,
				ProcessObjectDependency,
				LocalisationNamespace
			);
		}
	}

	static void DeserializeIntoTransient(
		FSubobjectSnapshotData& SerializedComponentData,
		UActorComponent* ComponentToDeserializeInto,
		FWorldSnapshotData& WorldData,
		const FProcessObjectDependency& ProcessObjectDependency,
		FSnapshotDataCache& Cache,
		UPackage* InLocalisationSnapshotPackage)
	{
		if (ComponentToDeserializeInto->CreationMethod == EComponentCreationMethod::UserConstructionScript)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Components created dynamically in the construction script are not supported (%ls). Skipping...", *ComponentToDeserializeInto->GetPathName());
			return;
		}
		
		// We cannot supply a template object for components whose CreationMethod is not Instance.
		// Yet every actor class, native and Blueprint, has their own archetype for each component, i.e. in the details panel the property has a yellow "Reset to Default" icon next to it if the value is different from the Blueprint's value.
		// We apply the CDO data we saved before we apply the component's serialized data.
			// Scenario: We have a custom Blueprint StaticMeshBP that has a UStaticMeshComponent. Suppose we change some default property value, like ComponentTags.
			// 1st case: We change no default values, then take a snapshot.
			//   - After the snapshot was taken, we change a default value.
			//   - The value of the property was not saved in the component's serialized data because the value was equal to the CDO's value.
			//   - When applying the snapshot, we will override the new default value because we serialize the CDO here. Good.
			// 2nd case: We change some default value, then take a snapshot
			//   - The value was saved into the component's serialized data because it was different from the CDO's default value at the time of taking the snapshot
			//   - We apply the CDO and afterwards we override it with the serialized data. Good.
		const FSubobjectArchetypeFallbackInfo Fallback{ ComponentToDeserializeInto->GetOuter(), ComponentToDeserializeInto->GetFName(), SerializedComponentData.GetObjectFlags() };
		SerializeClassDefaultsIntoSubobject(ComponentToDeserializeInto, WorldData, SerializedComponentData.ClassIndex, Cache, Fallback);
		
		FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(SerializedComponentData, WorldData, Cache, ComponentToDeserializeInto, ProcessObjectDependency, InLocalisationSnapshotPackage);
	}

	static TOptional<TNonNullPtr<AActor>> GetDeserialized(const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* InLocalisationSnapshotPackage, bool bWarnIfNotFound)
	{
		SCOPED_SNAPSHOT_CORE_TRACE(GetDeserialized);
		UE_LOGF(LogLevelSnapshots, Verbose, "========== Get Deserialized %ls ==========", *OriginalActorPath.ToString());
		
		const TOptional<TNonNullPtr<AActor>> Preallocated = GetPreallocated(OriginalActorPath, WorldData, Cache, bWarnIfNotFound);
		if (!Preallocated)
		{
			return {};
		}
		
		FActorSnapshotCache& ActorCache = Cache.ActorCache.FindOrAdd(OriginalActorPath);
		ActorCache.bReceivedSerialisation = true;
		
		FActorSnapshotData& ActorData = WorldData.ActorData[OriginalActorPath];
		const auto ProcessObjectDependency = [&Cache, &WorldData, &OriginalActorPath](int32 OriginalObjectDependency)
		{
			check(WorldData.SerializedObjectReferences.IsValidIndex(OriginalObjectDependency));
			
			// This look-up must be done every time because FSnapshotDataCache::ActorCache may have been reallocated 
			FActorSnapshotCache& ActorCache = Cache.ActorCache.FindOrAdd(OriginalActorPath);
			ActorCache.ObjectDependencies.Add(OriginalObjectDependency);
		};
		
		AActor* PreallocatedActor = Preallocated.GetValue();
		{
			const FRestoreObjectScope FinishRestore = PreActorRestore_SnapshotWorld(PreallocatedActor, ActorData.CustomActorSerializationData, WorldData, Cache, ProcessObjectDependency, InLocalisationSnapshotPackage);
			// CDO must be serialized into the actor 
			SerializeClassDefaultsIntoActor(PreallocatedActor, WorldData, ActorData.ClassIndex, Cache);
			FLoadSnapshotObjectArchive::ApplyToSnapshotWorldObject(ActorData.SerializedActorData, WorldData, Cache, PreallocatedActor, ProcessObjectDependency, InLocalisationSnapshotPackage);
#if WITH_EDITOR
			UE_LOGF(LogLevelSnapshots, Verbose, "ActorLabel is \"%ls\" for \"%ls\" (editor object path \"%ls\")", *PreallocatedActor->GetActorLabel(), *PreallocatedActor->GetPathName(), *OriginalActorPath.ToString());
#endif
		}
		
		DeserializeComponents(ActorData, PreallocatedActor, WorldData,
			[&WorldData, &ProcessObjectDependency, &Cache, InLocalisationSnapshotPackage](
				FSubobjectSnapshotData& SerializedCompData,
				UActorComponent* Comp,
				const FSoftObjectPath& OriginalComponentPath,
				FWorldSnapshotData& SharedData)
			{
				FSubobjectSnapshotCache& SubobjectCache = Cache.SubobjectCache.FindOrAdd(OriginalComponentPath);
				SubobjectCache.SnapshotObject = Comp;
				
				const FRestoreObjectScope FinishRestore = PreSubobjectRestore_SnapshotWorld(Comp, OriginalComponentPath, WorldData, Cache , ProcessObjectDependency, InLocalisationSnapshotPackage);
				DeserializeIntoTransient(SerializedCompData, Comp, SharedData, ProcessObjectDependency, Cache, InLocalisationSnapshotPackage);
			}
		);

		DeserializeSubobjectsForSnapshotActor(ActorData, WorldData, Cache, ProcessObjectDependency, InLocalisationSnapshotPackage);
		PostSerializeSnapshotActor(PreallocatedActor, OriginalActorPath, WorldData, Cache, InLocalisationSnapshotPackage);
		PreallocatedActor->UpdateComponentTransforms();
		{
			// GAllowActorScriptExecutionInEditor must be temporarily true so call to UserConstructionScript isn't skipped
			FEditorScriptExecutionGuard AllowConstructionScript;
			PreallocatedActor->UserConstructionScript();
		}
		
		return Preallocated;
	}
	
	static void ConditionallyRerunConstructionScript(AActor* RequiredActor, const TArray<int32>& OriginalObjectDependencies, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* LocalisationSnapshotPackage)
	{
		if (!RequiredActor)
		{
			return;
		}
		
		bool bHadActorDependencies = false;
		for (const int32 OriginalObjectIndex : OriginalObjectDependencies)
		{
			const FSoftObjectPath& ObjectPath = WorldData.SerializedObjectReferences[OriginalObjectIndex];
			if (WorldData.ActorData.Contains(ObjectPath))
			{
				GetDeserializedActor(ObjectPath, WorldData, Cache, LocalisationSnapshotPackage);
				bHadActorDependencies = true;
			}
		}
		
		if (bHadActorDependencies && ensure(RequiredActor))
		{
			// GAllowActorScriptExecutionInEditor must be temporarily true so call to UserConstructionScript isn't skipped
			FEditorScriptExecutionGuard AllowConstructionScript;
			RequiredActor->UserConstructionScript();
		}
	}

	static ULevel* DetermineMostSuitableLevel(const FSoftObjectPath& ActorPath, const FWorldSnapshotData& WorldData)
	{
		const FName LevelName = *ExtractPathWithoutSubobjects(ActorPath).GetAssetName();
		for (const UWorld* SublevelWorld : WorldData.SnapshotSublevels)
		{
			if (SublevelWorld->GetFName() == LevelName)
			{
				return SublevelWorld->PersistentLevel;
			}
		}
		return WorldData.SnapshotWorld->PersistentLevel;
	}
}

TOptional<TNonNullPtr<AActor>> UE::LevelSnapshots::Private::GetDeserializedActor(const FSoftObjectPath& OriginalObjectPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, UPackage* LocalisationSnapshotPackage, bool bWarnIfNotFound)
{
	{
		// Note: CacheBeforeDeserialize may become invalid after the GetDeserialized call
		const FActorSnapshotCache& CacheBeforeDeserialize = Cache.ActorCache.FindOrAdd(OriginalObjectPath);
		if (CacheBeforeDeserialize.bReceivedSerialisation && CacheBeforeDeserialize.CachedSnapshotActor.IsValid())
		{
			return { CacheBeforeDeserialize.CachedSnapshotActor.Get() };
		}
	}
	
	const TOptional<TNonNullPtr<AActor>> Result = Internal::GetDeserialized(OriginalObjectPath, WorldData, Cache, LocalisationSnapshotPackage, bWarnIfNotFound);
	// Look up cache again because GetDeserialized might have reallocated Cache.ActorCache hence CacheBeforeDeserialize is no longer valid
	const FActorSnapshotCache& CacheAfterDeserialize = Cache.ActorCache.FindOrAdd(OriginalObjectPath);
	Internal::ConditionallyRerunConstructionScript(GetRawPointerOrNull(Result), CacheAfterDeserialize.ObjectDependencies, WorldData, Cache, LocalisationSnapshotPackage);
	return Result;
}

TOptional<TNonNullPtr<UClass>> UE::LevelSnapshots::Private::GetActorClass(const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, bool bWarnIfNotFound)
{
	const FActorSnapshotData* ActorData = WorldData.ActorData.Find(OriginalActorPath);

	if (!ActorData)
	{
		UE_CLOGF(bWarnIfNotFound, LogLevelSnapshots, Warning, "Failed to get the UClass for actor %ls due to missing data", *OriginalActorPath.ToString());
		return {};
	}

	const FSoftClassPath ClassPath = GetClass(*ActorData, WorldData);
	UClass* ActorClass = ClassPath.TryLoadClass<AActor>();

	if (!ActorClass)
	{
		UE_LOGF(LogLevelSnapshots, Warning, "Failed to load the UClass for actor %ls.", *OriginalActorPath.ToString());
		return {};
	}

	return TOptional<TNonNullPtr<UClass>>(ActorClass);
}

TOptional<TNonNullPtr<AActor>> UE::LevelSnapshots::Private::GetPreallocatedIfCached(const FSoftObjectPath& OriginalActorPath, const FSnapshotDataCache& Cache)
{
	if (const FActorSnapshotCache* ActorCache = Cache.ActorCache.Find(OriginalActorPath))
	{
		return ActorCache->CachedSnapshotActor.IsValid() ? TOptional<TNonNullPtr<AActor>>(ActorCache->CachedSnapshotActor.Get()) : TOptional<TNonNullPtr<AActor>>();
	}
	return {};
}

TOptional<TNonNullPtr<AActor>> UE::LevelSnapshots::Private::GetPreallocated(const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, bool bWarnIfNotFound)
{
	SCOPED_SNAPSHOT_CORE_TRACE(GetPreallocated);
	FActorSnapshotCache& ActorCache = Cache.ActorCache.FindOrAdd(OriginalActorPath);
	if (!ActorCache.CachedSnapshotActor.IsValid())
	{
		UWorld* SnapshotWorld = WorldData.SnapshotWorld->GetWorld();
		const FActorSnapshotData* ActorData = WorldData.ActorData.Find(OriginalActorPath);
		if (!ActorData)
		{
			UE_CLOGF(bWarnIfNotFound, LogLevelSnapshots, Warning, "Failed to allocate %ls due to missing data", *OriginalActorPath.ToString());
			return {};
		}
		
		const FSoftClassPath ClassPath = GetClass(*ActorData, WorldData);
		UClass* ActorClass = ClassPath.TryLoadClass<AActor>();
		if (!ActorClass)
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Failed to allocate %ls due to missing class.", *OriginalActorPath.ToString());
			return {};
		}

		FActorSpawnParameters SpawnParams;
		SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		SpawnParams.OverrideLevel = Internal::DetermineMostSuitableLevel(OriginalActorPath, WorldData);
		const FName ActorName = *ExtractLastSubobjectName(OriginalActorPath);
		// Must still generate a unique name because the asset registry might have messed with our data during a rename and added a duplicate actor name...
		SpawnParams.Name = FindObjectFast<UObject>(SpawnParams.OverrideLevel, ActorName)
			? MakeUniqueObjectName(SpawnParams.OverrideLevel, ActorClass, ActorName)
			: ActorName;
		SpawnParams.bNoFail = true;
		ActorCache.CachedSnapshotActor = SnapshotWorld->SpawnActor<AActor>(ActorClass, SpawnParams);

		const bool bSpawnWasSuccess = ActorCache.CachedSnapshotActor.IsValid();
		UE_CLOGF(!bSpawnWasSuccess, LogLevelSnapshots, Error, "Failed to allocate %ls due to missing class.", *OriginalActorPath.ToString());
		if (ensure(bSpawnWasSuccess))
		{
#if WITH_EDITOR
			// Hide this actor so external systems can see that this components should not render, i.e. make USceneComponent::ShouldRender return false
			ActorCache.CachedSnapshotActor->SetIsTemporarilyHiddenInEditor(true);
#endif
			AllocateMissingComponentsForSnapshotActor(ActorCache.CachedSnapshotActor.Get(), OriginalActorPath, WorldData, Cache);
		}
	}

	return ActorCache.CachedSnapshotActor.IsValid() ? TOptional<TNonNullPtr<AActor>>(ActorCache.CachedSnapshotActor.Get()) : TOptional<TNonNullPtr<AActor>>();
}