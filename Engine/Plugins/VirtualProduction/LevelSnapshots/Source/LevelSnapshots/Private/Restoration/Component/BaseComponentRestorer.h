// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/ActorSnapshotData.h"
#include "Data/Util/ClassDataUtil.h"
#include "Data/WorldSnapshotData.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "Util/SoftObjectPathUtil.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "ComponentInstanceDataCache.h"

class AActor;
class UActorComponent;

struct FActorSnapshotData;
struct FComponentSnapshotData;
struct FSoftObjectPath;
struct FSubobjectSnapshotData;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/**
	 * Helper for restoring components into an actor.
	 */
	template<typename TDerived>
	class TBaseComponentRestorer
	{
		const TDerived* This() const { return static_cast<const TDerived*>(this); }
	public:

		TBaseComponentRestorer(AActor* ActorToRestore, const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData)
			: ActorToRestore(ActorToRestore)
			, OriginalActorPath(OriginalActorPath)
			, SnapshotData(WorldData.ActorData[OriginalActorPath])
			, WorldData(WorldData)
		{}

		/** Loops all saved component data and makes sure they are allocated. */
		void RecreateSavedComponents() const
		{
			for (auto CompIt = SnapshotData.ComponentData.CreateConstIterator(); CompIt; ++CompIt)
			{
				const int32 ReferenceIndex = CompIt->Key;
				FSubobjectSnapshotData* SubobjectData = WorldData.Subobjects.Find(ReferenceIndex); 
				if (ensure(SubobjectData))
				{
					const FSoftObjectPath& ComponentPath = WorldData.SerializedObjectReferences[ReferenceIndex];
					FindOrAllocateComponentInternal(ComponentPath, *SubobjectData, CompIt->Value);
				}
			}
		}

	protected:

		AActor* GetActorToRestore() const { return ActorToRestore; }
		const FSoftObjectPath& GetOriginalActorPath() const { return OriginalActorPath; }
		const FActorSnapshotData& GetSnapshotData() const { return SnapshotData; }
		FWorldSnapshotData& GetWorldData() const { return WorldData; }
		
		/**
		 * @param ComponentPath The original path of the component
		 * @param SubobjectData The subobject data associated with this component
		 * @param ComponentData The component data associated with this component
		 */
		UActorComponent* FindOrAllocateComponentInternal(const FSoftObjectPath& ComponentPath, FSubobjectSnapshotData& SubobjectData, const FComponentSnapshotData& ComponentData) const
		{
			UActorComponent* Component = UE::LevelSnapshots::Private::FindMatchingComponent(ActorToRestore, ComponentPath);
			if (!Component)
			{
				bool bIsOwnedByComponent;
				const FSoftObjectPath& ComponentOuterPath = WorldData.SerializedObjectReferences[SubobjectData.OuterIndex];
				const TOptional<TNonNullPtr<FActorSnapshotData>> ComponentOuterData = UE::LevelSnapshots::Private::FindSavedActorDataUsingObjectPath(WorldData.ActorData, ComponentOuterPath, bIsOwnedByComponent);
				const bool bIsOwnedByOtherActor = GetRawPointerOrNull(ComponentOuterData) != &SnapshotData;
				if (!ensureAlwaysMsgf(ComponentOuterData, TEXT("Failed to recreate component %s because its outer %s did not have any associated actor data. Investigate."), *ComponentPath.ToString(), *ComponentOuterPath.ToString())
					|| !ensureMsgf(!bIsOwnedByOtherActor, TEXT("Failed to recreate component %s because saved data indicates it is owned by another actor %s. Components normally are owned by the actor they're attached to. Investigate."), *ComponentPath.ToString(), *ComponentOuterPath.ToString()))
				{
					return nullptr;
				}
			
				UObject* ComponentOuter = bIsOwnedByComponent
					? FindOrAllocateComponentForComponentOuter(ComponentOuterPath) : ActorToRestore;
				if (ComponentOuter)
				{
					Component = AllocateComponent(ComponentPath, SubobjectData, ComponentData, ComponentOuter);
					if (Component)
					{
						// UActorComponent::PostInitProperties implicitly calls AddOwnedComponent but we have to manually add it to the other arrays
						AddToCorrectComponentArray(ComponentData, Component);
					}
				}
			}
		
			return Component;
		}

	private:

		UActorComponent* AllocateComponent(const FSoftObjectPath& ComponentPath, FSubobjectSnapshotData& SubobjectData, const FComponentSnapshotData& ComponentData, UObject* ComponentOuter) const
		{
			const FSoftClassPath ClassPath = GetClass(SubobjectData, WorldData);
			UClass* ComponentClass = ClassPath.TryLoadClass<UActorComponent>();
			if (!ComponentClass)
			{
				UE_LOGF(LogLevelSnapshots, Error, "Failed to load class %ls", *ClassPath.ToString())
				return nullptr;
			}
			
			const FName ComponentName = *UE::LevelSnapshots::Private::ExtractLastSubobjectName(ComponentPath);
			const EObjectFlags ObjectFlags = SubobjectData.GetObjectFlags();
			if constexpr (TDerived::IsRestoringIntoSnapshotWorld())
			{
				This()->PreCreateComponent(ComponentName, ComponentClass, ComponentData.CreationMethod);
				// Objects in transient snapshot world should not transact
				UActorComponent* Component = NewObject<UActorComponent>(ComponentOuter, ComponentClass, ComponentName, ObjectFlags & ~RF_Transactional);
				This()->PostCreateComponent(SubobjectData, Component);
				return Component;
			}
			else
			{
				This()->PreCreateComponent(ComponentName, ComponentClass, ComponentData.CreationMethod);
				// RF_Transactional must be set when the object is created so the creation transacted correctly
				UActorComponent* Component = NewObject<UActorComponent>(ComponentOuter, ComponentClass, ComponentName, ObjectFlags | RF_Transactional);
				Component->SetFlags(ObjectFlags);
				This()->PostCreateComponent(SubobjectData, Component);

				return Component;
			}
		}
		
		void AddToCorrectComponentArray(const FComponentSnapshotData& ComponentData, UActorComponent* Component) const
		{
			if (Component->CreationMethod != ComponentData.CreationMethod)
			{
				Component->CreationMethod = ComponentData.CreationMethod;
				switch(Component->CreationMethod)
				{
				case EComponentCreationMethod::Instance:
					ActorToRestore->AddInstanceComponent(Component);
					break;
				case EComponentCreationMethod::SimpleConstructionScript:
					ActorToRestore->BlueprintCreatedComponents.AddUnique(Component);
					break;
				case EComponentCreationMethod::UserConstructionScript:
					checkf(false, TEXT("Component created in construction script currently unsupported"));
					break;
						
				case EComponentCreationMethod::Native:
				default:
					break;
				}
			}
		}
		
		UObject* FindOrAllocateComponentForComponentOuter(const FSoftObjectPath& ComponentOuterPath) const
		{
			for (auto CompIt = SnapshotData.ComponentData.CreateConstIterator(); CompIt; ++CompIt)
			{
				const FSoftObjectPath& SavedComponentPath = WorldData.SerializedObjectReferences[CompIt->Key];
				if (SavedComponentPath == ComponentOuterPath)
				{
					const int32 ReferenceIndex = CompIt->Key;
					FSubobjectSnapshotData* SubobjectData = WorldData.Subobjects.Find(ReferenceIndex);
				
					return ensure(SubobjectData)
						? FindOrAllocateComponentInternal(ComponentOuterPath, *SubobjectData, CompIt->Value) : nullptr;
				}
			}

			UE_LOGF(LogLevelSnapshots, Error, "Failed to find outer for component %ls", *ComponentOuterPath.ToString());
			return nullptr;
		}
		
		AActor* ActorToRestore;
		const FSoftObjectPath OriginalActorPath;
		const FActorSnapshotData& SnapshotData;
		FWorldSnapshotData& WorldData;
	};
}