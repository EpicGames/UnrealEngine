// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotComponentUtil.h"
#include "BaseComponentRestorer.h"

#include "Archive/ApplySnapshotToEditorArchive.h"
#include "Data/ActorSnapshotData.h"
#include "Data/WorldSnapshotData.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Filtering/PropertySelectionMap.h"
#include "CustomSerialization/CustomObjectSerializationWrapper.h"
#include "Util/SoftObjectPathUtil.h"
#include "LevelSnapshotsModule.h"
#include "Restoration/Actor/ActorUtil.h"
#include "Restoration/Scope/ApplySnapshotToActorScope.h"
#include "Restoration/Scope/RemoveComponentScope.h"
#include "SnapshotDataCache.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "Templates/NonNullPointer.h"
#include "UObject/Package.h"

#if USE_STABLE_LOCALIZATION_KEYS
#include "Internationalization/TextPackageNamespaceUtil.h"
#endif

#if WITH_EDITOR
#include "LevelEditor.h"
#endif

namespace UE::LevelSnapshots::Private::Internal
{
	/** Recreates and removes components on actors that were matched in the editor world. */
	class FExistingActorComponentRestorer final : public TBaseComponentRestorer<FExistingActorComponentRestorer>
	{
	public:
		
		FExistingActorComponentRestorer(AActor* MatchedEditorActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache);

		//~ Begin TBaseComponentRestorer Interface
		void PreCreateComponent(FName ComponentName, UClass* ComponentClass, EComponentCreationMethod CreationMethod) const;
		void PostCreateComponent(FSubobjectSnapshotData& SubobjectData, UActorComponent* RecreatedComponent) const;
		static constexpr bool IsRestoringIntoSnapshotWorld() { return false; }
		//~ Begin TBaseComponentRestorer Interface
	
		void RemovedNewComponents(const FAddedAndRemovedComponentInfo& ComponentSelection) const;
		void RecreateMissingComponents(const FAddedAndRemovedComponentInfo& ComponentSelection, const FPropertySelectionMap& SelectionMap, UPackage* LocalisationSnapshotPackage) const;
		void UpdateDetailsViewAfterUpdatingComponents() const;
	
	private:
		
		TPair<int32, const FComponentSnapshotData*> FindMatchingSubobjectIndex(const FString& ComponentName) const;
	
		struct FDeferredComponentSerializationTask
		{
			FSubobjectSnapshotData* SubobjectData;
			UActorComponent* Snapshot;
			UActorComponent* Original;

			FDeferredComponentSerializationTask(FSubobjectSnapshotData* SubobjectData, UActorComponent* Snapshot, UActorComponent* Original)
				:
				SubobjectData(SubobjectData),
				Snapshot(Snapshot),
				Original(Original)
			{}
		};

		mutable TArray<FDeferredComponentSerializationTask> DeferredSerializationsTasks;
		mutable TArray<UActorComponent*> CompsWithMissingSnapshotCounterpart;
		FSnapshotDataCache& Cache;
	};

	FExistingActorComponentRestorer::FExistingActorComponentRestorer(AActor* MatchedEditorActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache)
		: TBaseComponentRestorer<FExistingActorComponentRestorer>(MatchedEditorActor, MatchedEditorActor, WorldData)
		, Cache(Cache)
	{}

	void FExistingActorComponentRestorer::PreCreateComponent(FName ComponentName, UClass* ComponentClass, EComponentCreationMethod CreationMethod) const 
	{
		FLevelSnapshotsModule::GetInternalModuleInstance().OnPreRecreateComponent({ GetActorToRestore(), ComponentName, ComponentClass, CreationMethod });
	}

	void FExistingActorComponentRestorer::PostCreateComponent(FSubobjectSnapshotData& SubobjectData, UActorComponent* RecreatedComponent) const 
	{
		const TOptional<TNonNullPtr<AActor>> SnapshotActor = GetPreallocatedIfCached(GetOriginalActorPath(), Cache);
		checkf(SnapshotActor, TEXT("Snapshot actor was expected to be allocated at this point"));
		
		const FSoftObjectPath RecreatedComponentPath(RecreatedComponent);
		UActorComponent* SnapshotComponent = UE::LevelSnapshots::Private::FindMatchingComponent(SnapshotActor.GetValue(), RecreatedComponentPath);
		if (ensure(SnapshotComponent))
		{
			FSubobjectSnapshotCache& SubobjectCache = Cache.SubobjectCache.FindOrAdd(GetOriginalActorPath());
			SubobjectCache.EditorObject = RecreatedComponent;
			
			FLevelSnapshotsModule::GetInternalModuleInstance().OnPostRecreateComponent(RecreatedComponent);
			DeferredSerializationsTasks.Add({ &SubobjectData, SnapshotComponent, RecreatedComponent });
			RecreatedComponent->RegisterComponent();
		}
		else
		{
			UE_LOGF(LogLevelSnapshots, Warning, "Failed to find snapshot counterpart for %ls", *RecreatedComponentPath.ToString());
			CompsWithMissingSnapshotCounterpart.Add(RecreatedComponent);
		}
	}

	void FExistingActorComponentRestorer::RemovedNewComponents(const FAddedAndRemovedComponentInfo& ComponentSelection) const
	{
		for (const TWeakObjectPtr<UActorComponent>& ComponentToRemove : ComponentSelection.EditorWorldComponentsToRemove)
		{
			// Evil user might have manually deleted the component between filtering and applying the snapshot
			if (ComponentToRemove.IsValid())
			{
				ComponentToRemove->Modify();
				const FRemoveComponentScope NotifySnapshotListeners(ComponentToRemove.Get());
				// Subscribers to ISnapshotListener::PreRemoveComponent are allowed to manually remove the component
				if (ComponentToRemove.IsValid())
				{
					ComponentToRemove->DestroyComponent();
				}
			}
		}
	}

	void FExistingActorComponentRestorer::RecreateMissingComponents(const FAddedAndRemovedComponentInfo& ComponentSelection, const FPropertySelectionMap& SelectionMap, UPackage* LocalisationSnapshotPackage) const
	{
		// 1. Recreate missing components
		ensureMsgf(DeferredSerializationsTasks.Num() == 0, TEXT("RecreateMissingComponents is expected to only be once per instance"));
		DeferredSerializationsTasks.Empty();
		
		// Edge case: a component in SnapshotComponentsToAdd has an outer which is not in SnapshotComponentsToAdd.
		// In this case, the outer components will be recreated even if they are not in SnapshotComponentsToAdd.  
		for (const TWeakObjectPtr<UActorComponent>& CurrentComponentToRestore : ComponentSelection.SnapshotComponentsToAdd)
		{
			const FName NameOfComponentToRecreate = CurrentComponentToRestore->GetFName();
			// TODO: Look at outer chain like SnapshotUtil::FindMatchingComponent instead of just by component name in FindMatchingSubobjectIndex
			const TPair<int32, const FComponentSnapshotData*> ComponentInfo = FindMatchingSubobjectIndex(NameOfComponentToRecreate.ToString());
			const int32 ReferenceIndex = ComponentInfo.Key;
			FSubobjectSnapshotData* SubobjectData = GetWorldData().Subobjects.Find(ReferenceIndex); 
			if (ensure(SubobjectData))
			{
				const FSoftObjectPath& ComponentPath = GetWorldData().SerializedObjectReferences[ReferenceIndex];
				FindOrAllocateComponentInternal(ComponentPath, *SubobjectData, *ComponentInfo.Value);
			}
		}

		// 2. Serialize saved data into components
		for (const FDeferredComponentSerializationTask& Task : DeferredSerializationsTasks)
		{
			const FRestoreObjectScope FinishRestore = PreSubobjectRestore_EditorWorld(Task.Snapshot, Task.Original, GetWorldData(), Cache, SelectionMap, LocalisationSnapshotPackage);
			FApplySnapshotToEditorArchive::ApplyToEditorWorldObjectRecreatedWithArchetype(*Task.SubobjectData, GetWorldData(), Cache, Task.Original, SelectionMap); 
		}
		for (UActorComponent* Fail : CompsWithMissingSnapshotCounterpart)
		{
			Fail->DestroyComponent();
		}
	}

	void FExistingActorComponentRestorer::UpdateDetailsViewAfterUpdatingComponents() const
	{
#if WITH_EDITOR
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.BroadcastComponentsEdited();
#endif
	}

	FString ComponentListToString(const FActorSnapshotData& SnapshotData, const FWorldSnapshotData& WorldData)
	{
		FString Result;
		for (auto CompIt = SnapshotData.ComponentData.CreateConstIterator(); CompIt; ++CompIt)
		{
			const int32 ReferenceIndex = CompIt->Key;
			const FSoftObjectPath& SavedComponentPath = WorldData.SerializedObjectReferences[ReferenceIndex];
			Result += FString::Printf(TEXT("%s "), *SavedComponentPath.ToString());
		}
		return Result;
	}

	TPair<int32, const FComponentSnapshotData*> FExistingActorComponentRestorer::FindMatchingSubobjectIndex(const FString& ComponentName) const
	{
		for (auto CompIt = GetSnapshotData().ComponentData.CreateConstIterator(); CompIt; ++CompIt)
		{
			const int32 ReferenceIndex = CompIt->Key;
			const FSoftObjectPath& SavedComponentPath = GetWorldData().SerializedObjectReferences[ReferenceIndex];
			const FString SavedComponentName = UE::LevelSnapshots::Private::ExtractLastSubobjectName(SavedComponentPath);
			if (SavedComponentName.Equals(ComponentName))
			{
				return TPair<int32, const FComponentSnapshotData*>(ReferenceIndex, &CompIt->Value);
			}
		}

		UE_LOGF(LogLevelSnapshots, Warning, "Failed to component data for %ls (Components: [%ls])", *ComponentName, *ComponentListToString(GetSnapshotData(), GetWorldData()));
		return TPair<int32, const FComponentSnapshotData*>(INDEX_NONE, nullptr);
	}
}

void UE::LevelSnapshots::Private::AddAndRemoveComponentsSelectedForRestore(AActor* MatchedEditorActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, const FPropertySelectionMap& SelectionMap, UPackage* LocalisationSnapshotPackage)
{
	if (const FAddedAndRemovedComponentInfo* ComponentSelection = SelectionMap.GetObjectSelection(MatchedEditorActor).GetComponentSelection())
	{
		Internal::FExistingActorComponentRestorer Restorer(MatchedEditorActor, WorldData, Cache);
		Restorer.RemovedNewComponents(*ComponentSelection);
		Restorer.RecreateMissingComponents(*ComponentSelection, SelectionMap, LocalisationSnapshotPackage);

		const bool bChangedHierarchy = ComponentSelection->SnapshotComponentsToAdd.Num() > 0 || ComponentSelection->EditorWorldComponentsToRemove.Num() > 0;
		if (bChangedHierarchy)
		{
			Restorer.UpdateDetailsViewAfterUpdatingComponents();
		}
	}
}


