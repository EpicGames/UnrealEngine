// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseComponentRestorer.h"

#include "Data/WorldSnapshotData.h"
#include "SnapshotComponentUtil.h"
#include "SnapshotDataCache.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

namespace UE::LevelSnapshots::Private::Internal
{
	/** Recreates components on actors that were recreated in the editor world. */
	class FRecreatedActorComponentRestorer final : public TBaseComponentRestorer<FRecreatedActorComponentRestorer>
	{
		FSnapshotDataCache& Cache;
	public:

		FRecreatedActorComponentRestorer(AActor* RecreatedEditorActor, const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache)
			: TBaseComponentRestorer<FRecreatedActorComponentRestorer>(RecreatedEditorActor, OriginalActorPath, WorldData)
			, Cache(Cache)
		{}
		
		//~ Begin TBaseComponentRestorer Interface
		void PreCreateComponent(FName, UClass*, EComponentCreationMethod) const
		{}
		
		void PostCreateComponent(FSubobjectSnapshotData& SubobjectData, UActorComponent* RecreatedComponent) const
		{
			FSubobjectSnapshotCache& SubobjectCache = Cache.SubobjectCache.FindOrAdd(GetOriginalActorPath());
			SubobjectCache.EditorObject = RecreatedComponent;
			RecreatedComponent->RegisterComponent();
		}
		
		static constexpr bool IsRestoringIntoSnapshotWorld()
		{
			return false;
		}
		//~ Begin TBaseComponentRestorer Interface
	};
	
}

void UE::LevelSnapshots::Private::AllocateMissingComponentsForRecreatedActor(AActor* RecreatedEditorActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache)
{
	Internal::FRecreatedActorComponentRestorer Restorer(RecreatedEditorActor, RecreatedEditorActor, WorldData, Cache);
	Restorer.RecreateSavedComponents();
}

