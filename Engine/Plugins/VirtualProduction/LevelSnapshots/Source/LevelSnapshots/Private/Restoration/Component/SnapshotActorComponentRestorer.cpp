// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseComponentRestorer.h"

#include "Data/WorldSnapshotData.h"
#include "SnapshotComponentUtil.h"
#include "SnapshotDataCache.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

namespace UE::LevelSnapshots::Private::Internal
{
	/** Recreates component on actors in the snapshot world. */
	class FSnapshotActorComponentRestorer final : public TBaseComponentRestorer<FSnapshotActorComponentRestorer>
	{
		FSnapshotDataCache& Cache;
	public:
		
		FSnapshotActorComponentRestorer(AActor* SnapshotActor, const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache)
			: TBaseComponentRestorer<FSnapshotActorComponentRestorer>(SnapshotActor, OriginalActorPath, WorldData)
			, Cache(Cache)
		{}

		//~ Begin TBaseComponentRestorer Interface
		void PreCreateComponent(FName, UClass*, EComponentCreationMethod) const
		{}
		
		void PostCreateComponent(FSubobjectSnapshotData& SubobjectData, UActorComponent* RecreatedComponent) const
		{
			FSubobjectSnapshotCache& SubobjectCache = Cache.SubobjectCache.FindOrAdd(GetOriginalActorPath());
			SubobjectCache.SnapshotObject = RecreatedComponent;
		}
		
		static constexpr bool IsRestoringIntoSnapshotWorld()
		{
			return true;
		}
		//~ Begin TBaseComponentRestorer Interface
	};
	
}

void UE::LevelSnapshots::Private::AllocateMissingComponentsForSnapshotActor(AActor* SnapshotActor, const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache)
{
	Internal::FSnapshotActorComponentRestorer Restorer(SnapshotActor, OriginalActorPath, WorldData, Cache);
	Restorer.RecreateSavedComponents();
}