// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class AActor;
class UActorComponent;
class UPackage;

struct FActorSnapshotData;
struct FPropertySelectionMap;
struct FSnapshotDataCache;
struct FSoftObjectPath;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/** Adds and removes components on an actor that exists in the snapshot world (and existed the snapshot were applied). */
	void AddAndRemoveComponentsSelectedForRestore(AActor* MatchedEditorActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache, const FPropertySelectionMap& SelectionMap, UPackage* LocalisationSnapshotPackage);
	
	/** Recreates all components an actor recreated in the editor world. */
	void AllocateMissingComponentsForRecreatedActor(AActor* RecreatedEditorActor, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache);

	/** Recreates all components on the actor in the snapshot world. */
	void AllocateMissingComponentsForSnapshotActor(AActor* SnapshotActor, const FSoftObjectPath& OriginalActorPath, FWorldSnapshotData& WorldData, FSnapshotDataCache& Cache);
}



