// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplySnapshotPropertiesScope.h"

#include "LevelSnapshotsModule.h"

UE::LevelSnapshots::Private::FApplySnapshotPropertiesScope::FApplySnapshotPropertiesScope(const FApplySnapshotPropertiesParams& InParams)
	:
	Params(InParams)
{
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPreApplySnapshotProperties(InParams);
}

UE::LevelSnapshots::Private::FApplySnapshotPropertiesScope::~FApplySnapshotPropertiesScope()
{
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPostApplySnapshotProperties(Params);
}
