// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplySnapshotToActorScope.h"

#include "LevelSnapshotsModule.h"

UE::LevelSnapshots::Private::FApplySnapshotToActorScope::FApplySnapshotToActorScope(const FApplySnapshotToActorParams& Params)
	:
	Params(Params)
{
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPreApplySnapshotToActor(Params);
}

UE::LevelSnapshots::Private::FApplySnapshotToActorScope::~FApplySnapshotToActorScope()
{
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPostApplySnapshotToActor(Params);
}
