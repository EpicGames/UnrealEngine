// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IRestorationListener.h"
#include "Templates/UnrealTemplate.h"

struct FPropertySelectionMap;

namespace UE::LevelSnapshots::Private
{
	/**
	 * Convenience type that calls FLevelSnaphshotsModule::OnPreApplySnapshot and FLevelSnaphshotsModule::OnPostApplySnapshot.
	 */
	class FApplySnapshotToActorScope : public FNoncopyable
	{
		const FApplySnapshotToActorParams Params;
	public:

		FApplySnapshotToActorScope(const FApplySnapshotToActorParams& Params);
		~FApplySnapshotToActorScope();
	};
}

