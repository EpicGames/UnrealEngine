// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoveComponentScope.h"

#include "LevelSnapshotsModule.h"

#include "Components/ActorComponent.h"

UE::LevelSnapshots::Private::FRemoveComponentScope::FRemoveComponentScope(UActorComponent* RemovedComponent)
	:
	Params({ RemovedComponent->GetOwner(), RemovedComponent->GetFName(), RemovedComponent })
{
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPreRemoveComponent(RemovedComponent);
}

UE::LevelSnapshots::Private::FRemoveComponentScope::~FRemoveComponentScope()
{
	FLevelSnapshotsModule::GetInternalModuleInstance().OnPostRemoveComponent(Params);
}
