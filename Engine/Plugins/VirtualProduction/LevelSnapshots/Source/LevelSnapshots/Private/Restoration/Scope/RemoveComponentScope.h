// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IRestorationListener.h"
#include "Templates/UnrealTemplate.h"

class UActorComponent;

namespace UE::LevelSnapshots::Private
{
	/**
	 * Convenience type that calls FLevelSnaphshotsModule::OnPreRemoveComponent and FLevelSnaphshotsModule::OnPostRemoveComponent.
	 */
	class FRemoveComponentScope : public FNoncopyable
	{
		FPostRemoveComponentParams Params;
	public:

		FRemoveComponentScope(UActorComponent* RemovedComponent);
		~FRemoveComponentScope();
	};
}


