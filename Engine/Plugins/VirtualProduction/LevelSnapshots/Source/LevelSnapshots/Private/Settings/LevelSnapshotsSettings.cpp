// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/LevelSnapshotsSettings.h"

#include "Data/Hashing/ActorHashUtil.h"
#include "PropertyInfoHelpers.h"

ULevelSnapshotsSettings* ULevelSnapshotsSettings::Get()
{
	return GetMutableDefault<ULevelSnapshotsSettings>();
}

void ULevelSnapshotsSettings::PostInitProperties()
{
	using namespace UE::LevelSnapshots;
	UObject::PostInitProperties();
	
	UpdateDecimalComparisionPrecision(FloatComparisonPrecision, DoubleComparisonPrecision);
	Private::GHashSettings = HashSettings;
}

#if WITH_EDITOR
void ULevelSnapshotsSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::LevelSnapshots;
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULevelSnapshotsSettings, FloatComparisonPrecision)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULevelSnapshotsSettings, DoubleComparisonPrecision))
	{
		UpdateDecimalComparisionPrecision(FloatComparisonPrecision, DoubleComparisonPrecision);
	}

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULevelSnapshotsSettings, HashSettings))
	{
		Private::GHashSettings = HashSettings;
	}
	
	UObject::PostEditChangeProperty(PropertyChangedEvent);
}
#endif
