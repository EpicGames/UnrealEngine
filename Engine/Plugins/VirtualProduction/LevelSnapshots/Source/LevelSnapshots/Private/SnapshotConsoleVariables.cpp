// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotConsoleVariables.h"

#include "Containers/UnrealString.h"

TAutoConsoleVariable<FString> UE::LevelSnapshots::ConsoleVariables::CVarBreakOnSnapshotActor(TEXT("LevelSnapshots.BreakOnSnapshotActor"), TEXT(""), TEXT("Hit a debug break point when an actor with the given name is snapshot"));
TAutoConsoleVariable<FString> UE::LevelSnapshots::ConsoleVariables::CVarBreakOnDiffMatchedActor(TEXT("LevelSnapshots.BreakOnDiffMatchedActor"), TEXT(""), TEXT("Hit a debug break point when a matched actor with the given name is discovered while diffing world"));

#if UE_BUILD_DEBUG
TAutoConsoleVariable<FString> UE::LevelSnapshots::ConsoleVariables::CVarBreakOnSerializedPropertyName(TEXT("LevelSnapshots.BreakOnSerializedPropertyName"), TEXT(""), TEXT("Halt execution when encountering this property name during serialisation"));
#endif

TAutoConsoleVariable<bool> UE::LevelSnapshots::ConsoleVariables::CVarLogTimeDiffingMatchedActors(TEXT("LevelSnapshots.LogTimeDiffingMatchedActors"), false, TEXT("Enable logging of how long matched actors take to diff"));
TAutoConsoleVariable<bool> UE::LevelSnapshots::ConsoleVariables::CVarLogTimeTakingSnapshots(TEXT("LevelSnapshots.LogTimeTakingSnapshots"), false, TEXT("Enable logging of how long actors take to get snapshots taken off."));
TAutoConsoleVariable<bool> UE::LevelSnapshots::ConsoleVariables::CVarLogTimeRecreatingActors(TEXT("LevelSnapshots.LogTimeRecreatingActors"), false, TEXT("Enable logging of how it takes to recreate each actor (\"Actors to Add\")"));
TAutoConsoleVariable<bool> UE::LevelSnapshots::ConsoleVariables::CVarLogTimeRestoringMatchedActors(TEXT("LevelSnapshots.LogTimeRestoringMatchedActors"), false, TEXT("Enable logging of how it takes to serialize into matched actors (\"Modified\")."));
TAutoConsoleVariable<bool> UE::LevelSnapshots::ConsoleVariables::CVarLogSelectionMap(TEXT("LevelSnapshots.LogSelectionMap"), false, TEXT("Logs all filtered properties a filter is applied."));
