// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"

namespace UE::LevelSnapshots::ConsoleVariables
{
	/** Hit a debug break point when an actor with the given name is snapshot */
	extern TAutoConsoleVariable<FString> CVarBreakOnSnapshotActor;

	/** Hit a debug break point when a matched actor with the given name is discovered while diffing world */
	extern TAutoConsoleVariable<FString> CVarBreakOnDiffMatchedActor;
	
#if UE_BUILD_DEBUG
	/** Halt execution when encountering this property name during serialisation */
	extern TAutoConsoleVariable<FString> CVarBreakOnSerializedPropertyName;
#endif
	
	/** Enable logging of how long matched actors take to diff*/
	extern TAutoConsoleVariable<bool> CVarLogTimeDiffingMatchedActors;

	/** Enable logging of how long actors take to get snapshots taken of.*/
	extern TAutoConsoleVariable<bool> CVarLogTimeTakingSnapshots;
	
	/** Enable logging of how it takes to recreate each actor ("Actors to Add").*/
	extern TAutoConsoleVariable<bool> CVarLogTimeRecreatingActors;
	
	/** Enable logging of how it takes to serialize into matched actors ("Modified").*/
	extern TAutoConsoleVariable<bool> CVarLogTimeRestoringMatchedActors;

	/** Logs all filtered properties whenever a filter is applied */
	extern TAutoConsoleVariable<bool> CVarLogSelectionMap;
}
