// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorGroupRestoration.h"

#include "LevelSnapshot.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Filtering/PropertySelectionMap.h"
#include "GameFramework/WorldSettings.h"
#include "Params/PropertyComparisonParams.h"

#include "Algo/AllOf.h"
#include "Editor/GroupActor.h"

namespace UE::LevelSnapshots::Private::ActorGroupRestoration
{
#if WITH_EDITORONLY_DATA
	static void AddActorGroupSupport(UE::LevelSnapshots::Private::FLevelSnapshotsModule& Module)
	{
		const FProperty* bLocked = AGroupActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AGroupActor, bLocked)); 
		const FProperty* GroupActors = AGroupActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AGroupActor, GroupActors));
		const FProperty* SubGroups = AGroupActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AGroupActor, SubGroups));
		if (ensure(bLocked && GroupActors && SubGroups))
		{
			Module.AddExplicitilySupportedProperties({ bLocked, GroupActors, SubGroups });
		}
	}
	
	class FActorGroupRestoration : public IRestorationListener, public IPropertyComparer, public IActorSnapshotFilter
	{
		const FProperty* GroupActorsProperty = AGroupActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AGroupActor, GroupActors));
		const FProperty* SubGroupsProperty = AGroupActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AGroupActor, SubGroups));

		TSet<TWeakObjectPtr<AGroupActor>> ActorsWithDummyChild;
		
		template<typename TActor>
		void RegisterActorsWithGroup(AGroupActor& GroupActor, TArray<TObjectPtr<TActor>>& Actors)
		{
			const ULevel* GroupLevel = GroupActor.GetLevel(); 
			for (auto ActorIt = Actors.CreateIterator(); ActorIt; ++ActorIt)
			{
				AActor* Actor = *ActorIt;
				if (Actor == nullptr)
				{
					// Should happen when actor is not in the same level as the group (because user moved actor to another level or so)
					ActorIt.RemoveCurrent();
					continue;
				}

				const ULevel* ActorLevel = Actor->GetLevel();
				if (ensure(ActorLevel == GroupLevel))
				{
					// Note that the iterator will iterate this as well - it does not matter because there is an AddUnique in AGroupActor::Add.
					GroupActor.Add(*Actor);
				}
			}
		}

		template<typename TActor>
		void UnregisterActorsFromGroup(AGroupActor& GroupActor, TArray<TObjectPtr<TActor>>& Actors)
		{
			for (AActor* Actor : Actors)
			{
				if (ensureMsgf(Actor != nullptr, TEXT("GroupActor is not supposed to contain nullptr elements (maybe snapshot restored it incorrectly?)")))
				{
					Actor->GroupActor = nullptr;
				}
			}
		}

		template<typename TActor>
		bool AreEquivalent(ULevelSnapshot* Snapshot, const TArray<TObjectPtr<TActor>>& EditorData, const TArray<TObjectPtr<TActor>>& SnapshotData) const
		{
			for (int32 SnaphshotIdx = 0; SnaphshotIdx < SnapshotData.Num(); ++SnaphshotIdx)
			{
				if (SnapshotData[SnaphshotIdx] != nullptr // Could not be resolved, e.g. due to Level change?
					&& !HasEquivalentEditorActor(Snapshot, SnapshotData[SnaphshotIdx], EditorData))
				{
					return false;
				}
			}
			for (int32 EditorIdx = 0; EditorIdx < EditorData.Num(); ++EditorIdx)
			{
				if (!HasEquivalentSnapshotActor(Snapshot, EditorData[EditorIdx], SnapshotData))
				{
					return false;
				}
			}
			return true;
		}

		template<typename TActor>
		bool HasEquivalentEditorActor(ULevelSnapshot* Snapshot, AActor* SnapshotActor, const TArray<TObjectPtr<TActor>>& EditorActors) const
		{
			for (int32 i = 0; i < EditorActors.Num(); ++i)
			{
				if (AreActorsEquivalent(SnapshotActor, EditorActors[i], Snapshot->GetSerializedData(), Snapshot->GetCache()))
				{
					return true;
				}
			}
			return false;
		}
		template<typename TActor>
		bool HasEquivalentSnapshotActor(ULevelSnapshot* Snapshot, AActor* EditorActor, const TArray<TObjectPtr<TActor>>& SnapshotActors) const
		{
			for (int32 i = 0; i < SnapshotActors.Num(); ++i)
			{
				if (AreActorsEquivalent(SnapshotActors[i], EditorActor, Snapshot->GetSerializedData(), Snapshot->GetCache()))
				{
					return true;
				}
			}
			return false;
		}

		AActor* GetHiddenDummyActorInWorld(UWorld& World)
		{
			// Should not be nullptr
			return World.GetWorldSettings();
		}

	public:

		FActorGroupRestoration()
		{
			check(GroupActorsProperty && SubGroupsProperty);
		}

		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override
		{
			// This "redundant" if is actually an optimization 
			if (Params.LeafProperty == GroupActorsProperty || Params.LeafProperty == SubGroupsProperty)
			{
				const AGroupActor* EditorActor = Cast<AGroupActor>(Params.WorldActor);
				const AGroupActor* SnapshotActor = Cast<AGroupActor>(Params.SnapshotActor);
				check(EditorActor && SnapshotActor);
				
				if (Params.LeafProperty == GroupActorsProperty)
				{
					return AreEquivalent(Params.Snapshot, EditorActor->GroupActors, SnapshotActor->GroupActors)
						? EPropertyComparison::TreatEqual
						: EPropertyComparison::TreatUnequal;
				}
				
				return AreEquivalent(Params.Snapshot, EditorActor->SubGroups, SnapshotActor->SubGroups)
					? EPropertyComparison::TreatEqual
					: EPropertyComparison::TreatUnequal;
			}

			return EPropertyComparison::CheckNormally;
		}
		
		virtual void PreRemoveActor(const FPreRemoveActorParams& Params) override
		{
			if (AGroupActor* GroupActor = Cast<AGroupActor>(Params.ActorToRemove))
			{
				// This prevents the actors part of this group from also being deleted.
				// This still allows users to include them in the deleted set of actors (that's good).
				GroupActor->GroupActors.Reset();
				GroupActor->SubGroups.Reset();
			}
			else if (AGroupActor* OwningActor = Cast<AGroupActor>(Params.ActorToRemove->GroupActor))
			{
				AActor* DummyActor = GetHiddenDummyActorInWorld(*OwningActor->GetWorld());
				// Workflow: 1. Remove all actors in a group (e.g. they're all new) 2. Serialize new children into GroupActor
				// The removal process would accidentally delete remove the group actor because it would no longer have any children
				if (const FPropertySelection* Selection = Params.SelectedProperties.GetObjectSelection(OwningActor).GetPropertySelection()
					; Selection && Selection->IsPropertySelected(nullptr, GroupActorsProperty) && DummyActor)
				{
					// This is pretty hacky but it will prevent AGroupActor::PostRemove from deleting the actor
					OwningActor->GroupActors.Add(DummyActor);
					ActorsWithDummyChild.Add(OwningActor);
				}
			}
		}

		virtual void PostRemoveActors(const FPostRemoveActorsParams& Params) override
		{
			// "Fix" the hack we did in PreRemoveActor
			for (TWeakObjectPtr<AGroupActor> GroupActor : ActorsWithDummyChild)
			{
				if (GroupActor.IsValid())
				{
					AActor* DummyActor = GetHiddenDummyActorInWorld(*GroupActor->GetWorld());
					GroupActor->GroupActors.Remove(DummyActor);
				}
			}
			ActorsWithDummyChild.Reset();
		}

		virtual void PreApplySnapshotToActor(const FApplySnapshotToActorParams& Params) override
		{
			if (AGroupActor* GroupActor = Cast<AGroupActor>(Params.Actor))
			{
				const FRestorableObjectSelection Selection = Params.SelectedProperties.GetObjectSelection(GroupActor);
				
				// Clear every actor's AActor::GroupActor property if we're about to modify the GroupActor.
				// The new actors will get reregistered in PostApplySnapshotToActor
				// This is done so actors that will no longer be part of the group after the restore know they're no longer part of the group;
				// otherwise you can click an actor that was removed in the World Outliner and it will highlight the old group
				if (Selection.GetPropertySelection() && Selection.GetPropertySelection()->IsPropertySelected(nullptr, GroupActorsProperty))
				{
					UnregisterActorsFromGroup(*GroupActor, GroupActor->GroupActors);
				}
				if (Selection.GetPropertySelection() && Selection.GetPropertySelection()->IsPropertySelected(nullptr, SubGroupsProperty))
				{
					UnregisterActorsFromGroup(*GroupActor, GroupActor->SubGroups);
				}
			}
		}
		
		virtual void PostApplySnapshotToActor(const FApplySnapshotToActorParams& Params) override
		{
			if (AGroupActor* GroupActor = Cast<AGroupActor>(Params.Actor))
			{
				// Usually done by PostLoad, etc. Done in case we re-added the actor
				GroupActor->GetWorld()->ActiveGroupActors.AddUnique(GroupActor);
				
				RegisterActorsWithGroup(*GroupActor, GroupActor->GroupActors);
				RegisterActorsWithGroup(*GroupActor, GroupActor->SubGroups);

				// Hacky way of forcing GroupActor to reconsider whether it should exist...
				// References can fail to restore ... possibly there are no children
				GroupActor->Remove(*Cast<AGroupActor>(Params.Actor)); // Silly cast used here to fix silly static analysis warning...
			}
		}

		virtual FFilterResultData CanRecreateMissingActor(const FCanRecreateActorParams& Params) override
		{
			if (Params.Class == AGroupActor::StaticClass())
			{
				if (const TOptional<TNonNullPtr<AActor>> SnapshotActor = Params.DeserializeFromSnapshotFunc())
				{
					const AGroupActor* AsGroupActor = Cast<AGroupActor>(SnapshotActor->Get());
					check(AsGroupActor);
					return Algo::AllOf(AsGroupActor->GroupActors, [](AActor* Actor){ return Actor != nullptr; })
						? EFilterResult::DoNotCare : EFilterResult::Disallow;
				}
			}

			return EFilterResult::DoNotCare;
		}
	};
#endif
	
	void Register(FLevelSnapshotsModule& Module)
	{
#if WITH_EDITORONLY_DATA
		AddActorGroupSupport(Module);
		const TSharedRef<FActorGroupRestoration> ActorGroupSupport = MakeShared<FActorGroupRestoration>();
		Module.RegisterRestorationListener(ActorGroupSupport);
		Module.RegisterGlobalActorFilter(ActorGroupSupport);
		Module.RegisterPropertyComparer(AGroupActor::StaticClass(), ActorGroupSupport);
#endif
	}
}
