// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsModule.h"

namespace UE::LevelSnapshots::Private::ActorGroupRestoration
{
	void Register(FLevelSnapshotsModule& Module);
}
