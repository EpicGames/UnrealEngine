// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorLabelRestoration.h"

#include "Filtering/PropertySelection.h"

#include "GameFramework/Actor.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UnrealType.h"

namespace UE::LevelSnapshots::Private::ActorLabelRestoration
{
#if WITH_EDITOR // Actor labels are an editor only property
	static void AddActorLabelSupport(ILevelSnapshotsModule& Module)
	{
		const FProperty* ActorLabel = AActor::StaticClass()->FindPropertyByName(TEXT("ActorLabel"));
		if (ensure(ActorLabel))
		{
			Module.AddExplicitilySupportedProperties({ ActorLabel });
		}
	}

	/** Does a cosmetic update to actors that have their ActorLabel property restored by refreshing the world outliner with the new label */
	class FActorGroupRestoration : public IRestorationListener
	{
		FProperty* const ActorLabel = AActor::StaticClass()->FindPropertyByName(TEXT("ActorLabel"));
	public:

		FActorGroupRestoration()
		{
			ensure(ActorLabel);
		}
		
		//~ Begin IRestorationListener Interface
		virtual void PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) override
		{
			const bool bNeedsToUpdateActorLabel = Params.bWasRecreated
				|| (ensureMsgf(Params.PropertySelection, TEXT("Supposed to be valid when !bWasRecreated"))
					&& Params.PropertySelection.GetValue()->IsPropertySelected(nullptr, ActorLabel));
			if (AActor* const RestoredActor = Cast<AActor>(Params.Object);
				RestoredActor && bNeedsToUpdateActorLabel)
			{
				// Cannot call SetActorLabel directly because it checks whether the label has changed; the new value was already serialized into the actor
				FPropertyChangedEvent PropertyEvent(ActorLabel);
				RestoredActor->PostEditChangeProperty(PropertyEvent);
				FCoreDelegates::OnActorLabelChanged.Broadcast(RestoredActor);
			}
		}
		//~ End IRestorationListener Interface
	};
#endif
	
	void Register(FLevelSnapshotsModule& Module)
	{
#if WITH_EDITOR // Actor labels are an editor only property
		AddActorLabelSupport(Module);
		Module.RegisterRestorationListener(MakeShared<FActorGroupRestoration>());
#endif
	}
}