// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetUserDataRestoration.h"

#include "Restorability/Interfaces/IRestorationListener.h"
#include "Restorability/Interfaces/ISnapshotFilterExtender.h"
#include "Restorability/Interfaces/ITakeSnapshotListener.h"

#include "Components/ActorComponent.h"
#include "Engine/AssetUserData.h"
#include "Templates/Function.h"
#include "UObject/Package.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

namespace UE::LevelSnapshots::Private::AssetUserDataRestoration
{
	/**
	 * Makes sure that Level Snapshots ignores transient UAssetUserData in IInterface_AssetUserData providers, which is mainly UActorComponent.
	 * It does this by removing and adding back transient objects before and after taking, applying and filtering a snapshot, respectively.
	 */
	class FAssetUserDataRestoration : public ITakeSnapshotListener, public ISnapshotFilterExtender, public IRestorationListener
	{
		const FArrayProperty* AssetUserDataProperty;
	public:
		
		FAssetUserDataRestoration()
			: AssetUserDataProperty(CastField<FArrayProperty>(UActorComponent::StaticClass()->FindPropertyByName(TEXT("AssetUserData"))))
		{
			check(AssetUserDataProperty);
		}

		virtual void PreTakeObjectSnapshot(const FPreTakeObjectSnapshotEvent& Params) override
		{
			RemoveTransientAssetUserData(Params.Object);
		}
		virtual void PostTakeObjectSnapshot(const FPostTakeObjectSnapshotEvent& Params) override
		{
			AddBackTransientAssetUserData(Params.Object);
		}
		
		virtual void PreApplyFilters(const FPreApplyFiltersParams& Params) override
		{
			RemoveTransientAssetUserData(Params.EditorObject);
			RemoveTransientAssetUserDataFromSnapshotObject(Params.SnapshotObject, Params.EditorObject);

			// TODO: For backwards compatibility remove clear null references from UActorComponent::AssetUserData array property
		}
		virtual FPostApplyFiltersResult PostApplyFilters(const FPostApplyFiltersParams& Params) override
		{
			AddBackTransientAssetUserData(Params.EditorObject);
			// Don't bother adding back transient asset user data that was removed in PreApplyFilters because it should always be ignored anyways
			return {};
		}

		virtual void PreApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) override
		{
			RemoveTransientAssetUserData(Params.Object);
		}
		virtual void PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) override
		{
			AddBackTransientAssetUserData(Params.Object);
		}

	private:

		TMap<UObject*, TArray<UAssetUserData*>> RemovedAssetUserData;

		void RemoveTransientAssetUserDataFromSnapshotObject(UObject* SnapshotObject, const UObject* EditorCounterpart)
		{
			// Snapshot objects are slightly different... they are already in a transient package.
			// This means that all asset user data is transient and would be removed.
			// Hence, we'll remove those objects that match by class (since every UClass in the asset user data must be unique).

			if (const TArray<UAssetUserData*>* RemovedFromEditorCounterpart = RemovedAssetUserData.Find(EditorCounterpart))
			{
				const TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface = SnapshotObject;
				checkf(RemovedFromEditorCounterpart, TEXT("Should be valid because EditorCounterpart must have implemented IInterface_AssetUserData as there is data for it!"));
				for (const UAssetUserData* EditorAssetUserData : *RemovedFromEditorCounterpart)
				{
					AssetUserDataInterface->RemoveUserDataOfClass(EditorAssetUserData->GetClass());
				}
			}
		}
		
		void RemoveTransientAssetUserData(UObject* Object)
		{
			const TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface = Object;
			if (!AssetUserDataInterface)
			{
				return;
			}

			TArray<UAssetUserData*> TransientObjects;
			ForEachTransientAssetUserData(AssetUserDataInterface, [&TransientObjects](UAssetUserData* Data)
			{
				TransientObjects.Add(Data);
			});

			if (!TransientObjects.IsEmpty())
			{
				for (UAssetUserData* Data : TransientObjects)
				{
					AssetUserDataInterface->RemoveUserDataOfClass(Data->GetClass());
				}
				
				RemovedAssetUserData.Add(Object, MoveTemp(TransientObjects));
			}
		}

		void AddBackTransientAssetUserData(UObject* Object)
		{
			TArray<UAssetUserData*> TransientDataToRestore;
			if (RemovedAssetUserData.RemoveAndCopyValue(Object, TransientDataToRestore))
			{
				const TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface(Object);
				checkf(AssetUserDataInterface, TEXT("Interface cast should have worked in RemoveTransientAssetUserData"))
				for (UAssetUserData* Data : TransientDataToRestore)
				{
					AssetUserDataInterface->AddAssetUserData(Data);
				}
			}
		}

		void ForEachTransientAssetUserData(const TScriptInterface<IInterface_AssetUserData>& AssetUserDataInterface, TFunctionRef<void(UAssetUserData* TransientAssetUserData)> ProcessData)
		{
			auto IsTransient = [](UAssetUserData* Data)
			{
				UPackage* Package = Data ? Data->GetPackage() : nullptr;
				return Data && (Data->HasAnyFlags(RF_Transient) || Package->HasAnyFlags(RF_Transient));
			};

			if (const TArray<UAssetUserData*>* Array = AssetUserDataInterface->GetAssetUserDataArray())
			{
				for (UAssetUserData* AssetUserData : *Array)
				{
					if (IsTransient(AssetUserData))
					{
						ProcessData(AssetUserData);
					}
				}
			}
			// UActorComponent does not implement GetAssetUserDataArray ...
			else if (UActorComponent* Component = Cast<UActorComponent>(AssetUserDataInterface.GetObject()))
			{
				// ... so we must hack access to it with reflection
				FScriptArrayHelper AssetUserDataArray(AssetUserDataProperty, AssetUserDataProperty->ContainerPtrToValuePtr<void>(Component));
				for (int32 i = 0; i < AssetUserDataArray.Num(); ++i)
				{
					void* ElementPtr = AssetUserDataArray.GetRawPtr(i);
					TObjectPtr<UAssetUserData>* TypedPtr = static_cast<TObjectPtr<UAssetUserData>*>(ElementPtr);
					if (IsTransient(TypedPtr->Get()))
					{
						ProcessData(TypedPtr->Get());
					}
				}
			}
		}
	};
	
	void Register(FLevelSnapshotsModule& Module)
	{
		const TSharedRef<FAssetUserDataRestoration> AssetUserDataRestoration = MakeShared<FAssetUserDataRestoration>();
		Module.RegisterTakeSnapshotListener(AssetUserDataRestoration);
		Module.RegisterSnapshotFilterExtender(AssetUserDataRestoration);
		Module.RegisterRestorationListener(AssetUserDataRestoration);
	}
}