// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttachParentShowsTransformProperties.h"

#include "Filtering/PropertySelectionMap.h"
#include "Interfaces/ISnapshotFilterExtender.h"

#include "Components/SceneComponent.h"

namespace UE::LevelSnapshots::Private::AttachParentShowsTransformPropertiesFix
{
	/**
	 * By default only modified properties are shown. However in the below case we show the transform properties (Location,
	 * Rotation, and Scale) if one of the selected properties is the AttachParent.
	 * 
	 *	1. Create cube
	 *	2. Rotate it to 40 degrees
	 *  3. Take a snapshot
	 *  4. Create new empty actor
	 *	5. Attach cube to empty actor
	 *	6. Rotate empty actor by 10 degrees
	 *	7. Apply the snapshot
	 *	
	 *	Result: After applying, the cube has a diff because its rotation is 50 degrees.
	 *
	 *	The reason is because before applying the only thing that was different was the cube's attach parent. When the
	 *	parent actor is removed, its world-space transform is retained; actors are removed first and then the properties
	 *	of modified actors are saved. The solution is to add the cube's transform properties even though they're equal to
	 *	the snapshot version. Users will see the property in the UI and can opt to not restore it.
	 */
	class FAttachParentShowsTransformProperties : public ISnapshotFilterExtender
	{
		const FProperty* AttachParentProperty = USceneComponent::StaticClass()->FindPropertyByName(TEXT("AttachParent"));
		const FProperty* RelativeLocationProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName());
		const FProperty* RelativeRotationProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeRotationPropertyName());
		const FProperty* RelativeScaleProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName());
		const TArray<FLevelSnapshotPropertyChain> AdditionalPropertiesToFilter = MakeAdditionalPropertiesToFilter();
		
		TArray<FLevelSnapshotPropertyChain> MakeAdditionalPropertiesToFilter() const
		{
			const FLevelSnapshotPropertyChain RelativeLocation = RelativeLocationProperty;
			const FLevelSnapshotPropertyChain RelativeRotation = RelativeRotationProperty;
			const FLevelSnapshotPropertyChain RelativeScale = RelativeScaleProperty;
			TArray<FLevelSnapshotPropertyChain> Result  { RelativeLocation, RelativeRotation, RelativeScale };
			AddStructProperties(RelativeLocation, CastField<FStructProperty>(RelativeLocationProperty), Result);
			AddStructProperties(RelativeRotation, CastField<FStructProperty>(RelativeRotationProperty), Result);
			AddStructProperties(RelativeScale, CastField<FStructProperty>(RelativeScaleProperty), Result);
			return Result;
		}

		void AddStructProperties(const FLevelSnapshotPropertyChain& BaseChain, const FStructProperty* Property, TArray<FLevelSnapshotPropertyChain>& Result) const
		{
			for (TFieldIterator<const FProperty> RelativeLocPropIt(CastField<FStructProperty>(RelativeLocationProperty)->Struct); RelativeLocPropIt; ++RelativeLocPropIt)
			{
				Result.Emplace(BaseChain.MakeAppended(*RelativeLocPropIt));
			}
		}
		
	public:

		FAttachParentShowsTransformProperties()
		{
			check(AttachParentProperty && RelativeLocationProperty && RelativeRotationProperty && RelativeScaleProperty);
		}
		
		virtual FPostApplyFiltersResult PostApplyFilters(const FPostApplyFiltersParams& Params) override
		{
			if (Params.SelectedProperties.ShouldSerializeProperty(nullptr, AttachParentProperty))
			{
				return { AdditionalPropertiesToFilter, {} };
			}

			return {};
		}
	};

	void Register(FLevelSnapshotsModule& Module)
	{
		Module.RegisterSnapshotFilterExtender(MakeShared<FAttachParentShowsTransformProperties>());
	}
}
