// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsModule.h"

namespace UE::LevelSnapshots::Private::AttachParentShowsTransformPropertiesFix
{
	/**
	 * By default only modified properties are shown. However in the below case we show the transform properties (Location,
	 * Rotation, and Scale) if one of the selected properties is the AttachParent.
	 * 
	 *	1. Create cube
	 *	2. Rotate it to 40 degrees
	 *  3. Take a snapshot
	 *  4. Create new empty actor
	 *	5. Attach cube to empty actor
	 *	6. Rotate empty actor by 10 degrees
	 *	7. Apply the snapshot
	 *	
	 *	Result: After applying, the cube has a diff because its rotation is 50 degrees.
	 *
	 *	The reason is because before applying the only thing that was different was the cube's attach parent. When the
	 *	parent actor is removed, its world-space transform is retained; actors are removed first and then the properties
	 *	of modified actors are saved. The solution is to add the cube's transform properties even though they're equal to
	 *	the snapshot version. Users will see the property in the UI and can opt to not restore it.
	 */
	void Register(FLevelSnapshotsModule& Module);
}


