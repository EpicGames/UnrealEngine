// Copyright Epic Games, Inc. All Rights Reserved.

#include "Restorability/ClassRestorationSkipper.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

UE::LevelSnapshots::ISnapshotRestorabilityOverrider::ERestorabilityOverride UE::LevelSnapshots::FClassRestorationSkipper::IsActorDesirableForCapture(const AActor* Actor)
{
	const FSkippedClassList& SkippedClasses = GetSkippedClassListCallback.Execute();
	for (UClass* ActorClass = Actor->GetClass(); ActorClass; ActorClass = ActorClass->GetSuperClass())
	{
		if (SkippedClasses.SkippedClasses.Contains(ActorClass))
		{
			return ERestorabilityOverride::Disallow;
		}
	}
	
	return ERestorabilityOverride::DoNotCare;
}

UE::LevelSnapshots::ISnapshotRestorabilityOverrider::ERestorabilityOverride UE::LevelSnapshots::FClassRestorationSkipper::IsComponentDesirableForCapture(const UActorComponent* Component)
{
	const FSkippedClassList& SkippedClasses = GetSkippedClassListCallback.Execute();
	for (UClass* ComponentClass = Component->GetClass(); ComponentClass; ComponentClass = ComponentClass->GetSuperClass())
	{
		if (SkippedClasses.SkippedClasses.Contains(ComponentClass))
		{
			return ERestorabilityOverride::Disallow;
		}
	}
	
	return ERestorabilityOverride::DoNotCare;
}