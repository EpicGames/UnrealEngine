// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionRestoration.h"

#include "Params/PropertyComparisonParams.h"
#include "Filtering/Diffing/EquivalenceUtil.h"

#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "PhysicsEngine/BodyInstance.h"
#include "UObject/UnrealType.h"

void UE::LevelSnapshots::Private::FCollisionRestoration::Register(FLevelSnapshotsModule& Module)
{
	const TSharedRef<FCollisionRestoration> CollisionPropertyFix = MakeShared<FCollisionRestoration>();
	
	Module.RegisterSnapshotLoader(CollisionPropertyFix);
	Module.RegisterRestorationListener(CollisionPropertyFix);
	Module.RegisterPropertyComparer(UStaticMeshComponent::StaticClass(), CollisionPropertyFix);
}

UE::LevelSnapshots::Private::FCollisionRestoration::FCollisionRestoration()
{
	BodyInstanceProperty = FindFProperty<FProperty>(UPrimitiveComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UPrimitiveComponent, BodyInstance));
	ObjectTypeProperty = FindFProperty<FProperty>(FBodyInstance::StaticStruct(), FName("ObjectType"));
	CollisionEnabledProperty = FindFProperty<FProperty>(FBodyInstance::StaticStruct(), FName("CollisionEnabled"));
	CollisionResponsesProperty = FindFProperty<FProperty>(FBodyInstance::StaticStruct(), FName("CollisionResponses"));
	check(BodyInstanceProperty && ObjectTypeProperty && CollisionEnabledProperty && CollisionResponsesProperty);
}

UE::LevelSnapshots::IPropertyComparer::EPropertyComparison UE::LevelSnapshots::Private::FCollisionRestoration::ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const
{
	if (Params.LeafProperty == BodyInstanceProperty)
	{
		UStaticMeshComponent* SnapshotObject = Cast<UStaticMeshComponent>(Params.SnapshotObject);
		UStaticMeshComponent* WorldObject = Cast<UStaticMeshComponent>(Params.WorldObject);
#if UE_BUILD_DEBUG
		// Should never fail, double-check on debug builds
		check(SnapshotObject);
		check(WorldObject);
#endif

		if (HaveNonDefaultCollisionPropertiesChanged(Params, SnapshotObject, WorldObject))
		{
			return EPropertyComparison::CheckNormally;
		}

		// These properties may have different values but they do not matter if UStaticMeshComponent::bUseDefaultCollision == true
		const FBodyInstance& SnapshotBody = SnapshotObject->BodyInstance;
		const FBodyInstance& WorldBody = WorldObject->BodyInstance;
		const bool bDidDefaultCollisionPropertiesChange =
			SnapshotBody.GetCollisionEnabled() != WorldBody.GetCollisionEnabled()
			|| SnapshotBody.GetObjectType() != WorldBody.GetObjectType()
			|| SnapshotBody.GetCollisionResponse() != WorldBody.GetCollisionResponse();

		const bool bDefaultCollisionConfigIsEqual = SnapshotObject->bUseDefaultCollision == WorldObject->bUseDefaultCollision;
		const bool bUseDefaultCollision = SnapshotObject->bUseDefaultCollision;
		return bDefaultCollisionConfigIsEqual && (bUseDefaultCollision || !bDidDefaultCollisionPropertiesChange) 
			? EPropertyComparison::TreatEqual : EPropertyComparison::CheckNormally;
	}

	return EPropertyComparison::CheckNormally;
}

namespace
{
	void UpdateCollisionProfile(UPrimitiveComponent* PrimitiveComponent)
	{
#if WITH_EDITOR
		PrimitiveComponent->UpdateCollisionProfile();
#else
		PrimitiveComponent->BodyInstance.LoadProfileData(false);
#endif
	}
}

void UE::LevelSnapshots::Private::FCollisionRestoration::PostLoadSnapshotObject(const FPostLoadSnapshotObjectParams& Params)
{
	if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Params.SnapshotObject))
	{
		// This is needed to restore transient collision profile data.
		UpdateCollisionProfile(PrimitiveComponent);
	}
}

void UE::LevelSnapshots::Private::FCollisionRestoration::PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params)
{
	if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Params.Object))
	{
		// This is needed to restore transient collision profile data.
		// Technically, we need to check whether collision profile was modified ... calling it always is easier though
		UpdateCollisionProfile(PrimitiveComponent);
	}
}

bool UE::LevelSnapshots::Private::FCollisionRestoration::HaveNonDefaultCollisionPropertiesChanged(const FPropertyComparisonParams& Params, UStaticMeshComponent* SnapshotObject, UStaticMeshComponent* WorldObject) const
{
	void* SnapshotValuePtr = BodyInstanceProperty->ContainerPtrToValuePtr<void>(SnapshotObject);
	void* EditorValuePtr = BodyInstanceProperty->ContainerPtrToValuePtr<void>(WorldObject);
	for (TFieldIterator<FProperty> FieldIt(FBodyInstance::StaticStruct()); FieldIt; ++FieldIt)
	{
		const FProperty* Property = *FieldIt;
		const bool bIsAffectedByDefaultCollision = Property == ObjectTypeProperty || Property == CollisionEnabledProperty || Property == CollisionResponsesProperty;
		if (!bIsAffectedByDefaultCollision && !AreSnapshotAndOriginalPropertiesEquivalent(Params.Snapshot, Property, SnapshotValuePtr, EditorValuePtr, SnapshotObject->GetOwner(), WorldObject->GetOwner()))
		{
			return true;
		}
	}
	return false;
}
