// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsModule.h"
#include "Interfaces/IPropertyComparer.h"
#include "Interfaces/ISnapshotLoader.h"

class UStaticMeshComponent;

namespace UE::LevelSnapshots::Private
{
	/**
	* UPrimitiveComponent::BodyInstance requires special logic for restoring & loading collision information.
	*/
	class FCollisionRestoration
		:
		public IPropertyComparer,
		public ISnapshotLoader,
		public IRestorationListener
	{
	public:
	
		static void Register(FLevelSnapshotsModule& Module);

		FCollisionRestoration();

		//~ Begin IPropertyComparer Interface
		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override;
		//~ End IPropertyComparer Interface

		//~ Begin ISnapshotLoader Interface
		virtual void PostLoadSnapshotObject(const FPostLoadSnapshotObjectParams& Params) override;
		//~ End ISnapshotLoader Interface

		//~ Begin IRestorationListener Interface
		virtual void PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) override;
		//~ End IRestorationListener Interface

	private:

		bool HaveNonDefaultCollisionPropertiesChanged(
			const FPropertyComparisonParams& Params, 
			UStaticMeshComponent* SnapshotObject, 
			UStaticMeshComponent* WorldObject
			) const;
	
		const FProperty* BodyInstanceProperty;
		const FProperty* ObjectTypeProperty;
		const FProperty* CollisionEnabledProperty;
		const FProperty* CollisionResponsesProperty;
	};
}


