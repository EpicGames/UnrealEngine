// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineTypesRestorationFence.h"

#include "ActorGroupRestoration.h"
#include "ActorLabelRestoration.h"
#include "AssetUserDataRestoration.h"
#include "AttachParentShowsTransformProperties.h"
#include "CollisionRestoration.h"
#include "GridPlacementRestoration.h"
#include "LevelInstanceRestoration.h"
#include "PCGRestoration.h"
#include "SpecialActorPropertySupport.h"

#include "Algo/Transform.h"
#include "Components/SkeletalMeshComponent.h"
#include "MaterialInstanceSupport.h"
#include "Engine/Brush.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "Materials/MaterialInstance.h"

namespace UE::LevelSnapshots::Private::EngineTypesRestorationFence
{
	static void AddSoftObjectPathSupport(ILevelSnapshotsModule& Module)
	{
		// FSnapshotRestorability::IsRestorableProperty requires properties to have the CPF_Edit specifier
		// FSoftObjectPath does not have this so we need to explicitly allow its properties

		UStruct* SoftObjectClassPath = FindObject<UStruct>(nullptr, TEXT("/Script/CoreUObject.SoftObjectPath"));
		if (!ensureMsgf(SoftObjectClassPath, TEXT("Investigate why this class could not be found")))
		{
			return;
		}

		TSet<const FProperty*> SoftObjectPathProperties;
		Algo::Transform(TFieldRange<const FProperty>(SoftObjectClassPath), SoftObjectPathProperties, [](const FProperty* Prop) { return Prop;} );
		Module.AddExplicitilySupportedProperties(SoftObjectPathProperties);
	}

	static void AddAttachParentSupport(ILevelSnapshotsModule& Module)
	{
		// These properties are not visible by default because they're not CPF_Edit
		const FProperty* AttachParent = USceneComponent::StaticClass()->FindPropertyByName(FName("AttachParent"));
		const FProperty* AttachSocketName = USceneComponent::StaticClass()->FindPropertyByName(FName("AttachSocketName"));
		// RootComponent is usually set automatically but sometimes not... for example spawning AActor with instanced components only
		const FProperty* RootComponent = AActor::StaticClass()->FindPropertyByName(FName("RootComponent"));
		if (ensure(AttachParent && AttachSocketName))
		{
			Module.AddExplicitilySupportedProperties({ AttachParent, AttachSocketName, RootComponent });
		}
	}

	static void AddMediaPlateSupport(ILevelSnapshotsModule& Module)
	{
		UStruct* MediaPlateComponentClass = FindObject<UStruct>(nullptr, TEXT("/Script/MediaPlate.MediaPlateComponent"));
		if (!MediaPlateComponentClass)
		{
			return;
		}
		
		const FProperty* MediaPlaylistProperty = MediaPlateComponentClass->FindPropertyByName(TEXT("MediaPlaylist"));
		if (ensure(MediaPlaylistProperty))
		{
			Module.AddExplicitilySupportedProperties({ MediaPlaylistProperty });
		}
	}

	static void DisableIrrelevantBrushSubobjects(ILevelSnapshotsModule& Module)
	{
#if WITH_EDITORONLY_DATA
		// ABrush::BrushBuilder is CPF_Edit but no user ever cares about it. We don't want it to make volumes to show up as changed.
		const FProperty* BrushBuilder = ABrush::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ABrush, BrushBuilder));
		if (ensure(BrushBuilder))
		{
			Module.AddExplicitlyUnsupportedProperties({ BrushBuilder });
		}
#endif
	}

	static void DisableIrrelevantWorldSettings(ILevelSnapshotsModule& Module)
	{
		// AWorldSettings::NavigationSystemConfig is CPF_Edit but no user ever cares about it.
		const FProperty* NavigationSystemConfig = AWorldSettings::StaticClass()->FindPropertyByName(FName("NavigationSystemConfig"));
		if (ensure(NavigationSystemConfig))
		{
			Module.AddExplicitlyUnsupportedProperties({ NavigationSystemConfig });
		}
	}

	static void DisableIrrelevantMaterialInstanceProperties(ILevelSnapshotsModule& Module)
	{
		// This property causes diffs sometimes for unexplained reasons when creating in construction script... does not seem to be important
		const FProperty* BasePropertyOverrides = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, BasePropertyOverrides));
		if (ensure(BasePropertyOverrides))
		{
			Module.AddExplicitlyUnsupportedProperties({ BasePropertyOverrides });
		}
	}

	static void DisableIrrelevantActorProperties(ILevelSnapshotsModule& Module)
	{
#if WITH_EDITORONLY_DATA
		const FProperty* ActorGuid = AActor::StaticClass()->FindPropertyByName(FName("ActorGuid"));
		if (ensure(ActorGuid))
		{
			Module.AddExplicitlyUnsupportedProperties({ ActorGuid });
		}
#endif

		// Level Snapshots must spawn with AlwaysSpawn but since this is a property on the actor as well, they will show up as changed...
		// We'll just not support this property
		const FProperty* SpawnCollisionHandlingMethod = AActor::StaticClass()->FindPropertyByName(FName("SpawnCollisionHandlingMethod"));
		if (ensure(SpawnCollisionHandlingMethod))
		{
			Module.AddExplicitlyUnsupportedProperties({ SpawnCollisionHandlingMethod });
		}
	}

	static void DisableClothingSimulationFactory(ILevelSnapshotsModule& Module)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// This property is directly set by RegisterComponent and similar functions... we have no chance of restoring this property properly
		const FProperty* ClothingSimulationFactory = USkeletalMeshComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, ClothingSimulationFactory));
		if (ensure(ClothingSimulationFactory))
		{
			Module.AddExplicitlyUnsupportedProperties({ ClothingSimulationFactory });
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	void RegisterSpecialEngineTypeSupport(FLevelSnapshotsModule& Module)
	{
		// Enable / disable troublesome properties
		AddSoftObjectPathSupport(Module);
		AddAttachParentSupport(Module);
		AddMediaPlateSupport(Module);
		DisableIrrelevantBrushSubobjects(Module);
		DisableIrrelevantWorldSettings(Module);
		DisableIrrelevantMaterialInstanceProperties(Module);
		DisableIrrelevantActorProperties(Module);
		DisableClothingSimulationFactory(Module);

		// Interact with special engine features
		FCollisionRestoration::Register(Module);
		GridPlacementRestoration::Register(Module);
		ActorGroupRestoration::Register(Module);
		SpecialActorPropertySupport::Register(Module);
		AttachParentShowsTransformPropertiesFix::Register(Module);
		LevelInstanceRestoration::Register(Module);
		MaterialInstanceSupport::Register(Module);
		ActorLabelRestoration::Register(Module);
		PCGRestoration::Register(Module);
		AssetUserDataRestoration::Register(Module);
	}
}
