// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsModule.h"

namespace UE::LevelSnapshots::Private::EngineTypesRestorationFence
{
	/** Adds support for special cases for misc engine types */
	void RegisterSpecialEngineTypeSupport(FLevelSnapshotsModule& Module);
};
