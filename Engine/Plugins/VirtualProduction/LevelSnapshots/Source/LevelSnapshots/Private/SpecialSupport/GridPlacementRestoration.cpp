// Copyright Epic Games, Inc. All Rights Reserved.

#include "GridPlacementRestoration.h"

#include "Interfaces/IPropertyComparer.h"
#include "Params/PropertyComparisonParams.h"

#include "GameFramework/Actor.h"

#if WITH_EDITORONLY_DATA

namespace UE::LevelSnapshots::Private::Internal
{
	/**
	* UPrimitiveComponent::BodyInstance requires special logic for restoring & loading collision information.
	*/
	class FGridPlacementRestoration
		:
		public IPropertyComparer
	{
		const FProperty* GridPlacementProperty;
	public:
	
		static void Register(FLevelSnapshotsModule& Module)
		{
			const TSharedRef<FGridPlacementRestoration> GridPlacementFix = MakeShared<FGridPlacementRestoration>();
			Module.RegisterPropertyComparer(AActor::StaticClass(), GridPlacementFix);
		}

		FGridPlacementRestoration()
		{
			GridPlacementProperty = AActor::StaticClass()->FindPropertyByName(FName("GridPlacement"));
			check(GridPlacementProperty);
		}

		virtual IPropertyComparer::EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override
		{
			if (Params.LeafProperty == GridPlacementProperty)
			{
				return EPropertyComparison::CheckNormally;
			}

			return EPropertyComparison::CheckNormally;
		}
	};
}

#endif

void UE::LevelSnapshots::Private::GridPlacementRestoration::Register(FLevelSnapshotsModule& Module)
{
#if WITH_EDITORONLY_DATA
	Internal::FGridPlacementRestoration::Register(Module);
#endif
}
