// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceRestoration.h"

#include "ILevelSnapshotsModule.h"
#include "Restorability/Interfaces/ISnapshotRestorabilityOverrider.h"

#include "LevelInstance/LevelInstanceActor.h"

namespace UE::LevelSnapshots::Private::LevelInstanceRestoration
{
	/** Adds support for restoring level instances */
	class FLevelInstanceRestoration : public ISnapshotRestorabilityOverrider
	{
	public:
		
		virtual ERestorabilityOverride IsComponentDesirableForCapture(const UActorComponent* Component) override
		{
			// The root component is native and does not have a corresponding UPROPERTY() thus FComponentEditorUtils::CanEditComponentInstance will return false.
			// That would skip the capture of the component - we explicitly allow it (and non-root components for no particular reason) as well.
			return Component->GetOwner()->IsA<ALevelInstance>()
				? ERestorabilityOverride::Allow
				: ERestorabilityOverride::DoNotCare;
		}
	};
	
	void Register(ILevelSnapshotsModule& Module)
	{
		const TSharedRef<FLevelInstanceRestoration> LevelInstanceSupport = MakeShared<FLevelInstanceRestoration>();
		Module.RegisterRestorabilityOverrider(LevelInstanceSupport);
	}
}
