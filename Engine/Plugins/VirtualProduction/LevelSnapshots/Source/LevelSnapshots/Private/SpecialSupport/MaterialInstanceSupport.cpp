// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialInstanceSupport.h"

#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Params/PropertyComparisonParams.h"

#include "Materials/MaterialInstance.h"
#include "UObject/UnrealType.h"

namespace UE::LevelSnapshots::Private::MaterialInstanceSupport
{
	/** Sometimes the parameters (e.g. ScalarParameterValues) compared are out of order causing the material to show up as changed. */
	class FMaterialInstancePropertyComparer : public IPropertyComparer
	{
		const FProperty* Property_ScalarParameterValues = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, ScalarParameterValues));
		const FProperty* Property_VectorParameterValues = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, VectorParameterValues));
		const FProperty* Property_DoubleVectorParameterValues = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, DoubleVectorParameterValues));
		const FProperty* Property_TextureParameterValues = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, TextureParameterValues));
		const FProperty* Property_RuntimeVirtualTextureParameterValues = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, RuntimeVirtualTextureParameterValues));
		const FProperty* Property_FontParameterValues = UMaterialInstance::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialInstance, FontParameterValues));
	public:

		FMaterialInstancePropertyComparer()
		{
			check(Property_ScalarParameterValues
				&& Property_VectorParameterValues
				&& Property_DoubleVectorParameterValues
				&& Property_TextureParameterValues
				&& Property_RuntimeVirtualTextureParameterValues
				&& Property_FontParameterValues);
		}
		
		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override
		{
			// Could be written "nicer" but this code will be called often so make it fast to compute the if
			const bool bIsCustomProperty = Params.LeafProperty == Property_ScalarParameterValues
				|| Params.LeafProperty == Property_VectorParameterValues
				|| Params.LeafProperty == Property_DoubleVectorParameterValues
				|| Params.LeafProperty == Property_TextureParameterValues
				|| Params.LeafProperty == Property_RuntimeVirtualTextureParameterValues
				|| Params.LeafProperty == Property_FontParameterValues;
			
			if (bIsCustomProperty)
			{
				return AreArraysTransmutation(Params, CastFieldChecked<FArrayProperty>(Params.LeafProperty), Params.SnapshotContainer, Params.WorldContainer)
					? EPropertyComparison::TreatEqual
					: EPropertyComparison::TreatUnequal;
			}
			
			return EPropertyComparison::CheckNormally;
		}

		bool AreArraysTransmutation(const FPropertyComparisonParams& Params, const FArrayProperty* Property, void* SnapshotContainerPtr, void* WorldContainerPtr) const
		{
			void* SnapshotValuePtr = Property->ContainerPtrToValuePtr<void>(SnapshotContainerPtr);
			void* WorldValuePtr = Property->ContainerPtrToValuePtr<void>(WorldContainerPtr);

			FScriptArrayHelper SnapshotArray(Property, SnapshotValuePtr);
			FScriptArrayHelper WorldArray(Property, WorldValuePtr);
			if (SnapshotArray.Num() != WorldArray.Num())
			{
				return false;
			}

			for (int32 i = 0; i < SnapshotArray.Num(); ++i)
			{
				void* SnapshotElementValuePtr = SnapshotArray.GetRawPtr(i);
				if (!ContainsEquivalentElement(Params, Property, SnapshotElementValuePtr, WorldArray))
				{
					return false;
				}
			}
			
			return true;
		}

		bool ContainsEquivalentElement(const FPropertyComparisonParams& Params, const FArrayProperty* Property, void* SnapshotElementValuePtr, FScriptArrayHelper& WorldArray) const
		{
			for (int32 j = 0; j < WorldArray.Num(); ++j)
			{
				void* WorldElementValuePtr = WorldArray.GetRawPtr(j);
				if (AreSnapshotAndOriginalPropertiesEquivalent(Params.Snapshot, Property->Inner, SnapshotElementValuePtr, WorldElementValuePtr, Params.SnapshotActor, Params.WorldActor))
				{
					return true;
				}
			}
			return false;
		}
	};
	
	void Register(FLevelSnapshotsModule& Module)
	{
		const TSharedRef<FMaterialInstancePropertyComparer> MaterialInstancePropertyComparer = MakeShared<FMaterialInstancePropertyComparer>();
		Module.RegisterPropertyComparer(UMaterialInstance::StaticClass(), MaterialInstancePropertyComparer);
	}
}
