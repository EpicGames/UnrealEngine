// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGRestoration.h"

#include "Restorability/Interfaces/ISnapshotRestorabilityOverrider.h"

#include "Components/StaticMeshComponent.h"
#if WITH_EDITOR
#include "Interfaces/IPluginManager.h"
#endif

namespace UE::LevelSnapshots::Private::PCGRestoration
{
	/** Prevents restoration of components added to APCGVolume because they are refreshed and re-generated anyways. */
	class FPCGRestoration : public ISnapshotRestorabilityOverrider
	{
		/** Class of APCGVolume. Nullptr if PCG plugin is not enabled. */
		UClass* PCGVolumeClass;
	public:

		FPCGRestoration()
			: PCGVolumeClass(FindObject<UClass>(nullptr, TEXT("/Script/PCG.PCGVolume")))
		{
#if WITH_EDITOR
			const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PCG"));
			ensureMsgf(!Plugin || !Plugin->IsEnabled() || PCGVolumeClass, TEXT("PCG plugin is enabled but failed to find APCGVolume class. Was the class renamed?"));
#endif
		}
		
		virtual ERestorabilityOverride IsComponentDesirableForCapture(const UActorComponent* Component) override
		{
			const bool bIsOnPCGVolumne = PCGVolumeClass
				// Do not restore generated mesh components attached to APCGVolume
				&& Component->IsA<UStaticMeshComponent>()
				&& Component->GetOwner()->GetClass() == PCGVolumeClass;
			return bIsOnPCGVolumne
				? ERestorabilityOverride::Disallow
				: ERestorabilityOverride::DoNotCare;
		}
	};
	
	void Register(FLevelSnapshotsModule& Module)
	{
		Module.RegisterRestorabilityOverrider(MakeShared<FPCGRestoration>());
	}
}
