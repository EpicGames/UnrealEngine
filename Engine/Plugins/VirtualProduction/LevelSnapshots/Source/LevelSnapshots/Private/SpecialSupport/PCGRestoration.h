// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsModule.h"

/** PCG = Procedural Content Generation */
namespace UE::LevelSnapshots::Private::PCGRestoration
{
	void Register(FLevelSnapshotsModule& Module);
}
