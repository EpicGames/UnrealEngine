// Copyright Epic Games, Inc. All Rights Reserved.

#include "Restorability/SnapshotRestorability.h"

#include "Interfaces/IActorSnapshotFilter.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsModule.h"
#include "SnapshotCustomVersion.h"
#include "WorldSnapshotData.h"

#include "AI/NavigationSystemConfig.h"
#include "Algo/AllOf.h"
#include "Components/ActorComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/BrushBuilder.h"
#include "Engine/LevelScriptActor.h"
#include "GameFramework/Actor.h"
#include "GameFramework/DefaultPhysicsVolume.h"
#include "GameFramework/WorldSettings.h"
#include "Landscape.h"
#include "LandscapeGizmoActor.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR
#include "ActorEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"
#endif

namespace UE::LevelSnapshots::Restorability::Private::Internal
{
	static bool DoesActorHaveSupportedClass(const UClass* ActorClass)
	{
		const TSet<UClass*> UnsupportedClasses = 
		{
			ALevelScriptActor::StaticClass(),		// The level blueprint. Filtered out to avoid external map errors when saving a snapshot.
            ADefaultPhysicsVolume::StaticClass(),	// Does not show up in world outliner; always spawned with world.

			// Until proper landscape support is implemented, let's completely ignore landscapes to avoid capturing incomplete data
			ALandscapeProxy::StaticClass(),
			ALandscapeGizmoActor::StaticClass()
        };

		return Algo::AllOf(UnsupportedClasses, [ActorClass](UClass* Class) { return !ActorClass->IsChildOf(Class); });
	}

	static bool DoesActorHaveSupportedClassForRemoving(const AActor* Actor)
	{
		const TSet<UClass*> UnsupportedClasses = 
		{
			AWorldSettings::StaticClass(),			// Every sublevel creates a world settings. We do not want to be respawning them

			// Until proper landscape support is implemented, let's completely ignore landscapes to avoid capturing incomplete data
			ALandscapeProxy::StaticClass(),
			ALandscapeGizmoActor::StaticClass()
		};
		
		return Algo::AllOf(UnsupportedClasses, [Actor](UClass *Class) { return !Actor->IsA(Class); });
	}

	static bool DoesComponentHaveSupportedClassForCapture(const UActorComponent* Component)
	{
		const TSet<UClass*> UnsupportedClasses = 
		{
			UBillboardComponent::StaticClass(),		// Attached to all editor actors > It always has a different name so we will never be able to match it.
        };
		
		return Algo::AllOf(UnsupportedClasses, [Component](UClass *Class) { return !Component->IsA(Class); });
	}

	static bool DoesSubobjecttHaveSupportedClassForCapture(const UObject* Subobject)
	{
		const TSet<UClass*> UnsupportedClasses = 
		{
			UBrushBuilder::StaticClass(),				// Does not have state. Referenced by AVolumes, ABrushes, e.g. APostProcessVolume or ALightmassImportanceVolume
			UNavigationSystemConfig::StaticClass()		
		};
		
		return Algo::AllOf(UnsupportedClasses, [Subobject](UClass *Class) { return !Subobject->IsA(Class); });
	}

	static USceneComponent* GetParentComponent(const UActorComponent* Component)
	{
		if (const USceneComponent* AsSceneComponent = Cast<USceneComponent>(Component))
		{
			const bool bIsParentValid = AsSceneComponent
				&& AsSceneComponent->GetAttachParent()
				&& Component->GetOwner() != nullptr
				&& AsSceneComponent->GetAttachParent()->GetOwner() == Component->GetOwner();
			return bIsParentValid ? AsSceneComponent->GetAttachParent() : nullptr;
		}
		return nullptr;
	}
}

bool UE::LevelSnapshots::Restorability::IsActorDesirableForCapture(const AActor* Actor)
{
	SCOPED_SNAPSHOT_CORE_TRACE(IsActorDesirableForCapture);

	if (!IsValid(Actor) || !Private::Internal::DoesActorHaveSupportedClass(Actor->GetClass()))
	{
		return false;
	}
	
	bool bSomebodyAllowed = false;
	const TArray<TSharedRef<ISnapshotRestorabilityOverrider>>& Overrides = UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().GetOverrides(); 
	for (const TSharedRef<ISnapshotRestorabilityOverrider>& Override : Overrides)
	{
		const ISnapshotRestorabilityOverrider::ERestorabilityOverride Result = Override->IsActorDesirableForCapture(Actor);
		bSomebodyAllowed |= Result == ISnapshotRestorabilityOverrider::ERestorabilityOverride::Allow;
		if (Result == ISnapshotRestorabilityOverrider::ERestorabilityOverride::Disallow)
		{
			return false;
		}
	}

	if (bSomebodyAllowed)
	{
		return true;
	}
	
	return !Actor->IsTemplate()								// Should never happen, but we never want CDOs
			&& !Actor->HasAnyFlags(RF_Transient)				// Don't add transient actors in non-play worlds
			&& Actor->HasAnyFlags(RF_Transactional)				// Non transactional actors are usually modified by some other system, e.g. UChildActorComponent's child actor is not transactional
#if WITH_EDITOR
            && Actor->IsEditable()
            && Actor->IsListedInSceneOutliner() 				// Only add actors that are allowed to be selected and drawn in editor
            && !FActorEditorUtils::IsABuilderBrush(Actor)		// Don't add the builder brush
#endif
        ;	
}

bool UE::LevelSnapshots::Restorability::IsActorRestorable(const AActor* Actor)
{
	const bool bIsChildActor = Actor->GetParentComponent() != nullptr;
	return !bIsChildActor && IsActorDesirableForCapture(Actor);
}

bool UE::LevelSnapshots::Restorability::IsComponentDesirableForCapture(const UActorComponent* Component)
{
	SCOPED_SNAPSHOT_CORE_TRACE(IsComponentDesirableForCapture);

	if (!IsValid(Component))
	{
		return false;
	}
	
	bool bSomebodyAllowed = false;
	const TArray<TSharedRef<ISnapshotRestorabilityOverrider>>& Overrides = UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().GetOverrides(); 
	for (const TSharedRef<ISnapshotRestorabilityOverrider>& Override : Overrides)
	{
		const ISnapshotRestorabilityOverrider::ERestorabilityOverride Result = Override->IsComponentDesirableForCapture(Component);
		bSomebodyAllowed |= Result == ISnapshotRestorabilityOverrider::ERestorabilityOverride::Allow;
		if (Result == ISnapshotRestorabilityOverrider::ERestorabilityOverride::Disallow)
		{
			return false;
		}
	}

	constexpr bool bAllowUserContructionScriptComps = false;
	const bool bIsAllowed =
			// Components created in construction script are not supported
			Component->CreationMethod != EComponentCreationMethod::UserConstructionScript
			&& Private::Internal::DoesComponentHaveSupportedClassForCapture(Component)
			&& !Component->HasAnyFlags(RF_Transient)
#if WITH_EDITORONLY_DATA
			// Generally, snapshots should only restore snapshots that are editable. However, even if the root component is not editable, we still want to be able to restore the actor's transform.
			// Ideally, we may not want to capture all component data in that case but for now that is exactly what we will do.
			// For any future developer: we could add a IPropertyComparer implementation that marks all non-transform properties as unchanged.
			&& (Component == Component->GetOwner()->GetRootComponent() || FComponentEditorUtils::CanEditComponentInstance(Component, Private::Internal::GetParentComponent(Component), bAllowUserContructionScriptComps))
#endif
	;
	return bSomebodyAllowed || bIsAllowed;
}

bool UE::LevelSnapshots::Restorability::IsSubobjectClassDesirableForCapture(const UClass* SubobjectClass)
{
	SCOPED_SNAPSHOT_CORE_TRACE(IsSubobjectClassDesirableForCapture);
	return Private::Internal::DoesSubobjecttHaveSupportedClassForCapture(SubobjectClass) && !UE::LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().ShouldSkipSubobjectClass(SubobjectClass);
}

bool UE::LevelSnapshots::Restorability::IsSubobjectDesirableForCapture(const UObject* Subobject)
{
	checkf(!Subobject->IsA<UClass>(), TEXT("Do you have a typo in your code?"));
	
	if (const UActorComponent* Component = Cast<UActorComponent>(Subobject))
	{
		return IsComponentDesirableForCapture(Component);	
	}

	return IsSubobjectClassDesirableForCapture(Subobject->GetClass()) 
		&& !Subobject->HasAnyFlags(RF_Transient);
}

bool UE::LevelSnapshots::Restorability::IsPropertyDesirableForCapture(const FProperty* Property)
{
	SCOPED_SNAPSHOT_CORE_TRACE(IsPropertyDesirableForCapture);
	
	const bool bIsExplicitlyIncluded = IsPropertyExplicitlySupportedForCapture(Property);
	const bool bIsExplicitlyExcluded = IsPropertyExplicitlyUnsupportedForCapture(Property);

	// To avoid saving every single subobject, only save the editable ones.
	const uint64 EditablePropertyFlag = CPF_Edit;
	const bool bIsObjectProperty = CastField<FObjectPropertyBase>(Property) != nullptr;

	// TODO: Technically, we should only save editable/visible properties. However, we not know whether this will cause any trouble for existing snapshots or has unintended side-effects.
	const bool bIsAllowed = !bIsExplicitlyExcluded
		&& (bIsExplicitlyIncluded
			|| !bIsObjectProperty															// Legacy / safety reasons: allow all non-object properties
			|| (bIsObjectProperty && Property->HasAnyPropertyFlags(EditablePropertyFlag))); // Implementing object support now... rather save less than too many subobjects
	return bIsAllowed;
}

bool UE::LevelSnapshots::Restorability::IsPropertyExplicitlyUnsupportedForCapture(const FProperty* Property)
{
	SCOPED_SNAPSHOT_CORE_TRACE(IsPropertyExplicitlyUnsupportedForCapture);
	return LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().IsPropertyExplicitlyUnsupported(Property);
}

bool UE::LevelSnapshots::Restorability::IsPropertyExplicitlySupportedForCapture(const FProperty* Property)
{
	SCOPED_SNAPSHOT_CORE_TRACE(IsPropertyExplicitlySupportedForCapture);
	return LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().IsPropertyExplicitlySupported(Property);
}

bool UE::LevelSnapshots::Restorability::ShouldConsiderMatchedActorForRestoration(const FCanModifyMatchedActorParams& Params, FText* ExclusionReason)
{
	return LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().CanModifyMatchedActor(Params, ExclusionReason);
}

bool UE::LevelSnapshots::Restorability::ShouldConsiderRemovedActorForRecreation(const FCanRecreateActorParams& Params, FText* ExclusionReason)
{
	// Two reasons RF_Transactional is required:
	// 1. It will not show up in Multiuser otherwise
	// 2. If an actor is no RF_Transactional, it is usually because some system intentionally removed that flag, e.g. UChildActorComponent
	const bool bTransacts = (Params.ObjectFlags& RF_Transactional) != 0;
	const bool bWereFlagsSaved = Params.ObjectFlags != 0 // Objects usually have at least one flag so if it's 0 then it is very likely the flags were not saved...
		// ... support for saving flags was added some time before ClassArchetypeRefactor. We check against ClassArchetypeRefactor for the unlikely case that in the future an object really has no flags.
		|| Params.WorldData.SnapshotVersionInfo.GetSnapshotCustomVersion() >= FSnapshotCustomVersion::ClassArchetypeRefactor;
	
	return (bTransacts || !bWereFlagsSaved)
		&& LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().CanRecreateActor(Params, ExclusionReason);
}

bool UE::LevelSnapshots::Restorability::ShouldConsiderNewActorForRemoval(const AActor* Actor, FText* ExclusionReason)
{
	return Private::Internal::DoesActorHaveSupportedClassForRemoving(Actor)
		&& IsActorDesirableForCapture(Actor)
		&& LevelSnapshots::Private::FLevelSnapshotsModule::GetInternalModuleInstance().CanDeleteActor(Actor, ExclusionReason);
}

bool UE::LevelSnapshots::Restorability::IsRestorableProperty(const FProperty* LeafProperty)
{
	using namespace UE::LevelSnapshots::Private;
	SCOPED_SNAPSHOT_CORE_TRACE(IsRestorableProperty);
	
	// Deprecated and transient properties should not cause us to consider the property different because we do not save these properties.
	const uint64 UnsavedProperties = CPF_Deprecated | CPF_Transient;
	// Property is not editable in details panels
	const int64 UneditableFlags = CPF_DisableEditOnInstance;
	
	
	// Only consider editable properties
	const uint64 RequiredFlags = CPF_Edit;

	FLevelSnapshotsModule& Module = FLevelSnapshotsModule::GetInternalModuleInstance();
	const bool bIsExplicitlyIncluded = Module.IsPropertyExplicitlySupported(LeafProperty);
	const bool bIsExplicitlyExcluded = Module.IsPropertyExplicitlyUnsupported(LeafProperty);
	const bool bPassesDefaultChecks =
		!LeafProperty->HasAnyPropertyFlags(UnsavedProperties | UneditableFlags)
        && LeafProperty->HasAllPropertyFlags(RequiredFlags);
	
	return bIsExplicitlyIncluded || (!bIsExplicitlyExcluded && bPassesDefaultChecks);
}
