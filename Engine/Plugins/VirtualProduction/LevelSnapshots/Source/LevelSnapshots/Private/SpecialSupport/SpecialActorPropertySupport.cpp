// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpecialActorPropertySupport.h"

#include "LevelSnapshotsModule.h"
#include "Params/PropertyComparisonParams.h"
#include "Restorability/Interfaces/IPropertyComparer.h"

namespace UE::LevelSnapshots::Private::SpecialActorPropertySupport
{
	class FSpecialActorPropertySupport : public IPropertyComparer
	{
		const FProperty* Property_bIsSpatiallyLoaded = AActor::StaticClass()->FindPropertyByName(AActor::GetIsSpatiallyLoadedPropertyName());
	public:

		FSpecialActorPropertySupport()
		{
			check(Property_bIsSpatiallyLoaded);
		}
		
		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override
		{
			if (Params.LeafProperty == Property_bIsSpatiallyLoaded)
			{
				return Params.WorldActor->CanChangeIsSpatiallyLoadedFlag()
					? EPropertyComparison::CheckNormally
					: EPropertyComparison::TreatEqual;
			}
			return EPropertyComparison::CheckNormally;
		}
	};
	
	void Register(FLevelSnapshotsModule& Module)
	{
		const TSharedRef<FSpecialActorPropertySupport> SpecialActorPropertySupport = MakeShared<FSpecialActorPropertySupport>();
		Module.RegisterPropertyComparer(AActor::StaticClass(), SpecialActorPropertySupport);
	}
}
