// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsModule.h"

namespace UE::LevelSnapshots::Private::SpecialActorPropertySupport
{
	void Register(FLevelSnapshotsModule& Module);
}



