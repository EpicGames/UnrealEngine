// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/PropertySelectionMap.h"
#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Util/SnapshotTestRunner.h"
#include "Types/SnapshotTestActor.h"

#include "Components/PointLightComponent.h"
#include "Engine/PointLight.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Misc/AutomationTest.h"

namespace UE::LevelSnapshots::Private::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreSimpleProperties, "VirtualProduction.LevelSnapshots.Snapshot.RestoreSimpleProperties", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreSimpleProperties::RunTest(const FString& Parameters)
	{
		const FVector StartLocation(100.f, -200.f, 300.f);
		const FVector EndLocation(500.f, -625.f, 750.f);
		const FRotator StartRotation(20.f, 30.f, -30.f);
		const FRotator EndRotation(50.f, -60.f, -40.f);
		const FVector StartScale(1.f, 1.f, 2.f);
		const FVector EndScale(2.f, 3.f, -2.f);

		const float StartRadius = 200.f;
		const float EndRadius = 500.f;
		const FColor StartColor = FColor::Red;
		const FColor EndColor = FColor::Blue;
		const uint32 bStartCastShadows = true;
		const uint32 bEndCastShaodws = false;
	
		AStaticMeshActor* StaticMesh	= nullptr;
		APointLight* PointLight			= nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				StaticMesh = World->SpawnActor<AStaticMeshActor>(StartLocation, StartRotation);
				StaticMesh->SetActorScale3D(StartScale);

				PointLight = World->SpawnActor<APointLight>(StartLocation, StartRotation);
				PointLight->PointLightComponent->AttenuationRadius = StartRadius;
				PointLight->PointLightComponent->LightColor = StartColor;
				PointLight->PointLightComponent->CastShadows = bStartCastShadows;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				StaticMesh->SetActorLocationAndRotation(EndLocation, EndRotation);
				StaticMesh->SetActorScale3D(EndScale);

				PointLight->PointLightComponent->AttenuationRadius = EndRadius;
				PointLight->PointLightComponent->LightColor = EndColor;
				PointLight->PointLightComponent->CastShadows = bEndCastShaodws;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("Static Mesh Location"), StaticMesh->GetActorLocation(), StartLocation);
				TestEqual(TEXT("Static Mesh Rotation"), StaticMesh->GetActorRotation(), StartRotation);
				TestEqual(TEXT("Static Mesh Scale"), StaticMesh->GetActorScale3D(), StartScale);

				TestEqual(TEXT("Point Light Radius"), PointLight->PointLightComponent->AttenuationRadius, StartRadius);
				TestEqual(TEXT("Point Light Colour"), PointLight->PointLightComponent->LightColor, StartColor);
				TestEqual(TEXT("Point Light Cast Shadows"), PointLight->PointLightComponent->CastShadows, bStartCastShadows);
			});
	
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreReferenceProperties, "VirtualProduction.LevelSnapshots.Snapshot.RestoreReferenceProperties", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreReferenceProperties::RunTest(const FString& Parameters)
	{
		for (TFieldIterator<FProperty> FieldIt(ASnapshotTestActor::StaticClass()); FieldIt; ++FieldIt)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(*FieldIt))
			{
				FSoftObjectPath ClassPath(StructProperty->Struct);
				FString AsString = ClassPath.ToString();
				UE_LOGF(LogTemp, Log, "%ls", *AsString);
			}
		}
		ASnapshotTestActor* FirstActor = nullptr;
		ASnapshotTestActor* SecondActor = nullptr;
		ASnapshotTestActor* PointToSelfActor = nullptr;
		ASnapshotTestActor* FromWorldToExternal = nullptr;
		ASnapshotTestActor* FromExternalToWorld = nullptr;
		ASnapshotTestActor* MaterialAndMesh = nullptr;

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				FirstActor = ASnapshotTestActor::Spawn(World, "FirstActor");
				SecondActor = ASnapshotTestActor::Spawn(World, "SecondActor");
				PointToSelfActor = ASnapshotTestActor::Spawn(World, "PointToSelfActor");
				FromWorldToExternal = ASnapshotTestActor::Spawn(World, "FromWorldToExternal");
				FromExternalToWorld = ASnapshotTestActor::Spawn(World, "FromExternalToWorld");
			
				MaterialAndMesh = ASnapshotTestActor::Spawn(World, "MaterialAndMesh");
			
				FirstActor->SetObjectReference(SecondActor);
				SecondActor->SetObjectReference(FirstActor);
				PointToSelfActor->SetObjectReference(PointToSelfActor);
				FromWorldToExternal->SetObjectReference(FirstActor);
				FromExternalToWorld->SetObjectReference(FromExternalToWorld->CubeMesh);
			
				MaterialAndMesh->InstancedMeshComponent->SetStaticMesh(MaterialAndMesh->CubeMesh);
				MaterialAndMesh->InstancedMeshComponent->SetMaterial(0, MaterialAndMesh->GradientLinearMaterial);
			})
			.TakeSnapshot()

			.ModifyWorld([&](UWorld* World)
			{
				FirstActor->ClearObjectReferences();
				// No change to SecondActor
				PointToSelfActor->SetObjectReference(FirstActor);
				FromWorldToExternal->SetObjectReference(FromWorldToExternal->CubeMesh);
				FromExternalToWorld->SetObjectReference(FirstActor);
			
				MaterialAndMesh->InstancedMeshComponent->SetStaticMesh(MaterialAndMesh->CylinderMesh);
				MaterialAndMesh->InstancedMeshComponent->SetMaterial(0, MaterialAndMesh->GradientLinearMaterial);
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestTrue(TEXT("Cyclic Reference Restored"), FirstActor->HasObjectReference(SecondActor));
				TestTrue(TEXT("Reference Not Lost"), SecondActor->HasObjectReference(FirstActor));
				TestTrue(TEXT("Restore Pointing To Self"), PointToSelfActor->HasObjectReference(PointToSelfActor));
				TestTrue(TEXT("World > External > World"), FromWorldToExternal->HasObjectReference(FirstActor));
				TestTrue(TEXT("External > World > External"), FromExternalToWorld->HasObjectReference(FromExternalToWorld->CubeMesh));

				TestEqual(TEXT("Mesh"), MaterialAndMesh->InstancedMeshComponent->GetStaticMesh().Get(), ToRawPtr(MaterialAndMesh->CubeMesh));
				TestEqual(TEXT("Material"), MaterialAndMesh->InstancedMeshComponent->GetMaterial(0), ToRawPtr(MaterialAndMesh->GradientLinearMaterial));
			
			});

		return true;
	}

	/**
	* Things like changing array / set / map element order, such as { A, null } to { null, A }, is detected and restored correctly.
	* Differs from FReorderSubobjectCollections because different code runs for subobjects.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FReorderReferenceCollections, "VirtualProduction.LevelSnapshots.Snapshot.ReorderReferenceCollections", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FReorderReferenceCollections::RunTest(const FString& Parameters)
	{
		// TODO: 
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInstancedStaticMesh, "VirtualProduction.LevelSnapshots.Snapshot.InstancedStaticMesh", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FInstancedStaticMesh::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* MaterialAndMesh = nullptr;

		const FTransform StartFirstTransform(FRotator(100.f).Quaternion(), FVector(110.f) , FVector(1.f));
		const FTransform EndFirstTransform(FRotator(200.f).Quaternion(), FVector(210.f) , FVector(2.f));
		const FTransform StartSecondTransform(FRotator(100.f).Quaternion(), FVector(-1100.f) , FVector(1.f));
		const FTransform NewThirdTransform(FRotator(-300.f).Quaternion(), FVector(-310.f) , FVector(30.f));
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				MaterialAndMesh = ASnapshotTestActor::Spawn(World);
			
				MaterialAndMesh->InstancedMeshComponent->AddInstance(StartFirstTransform);
				MaterialAndMesh->InstancedMeshComponent->AddInstance(StartSecondTransform);
			})
			.TakeSnapshot()

			.ModifyWorld([&](UWorld* World)
			{
				MaterialAndMesh->InstancedMeshComponent->UpdateInstanceTransform(0, EndFirstTransform);
				MaterialAndMesh->InstancedMeshComponent->AddInstance(NewThirdTransform);
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("Instance Count"), MaterialAndMesh->InstancedMeshComponent->GetInstanceCount(), 2);

				FTransform ActualFirstTransform, ActualSecondTransform;
				MaterialAndMesh->InstancedMeshComponent->GetInstanceTransform(0, ActualFirstTransform);
				MaterialAndMesh->InstancedMeshComponent->GetInstanceTransform(1, ActualSecondTransform);

				// TODO: Investigate rotation not working correcty... it is some kind of math problem that causes angle to flip, i.e. 80 and 100 represent the same angle but are obviously not equal floats...
				TestTrue(TEXT("Instance 1"), ActualFirstTransform.GetLocation().Equals(StartFirstTransform.GetLocation()) && ActualFirstTransform.GetScale3D().Equals(StartFirstTransform.GetScale3D()));
				TestTrue(TEXT("Instance 2"), ActualSecondTransform.GetLocation().Equals(StartSecondTransform.GetLocation())&& ActualSecondTransform.GetScale3D().Equals(StartSecondTransform.GetScale3D()));
			});

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSkipTransientAndDeprecatedProperties, "VirtualProduction.LevelSnapshots.Snapshot.SkipTransientAndDeprecatedProperties", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FSkipTransientAndDeprecatedProperties::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* TestActor = nullptr;

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				TestActor = ASnapshotTestActor::Spawn(World);
			})
			.TakeSnapshot()

			.ModifyWorld([&](UWorld* World)
			{
				TestActor->DeprecatedProperty_DEPRECATED = 500;
				TestActor->TransientProperty = 500;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("Skip Deprecated Property"), TestActor->DeprecatedProperty_DEPRECATED, 500);
				TestEqual(TEXT("Skip Transient Property"), TestActor->TransientProperty, 500);
			});

		return true;
	}

	/**
	*
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnchangedActorHasNoDiff, "VirtualProduction.LevelSnapshots.Snapshot.UnchangedActorHasNoDiff", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FUnchangedActorHasNoDiff::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World);
				Actor->AllocateSubobjects();
			})
			.TakeSnapshot()
			.FilterProperties(Actor, [&](const FPropertySelectionMap& PropertySelection)
			{
				TestEqual(TEXT("Actor stays unchanged"), PropertySelection.GetKeyCount(), 0);
			});

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FActorAttachParentRestores, "VirtualProduction.LevelSnapshots.Snapshot.ActorAttachParentRestores", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FActorAttachParentRestores::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* ParentOne = nullptr;
		ASnapshotTestActor* ParentTwo = nullptr;
		ASnapshotTestActor* Child = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ParentOne = ASnapshotTestActor::Spawn(World, "ParentOne");
				ParentTwo = ASnapshotTestActor::Spawn(World, "ParentTwo");
				Child = ASnapshotTestActor::Spawn(World, "Child");

				Child->AttachToActor(ParentOne, FAttachmentTransformRules::KeepRelativeTransform);
			})
			.TakeSnapshot()

			.ModifyWorld([&](UWorld* World)
			{
				Child->AttachToActor(ParentTwo, FAttachmentTransformRules::KeepRelativeTransform);
			})
			.FilterProperties(Child, [&](const FPropertySelectionMap& PropertySelection)
			{
				const FPropertySelection* ComponentSelection = PropertySelection.GetObjectSelection(Child->GetRootComponent()).GetPropertySelection();

				const bool bRootComponentSelected = ComponentSelection != nullptr;
				TestTrue(TEXT("Root Component has changed properties"), bRootComponentSelected);
				TestEqual(TEXT("No other objects changed"), PropertySelection.GetKeyCount(), 1);
				if (bRootComponentSelected)
				{
					// 13 = AttachParent + Relative Loc & Rot & Scale + X & Y & Z of each of Loc & Rot & Scale
					// See FAttachParentShowsTransformProperties for more info
					TestEqual(TEXT("No other properties changed"), ComponentSelection->GetSelectedProperties().Num(), 13);

					const FProperty* AttachParentProperty = USceneComponent::StaticClass()->FindPropertyByName(FName("AttachParent"));
					const FProperty* RelativeLocation = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName());
					const FProperty* RelativeRotation = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeRotationPropertyName());
					const FProperty* RelativeScale = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName());
					ensureMsgf(AttachParentProperty && RelativeLocation && RelativeRotation && RelativeScale, TEXT("Did property name change?"));
					TestTrue(TEXT("AttachParent changed"), ComponentSelection->IsPropertySelected(nullptr, AttachParentProperty));
					TestTrue(TEXT("RelativeLocation changed"), ComponentSelection->IsPropertySelected(nullptr, RelativeLocation));
					TestTrue(TEXT("RelativeRotation changed"), ComponentSelection->IsPropertySelected(nullptr, RelativeRotation));
					TestTrue(TEXT("RelativeScale changed"), ComponentSelection->IsPropertySelected(nullptr, RelativeScale));
				}
			})

			.AccessSnapshot([&](ULevelSnapshot* Snapshot)
			{
				const TOptional<TNonNullPtr<AActor>> SnapshotChild = Snapshot->GetDeserializedActor(Child);
				const TOptional<TNonNullPtr<AActor>> SnapshotParentOne = Snapshot->GetDeserializedActor(ParentOne);

				const bool bParentsWereDeserialized = SnapshotChild && SnapshotParentOne;
				TestTrue(TEXT("Parents were deserialized"), bParentsWereDeserialized);
				if (bParentsWereDeserialized)
				{
					const bool bHasOriginalChanged = UE::LevelSnapshots::Private::HasOriginalChangedPropertiesSinceSnapshotWasTaken(Snapshot, SnapshotChild.GetValue(), Child);
					TestTrue(TEXT("HasOriginalChangedPropertiesSinceSnapshotWasTaken == true"), bHasOriginalChanged);
				
					TestTrue(TEXT("Snapshot has correct attach parent"), SnapshotChild.GetValue()->GetAttachParentActor() == SnapshotParentOne.GetValue());
				}
			})

			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("Attach parent restored correctly"), Child->GetAttachParentActor() == ParentOne);
			});

		return true;
	}
}