// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/LevelSnapshot.h"
#include "Filtering/PropertySelectionMap.h"
#include "Util/SnapshotTestRunner.h"
#include "Types/SnapshotTestActor.h"
#include "Types/LevelSnapshotsTestAssetUserData.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"

namespace UE::LevelSnapshots::Private::Tests
{
	/** Shared logic for testing restoration of an IInterface_AssetUserData. */
	class FAssetUserDataTestCase
	{
		FString BaseName;
		FAutomationTestBase& Test;

		FString MakeTestName(const TCHAR* TestName) const
		{
			return FString::Printf(TEXT("%s - %s"), *BaseName, TestName);
		}
		
	public:

		FAssetUserDataTestCase(FString BaseName, FAutomationTestBase& Test)
			: BaseName(MoveTemp(BaseName))
			, Test(Test)
		{}
		virtual ~FAssetUserDataTestCase() = default;
		
		virtual TScriptInterface<IInterface_AssetUserData> SpawnAssetUserDataStorerObject(UWorld* World) = 0;
		virtual TScriptInterface<IInterface_AssetUserData> GetAsserUserDataStorerObjectInSnapshotWorld(ULevelSnapshot* Snapshot, const TScriptInterface<IInterface_AssetUserData>& MainWorldObject) = 0;

		void RunTest()
		{
			ULevelSnapshotsTestAssetUserData_Persistent* Data_Persistent;
			ULevelSnapshotsTestAssetUserData_MarkedTransient* Data_MarkedTransient;
			ULevelSnapshotsTestAssetUserData_TransientPackage* Data_TransientPackage;
			
			ULevelSnapshotsTestAssetUserData_MarkedTransient* Data_MarkedTransient_Snapshot;
			ULevelSnapshotsTestAssetUserData_TransientPackage* Data_TransientPackage_Snapshot;
			
			TScriptInterface<IInterface_AssetUserData> DataStorage;
			auto GetActorFromDataStorage = [&DataStorage]()
			{
				return DataStorage.GetObject()->IsA<AActor>()
					? Cast<AActor>(DataStorage.GetObject())
					: DataStorage.GetObject()->GetTypedOuter<AActor>();
			};

			// Test must take place in a non-transient world otherwise ULevelSnapshotsTestAssetUserData_Persistent would be removed during tests since it would be in a transient package.
			FSnapshotTestRunner(ETestFlags::NonTransientWorld)
				.ModifyWorld([&](UWorld* World)
				{
					DataStorage = SpawnAssetUserDataStorerObject(World);
					
					Data_Persistent = NewObject<ULevelSnapshotsTestAssetUserData_Persistent>(DataStorage.GetObject());
					Data_MarkedTransient = NewObject<ULevelSnapshotsTestAssetUserData_MarkedTransient>(DataStorage.GetObject(), ULevelSnapshotsTestAssetUserData_MarkedTransient::StaticClass(), NAME_None, RF_Transient);
					Data_TransientPackage = NewObject<ULevelSnapshotsTestAssetUserData_TransientPackage>(GetTransientPackage());

					Data_Persistent->Value = 10;

					// Intentionally put the persistent data inbetween the two transient ones that will be removed to test more positioning cases.
					DataStorage->AddAssetUserData(Data_MarkedTransient);
					DataStorage->AddAssetUserData(Data_Persistent);
					DataStorage->AddAssetUserData(Data_TransientPackage);
				})

				// After taking a snapshot the asset user data is still there
				.TakeSnapshot()
				.ModifyWorld([&](UWorld* World)
				{
					ULevelSnapshotsTestAssetUserData_Persistent* Retrieved_Persistent = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_Persistent>();
					ULevelSnapshotsTestAssetUserData_MarkedTransient* Retrieved_MarkedTransient = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_MarkedTransient>();
					ULevelSnapshotsTestAssetUserData_TransientPackage* Retrieved_TransientPackage = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_TransientPackage>();
					
					Test.TestTrue(MakeTestName(TEXT("Persistent - After taking snapshot")), Retrieved_Persistent == Data_Persistent);
					Test.TestTrue(MakeTestName(TEXT("Marked Transient - After taking snapshot")), Retrieved_MarkedTransient == Data_MarkedTransient);
					Test.TestTrue(MakeTestName(TEXT("Transient Package - After taking snapshot")), Retrieved_TransientPackage == Data_TransientPackage);
					// Not implemented by UActorComponent...
					if (DataStorage->GetAssetUserDataArray())
					{
						Test.TestEqual(MakeTestName(TEXT("No null entries - After taking snapshot")), DataStorage->GetAssetUserDataArray()->Num(), 3);
					}
				})
			
				// After filtering a snapshot the asset user data is still there
				.FilterProperties(GetActorFromDataStorage(), [&](const FPropertySelectionMap& SelectionMap)
				{
					ULevelSnapshotsTestAssetUserData_Persistent* Retrieved_Persistent = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_Persistent>();
					ULevelSnapshotsTestAssetUserData_MarkedTransient* Retrieved_MarkedTransient = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_MarkedTransient>();
					ULevelSnapshotsTestAssetUserData_TransientPackage* Retrieved_TransientPackage = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_TransientPackage>();
					
					Test.TestTrue(MakeTestName(TEXT("Persistent - After filtering")), Retrieved_Persistent == Data_Persistent);
					Test.TestTrue(MakeTestName(TEXT("Marked Transient - After filtering")), Retrieved_MarkedTransient == Data_MarkedTransient);
					Test.TestTrue(MakeTestName(TEXT("Transient Package - After filtering")), Retrieved_TransientPackage == Data_TransientPackage);
					// Not implemented by UActorComponent...
					if (DataStorage->GetAssetUserDataArray())
					{
						Test.TestEqual(MakeTestName(TEXT("No null entries - After filtering")), DataStorage->GetAssetUserDataArray()->Num(), 3);
					}

					const TArray<FSoftObjectPath> ModifiedObjects = SelectionMap.GetKeys();
					Test.TestEqual(MakeTestName(TEXT("No changes detected")), ModifiedObjects.Num(), 0);
				})
			
				// After applying a snapshot without diff, the data is still there
				.ApplySnapshot()
				.RunTest([&]()
				{
					ULevelSnapshotsTestAssetUserData_Persistent* Retrieved_Persistent = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_Persistent>();
					ULevelSnapshotsTestAssetUserData_MarkedTransient* Retrieved_MarkedTransient = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_MarkedTransient>();
					ULevelSnapshotsTestAssetUserData_TransientPackage* Retrieved_TransientPackage = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_TransientPackage>();
					
					Test.TestTrue(MakeTestName(TEXT("Persistent - After applying")), Retrieved_Persistent == Data_Persistent);
					Test.TestTrue(MakeTestName(TEXT("Marked Transient - After applying")), Retrieved_MarkedTransient == Data_MarkedTransient);
					Test.TestTrue(MakeTestName(TEXT("Transient Package - After applying")), Retrieved_TransientPackage == Data_TransientPackage);
					// Not implemented by UActorComponent...
					if (DataStorage->GetAssetUserDataArray())
					{
						Test.TestEqual(MakeTestName(TEXT("No null entries - After applying")), DataStorage->GetAssetUserDataArray()->Num(), 3);
					}
				})

				// Removing the transient data will not restore it nor add null entries ...
				.ModifyWorld([&](UWorld* World)
				{
					DataStorage->RemoveUserDataOfClass(ULevelSnapshotsTestAssetUserData_MarkedTransient::StaticClass());
					DataStorage->RemoveUserDataOfClass(ULevelSnapshotsTestAssetUserData_TransientPackage::StaticClass());
				})
				.FilterProperties(GetActorFromDataStorage(), [&](const FPropertySelectionMap& SelectionMap)
				{
					const TArray<FSoftObjectPath> ModifiedObjects = SelectionMap.GetKeys();
					Test.TestEqual(MakeTestName(TEXT("Transient objects should not be added back")), ModifiedObjects.Num(), 0);
				})
				.ApplySnapshot()
				.RunTest([&]()
				{
					ULevelSnapshotsTestAssetUserData_Persistent* Retrieved_Persistent = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_Persistent>();
					Test.TestTrue(MakeTestName(TEXT("Persistent - Still exists after removing transient data")), Retrieved_Persistent == Data_Persistent);
					// Not implemented by UActorComponent...
					if (DataStorage->GetAssetUserDataArray())
					{
						Test.TestEqual(MakeTestName(TEXT("Persistent - No null entries were added after removing transient data")), DataStorage->GetAssetUserDataArray()->Num(), 1);
					}
				})
			
				// ... and adding transient data will not prompt to remove it
				.TakeSnapshot(TEXT("WithoutTransientData"))
				.FilterProperties(GetActorFromDataStorage(), [&](const FPropertySelectionMap& SelectionMap)
				{
					const TArray<FSoftObjectPath> ModifiedObjects = SelectionMap.GetKeys();
					Test.TestEqual(MakeTestName(TEXT("Transient objects should not be added back")), ModifiedObjects.Num(), 0);
				}, nullptr, TEXT("WithoutTransientData"))
				.ApplySnapshot(nullptr, TEXT("WithoutTransientData"))
				.RunTest([&]()
				{
					ULevelSnapshotsTestAssetUserData_Persistent* Retrieved_Persistent = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_Persistent>();
					Test.TestTrue(MakeTestName(TEXT("Persistent - Still exists after removing transient data")), Retrieved_Persistent == Data_Persistent);
					// Not implemented by UActorComponent...
					if (DataStorage->GetAssetUserDataArray())
					{
						Test.TestEqual(MakeTestName(TEXT("Persistent - No null entries were added after removing transient data")), DataStorage->GetAssetUserDataArray()->Num(), 1);
					}
				})
			
				// Manually add back the transient data to perform more tests
				.ModifyWorld([&](UWorld* World)
				{
					DataStorage->AddAssetUserData(Data_MarkedTransient);
					DataStorage->AddAssetUserData(Data_TransientPackage);
				})

				// Simulate some engine system adding the transient object to the snapshot version...
				.AccessSnapshot([&](ULevelSnapshot* Snapshot)
				{
					const TScriptInterface<IInterface_AssetUserData> SnapshotObject = GetAsserUserDataStorerObjectInSnapshotWorld(Snapshot, DataStorage);
					Data_MarkedTransient_Snapshot = NewObject<ULevelSnapshotsTestAssetUserData_MarkedTransient>(SnapshotObject.GetObject(), ULevelSnapshotsTestAssetUserData_MarkedTransient::StaticClass(), NAME_None, RF_Transient);
					Data_TransientPackage_Snapshot = NewObject<ULevelSnapshotsTestAssetUserData_TransientPackage>(GetTransientPackage());
					SnapshotObject->AddAssetUserData(Data_MarkedTransient_Snapshot);
					SnapshotObject->AddAssetUserData(Data_TransientPackage_Snapshot);
				})
				.FilterProperties(GetActorFromDataStorage(), [&](const FPropertySelectionMap& SelectionMap)
				{
					const TArray<FSoftObjectPath> ModifiedObjects = SelectionMap.GetKeys();
					Test.TestEqual(MakeTestName(TEXT("Transient objects in snapshot world are ignored")), ModifiedObjects.Num(), 0);
				})
			
				// Persistent asset user data is restored and transient ones are left untouched
				.ModifyWorld([&](UWorld* World)
				{
					DataStorage->RemoveUserDataOfClass(ULevelSnapshotsTestAssetUserData_Persistent::StaticClass());
				})
				.ApplySnapshot()
				.RunTest([&]()
				{
					ULevelSnapshotsTestAssetUserData_Persistent* Retrieved_Persistent = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_Persistent>();
					ULevelSnapshotsTestAssetUserData_MarkedTransient* Retrieved_MarkedTransient = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_MarkedTransient>();
					ULevelSnapshotsTestAssetUserData_TransientPackage* Retrieved_TransientPackage = DataStorage->GetAssetUserData<ULevelSnapshotsTestAssetUserData_TransientPackage>();
					
					Test.TestTrue(MakeTestName(TEXT("Persistent - After removing and applying")), Retrieved_Persistent != nullptr);
					if (Retrieved_Persistent)
					{
						Test.TestEqual(MakeTestName(TEXT("Persistent - Correct value restored")), Retrieved_Persistent->Value, 10);
					}
					
					Test.TestTrue(MakeTestName(TEXT("Marked Transient - After removing and applying")), Retrieved_MarkedTransient == Data_MarkedTransient);
					Test.TestTrue(MakeTestName(TEXT("Transient Package - After removing and applying")), Retrieved_TransientPackage == Data_TransientPackage);
					// Not implemented by UActorComponent...
					if (DataStorage->GetAssetUserDataArray())
					{
						Test.TestEqual(MakeTestName(TEXT("No null entries - After removing and applying")), DataStorage->GetAssetUserDataArray()->Num(), 3);
					}
				})
			;
		}
	};

	/** Stores asset in an actor's component. */
	class FAssetUserDataTestCase_InComponent : public FAssetUserDataTestCase
	{
	public:

		using FAssetUserDataTestCase::FAssetUserDataTestCase;

		virtual TScriptInterface<IInterface_AssetUserData> SpawnAssetUserDataStorerObject(UWorld* World) override
		{
			return ASnapshotTestActor::Spawn(World)->TestComponent;
		}

		virtual TScriptInterface<IInterface_AssetUserData> GetAsserUserDataStorerObjectInSnapshotWorld(ULevelSnapshot* Snapshot, const TScriptInterface<IInterface_AssetUserData>& MainWorldObject)
		{
			UActorComponent* Component = Cast<UActorComponent>(MainWorldObject.GetObject());
			TOptional<TNonNullPtr<AActor>> SnapshotActor = Snapshot->GetDeserializedActor(Component->GetOwner());
			return SnapshotActor ? Cast<ASnapshotTestActor>(SnapshotActor->Get())->TestComponent : nullptr;
		}
	};
	
	/**
	 * This tests that UActorComponent::AssetUserData is restored correctly. 
	 * Some engine systems add transient objects to the AssetUserData. Under normal LS conditions, such objects would be nulled when restoring.
	 * 
	 * Level Snapshots has no general system for handling transient instanced objects in UPROPERTIES but there are special rules for AssetUserData:
	 * - If an AssetUserData object is transient and we're taking a snapshot, the object is removed before and added back after taking the snapshot.
	 * - If an AssetUserData object is transient and we're diffing the world, the object is removed before and added back after diffing the snapshot.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetUserDataTests, "VirtualProduction.LevelSnapshots.Snapshot.Other.AssetUserData", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FAssetUserDataTests::RunTest(const FString& Parameters)
	{
		FAssetUserDataTestCase_InComponent ComponentTest(TEXT("ComponentTest"), *this);
		// Adding a test for world settings was considered but AWorldSettings::AssetUserData property is not CPF_Editable so Level Snapshots does not restore it

		ComponentTest.RunTest();
		
		return true;
	}
}
