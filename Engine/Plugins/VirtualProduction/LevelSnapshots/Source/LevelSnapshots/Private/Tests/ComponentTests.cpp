// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/PropertySelection.h"

#include "Filtering/Diffing/EquivalenceUtil.h"
#include "Filtering/PropertySelectionMap.h"
#include "Util/SnapshotTestRunner.h"
#include "Types/SnapshotTestActor.h"

#include "Components/PointLightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "EngineUtils.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/AutomationTest.h"

namespace UE::LevelSnapshots::Private::Tests
{
	/**
	* Checks that instanced and native components are diffed correctly:
	*  - Added components are marked to be removed
	*  - Removed components are marked to be added
	*
	*  Additionally, this indirectly tests that components are recreated correctly on the snapshot actor:
	*  if components were not recreated correctly on the snapshot actor, the diff would not generate correctly in the first place.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAddedAndRemovedComponentsCorrectOnSnapshotActorAndGenerateDiff, "VirtualProduction.LevelSnapshots.Snapshot.Component.AddedAndRemovedComponentsCorrectOnSnapshotActorAndGenerateDiff", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FAddedAndRemovedComponentsCorrectOnSnapshotActorAndGenerateDiff::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
		UStaticMeshComponent* InstancedStaysOnActor = nullptr;
		UStaticMeshComponent* InstancedRemovedFromActor = nullptr;
		UStaticMeshComponent* InstancedAddedToActor = nullptr;
		UStaticMeshComponent* NativeRemovedFromActor = nullptr;
		UStaticMeshComponent* NativedAddedWithoutPropertyReference = nullptr;
		UStaticMeshComponent* NativedAddedWithPropertyReference = nullptr;

		// Generate properties
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World);;

				InstancedStaysOnActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedStaysOnActor");
				Actor->AddInstanceComponent(InstancedStaysOnActor);
				InstancedRemovedFromActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedRemovedFromActor");
				Actor->AddInstanceComponent(InstancedRemovedFromActor);
				NativeRemovedFromActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "NativeRemovedFromActor");

				InstancedStaysOnActor->SetRelativeLocation(FVector(10,20,30));
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Actor->RemoveOwnedComponent(InstancedRemovedFromActor);
				Actor->RemoveOwnedComponent(NativeRemovedFromActor);

				InstancedAddedToActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedAddedToActor");
				Actor->AddInstanceComponent(InstancedAddedToActor);
				NativedAddedWithoutPropertyReference = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "NativedAddedWithoutPropertyReference");

				InstancedStaysOnActor->SetRelativeLocation(FVector(100,200,300));
			})
	
			.FilterProperties(Actor, [&](const FPropertySelectionMap& PropertySelectionMap)
			{
				const UE::LevelSnapshots::FRestorableObjectSelection ObjectSelection = PropertySelectionMap.GetObjectSelection(Actor);
			
				const UE::LevelSnapshots::FAddedAndRemovedComponentInfo* Selection = ObjectSelection.GetComponentSelection();
				TestTrue(TEXT("Has Component Diff"), Selection != nullptr);
				if (!Selection)
				{
					return;
				}

				TestTrue(TEXT("Added instanced component should be removed"), Selection->EditorWorldComponentsToRemove.Contains(InstancedAddedToActor));
				TestEqual(TEXT("Num components to remove"), Selection->EditorWorldComponentsToRemove.Num(), 1);

				// Native components are only tracked if they have a component reference; see FComponentEditorUtils::CanEditComponentInstance
				TestTrue(TEXT("Added native component without property references is ignored"), !Selection->EditorWorldComponentsToRemove.Contains(NativedAddedWithoutPropertyReference));
			
				// NativeRemovedFromActor has no component reference so it is skipped; see FComponentEditorUtils::CanEditComponentInstance.
				TestEqual(TEXT("Num components to add back"), Selection->SnapshotComponentsToAdd.Num(), 1);
				TSet<FString> NamesToCheck { "InstancedRemovedFromActor" };
				for (auto AddBackIt = Selection->SnapshotComponentsToAdd.CreateConstIterator(); AddBackIt; ++AddBackIt)
				{
					TestTrue(TEXT("Component should be added back"), NamesToCheck.Contains(AddBackIt->Get()->GetName()));
				}

				TestTrue(TEXT("Remaining instanced component has property selection"), PropertySelectionMap.GetObjectSelection(InstancedStaysOnActor).GetPropertySelection() != nullptr);
			});

		// HasOriginalChangedPropertiesSinceSnapshotWasTaken return true
		AActor* ActorWithAddedComponent = nullptr;
		UActorComponent* AddedComponent = nullptr;
		AActor* ActorWithRemovedComponent = nullptr;
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ActorWithAddedComponent = ASnapshotTestActor::Spawn(World, "ActorWithAddedComponent");
				ActorWithRemovedComponent = ASnapshotTestActor::Spawn(World, "ActorWithRemovedComponent");

				AddedComponent = NewObject<UStaticMeshComponent>(ActorWithRemovedComponent, UStaticMeshComponent::StaticClass(), "AddedComponent");
				ActorWithRemovedComponent->AddInstanceComponent(AddedComponent);
			})
			.TakeSnapshot()

			.ModifyWorld([&](UWorld* World)
			{
				ActorWithRemovedComponent->RemoveOwnedComponent(AddedComponent);

				AddedComponent = NewObject<UStaticMeshComponent>(ActorWithAddedComponent, UStaticMeshComponent::StaticClass(), "AddedComponent");
				ActorWithAddedComponent->AddInstanceComponent(AddedComponent);
			})
			.AccessSnapshot([&](ULevelSnapshot* Snapshot)
			{
				const TOptional<TNonNullPtr<AActor>> SnapshotActorWithAddedComponent = Snapshot->GetDeserializedActor(ActorWithAddedComponent);
				const TOptional<TNonNullPtr<AActor>> SnapshotActorWithRemovedComponent = Snapshot->GetDeserializedActor(ActorWithRemovedComponent);
				if (!ensure(SnapshotActorWithAddedComponent.IsSet() && SnapshotActorWithRemovedComponent.IsSet()))
				{
					return;
				}

				TestTrue(TEXT("ActorWithAddedComponent: HasOriginalChangedPropertiesSinceSnapshotWasTaken"), UE::LevelSnapshots::Private::HasOriginalChangedPropertiesSinceSnapshotWasTaken(Snapshot, SnapshotActorWithAddedComponent.GetValue(), ActorWithAddedComponent));
				TestTrue(TEXT("ActorWithRemovedComponent: HasOriginalChangedPropertiesSinceSnapshotWasTaken"), UE::LevelSnapshots::Private::HasOriginalChangedPropertiesSinceSnapshotWasTaken(Snapshot, SnapshotActorWithRemovedComponent.GetValue(), ActorWithRemovedComponent));
			});
	
		return true;
	}

	/**
	* Given the below class, this tests that after applying to world MeshComponent points to the component in the world.
	* This used to be a bug: the reference used to point to the equivalent component in the snapshot world or a class default.
	* 
	* class AActor
	* {
	*		UPROPERTY()
	*		UStaticMeshComponent* MeshComponent;
	* }
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAreComponentReferenceValidAfterRestore, "VirtualProduction.LevelSnapshots.Snapshot.Component.AreComponentReferenceValidAfterRestore", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FAreComponentReferenceValidAfterRestore::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
	
		UInstancedStaticMeshComponent* OriginalMeshComp = nullptr;
		UPointLightComponent* OriginalLightComp = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World);;
			
				OriginalMeshComp = Actor->InstancedMeshComponent;
				OriginalLightComp = Actor->PointLightComponent;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Actor->PointLightComponent = nullptr;
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestEqual(TEXT("Component reference was not changed"), ToRawPtr(Actor->InstancedMeshComponent), OriginalMeshComp);
				TestEqual(TEXT("Component reference is correctly restored"), ToRawPtr(Actor->PointLightComponent), OriginalLightComp);
			});
	
		return true;
	}

	/**
	* Checks that component references as the one below do not show up in the selection set.
	* 
	* class AActor
	* {
	*		UPROPERTY()
	*		UStaticMeshComponent* MeshComponent;
	* }
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEquivalentObjectReferencesDoNotHaveDiff, "VirtualProduction.LevelSnapshots.Snapshot.Component.EquivalentObjectReferencesDoNotHaveDiff", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FEquivalentObjectReferencesDoNotHaveDiff::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World);;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				// Just here so we definitely detect changes on the actor
				Actor->IntProperty = 12345;
			})
			.FilterProperties(Actor, [&](const FPropertySelectionMap& PropertySelectionMap)
			{
				const FPropertySelection* SelectedSubobjectProperties = PropertySelectionMap.GetObjectSelection(Actor).GetPropertySelection();

				const FProperty* PointLightComponent = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, PointLightComponent));
				const FProperty* CubeMesh = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, CubeMesh));
			
				TestTrue(TEXT("Component does not have diff"), !SelectedSubobjectProperties->IsPropertySelected(nullptr, PointLightComponent));
				TestTrue(TEXT("Asset does not have diff"), !SelectedSubobjectProperties->IsPropertySelected(nullptr, CubeMesh));
			});

		return true;
	}

	/**
	* Tests that an instanced component is:
	* - recreated if it was removed
	* - serialized into if it still exists
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreInstancedComponent, "VirtualProduction.LevelSnapshots.Snapshot.Component.RestoreInstancedComponent", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreInstancedComponent::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
		UStaticMeshComponent* InstancedStaysOnActor = nullptr;
		UStaticMeshComponent* InstancedRemovedFromActor = nullptr; 
		UStaticMeshComponent* InstancedAddedToActor = nullptr; 
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World);;

				// This component will stay on the actor
				InstancedStaysOnActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedStaysOnActor");
				Actor->AddInstanceComponent(InstancedStaysOnActor);
			
				InstancedStaysOnActor->SetStaticMesh(Actor->CubeMesh);
				InstancedStaysOnActor->SetRelativeLocation(FVector(100, 200, 300));
				InstancedStaysOnActor->AttachToComponent(Actor->GetMesh(), FAttachmentTransformRules::KeepRelativeTransform);


				// This component will be recreated
				InstancedRemovedFromActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedRemovedFromActor");
				Actor->AddInstanceComponent(InstancedRemovedFromActor);

				InstancedRemovedFromActor->SetStaticMesh(Actor->CylinderMesh);
				InstancedRemovedFromActor->SetRelativeLocation(FVector(1, 2, 3));
				InstancedRemovedFromActor->AttachToComponent(Actor->GetMesh(), FAttachmentTransformRules::KeepRelativeTransform);
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				InstancedStaysOnActor->SetStaticMesh(nullptr);
				InstancedStaysOnActor->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
				InstancedStaysOnActor->SetRelativeLocation(FVector(1000, 2000, 3000));
			
				InstancedRemovedFromActor->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
				InstancedRemovedFromActor->SetRelativeLocation(FVector(10, 20, 30));
				InstancedRemovedFromActor->DestroyComponent();

				InstancedAddedToActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedAddedToActor");
				Actor->AddInstanceComponent(InstancedAddedToActor);
			})

			// Exists on snapshot?
			.AccessSnapshot([&](ULevelSnapshot* Snapshot)
			{
				const TOptional<TNonNullPtr<AActor>> CounterpartActor = Snapshot->GetDeserializedActor(Actor);
				if (!ensureAlways(CounterpartActor))
				{
					return;
				}

				UActorComponent* const* CounterpartComponent = CounterpartActor.GetValue()->GetInstanceComponents().FindByPredicate([](UActorComponent* Component)
				{
					return Component->GetFName() == "InstancedStaysOnActor";
				});
				TestTrue(TEXT("Instanced component in InstancedComponents array on snapshot counterpart"), CounterpartComponent != nullptr);
				if (!CounterpartComponent)
				{
					return;
				}

				USceneComponent* AsSceneComponent = Cast<USceneComponent>(*CounterpartComponent);
				ASnapshotTestActor* CastedCounterpartActor = Cast<ASnapshotTestActor>(CounterpartActor->Get());
				check(AsSceneComponent);
				TestTrue(TEXT("Snapshot attach parent correct"), AsSceneComponent->GetAttachParent() == CastedCounterpartActor->GetMesh());
			})

			// Restores correctly?
			.ApplySnapshot()
			.RunTest([&]()
			{
				// Instanced component that stayed
				TestEqual(TEXT("Instanced component static mesh was restored"), ToRawPtr(InstancedStaysOnActor->GetStaticMesh()), ToRawPtr(Actor->CubeMesh));
				TestEqual(TEXT("Instanced component relative location was restored"), InstancedStaysOnActor->GetRelativeLocation(), FVector(100, 200, 300));
				TestTrue(TEXT("Instanced component attach parent was restored"), InstancedStaysOnActor->GetAttachParent() == Actor->GetMesh());


				// Component was removed
				UActorComponent* const* RemovedComponent = Actor->GetInstanceComponents().FindByPredicate([&](UActorComponent* Component)
				{
					return Component->GetFName() == "InstancedAddedToActor";
				});
				TestTrue(TEXT("Instanced component was removed from actor"), RemovedComponent == nullptr);
				TestTrue(TEXT("Removed component is marked from destroy"), !IsValid(InstancedAddedToActor));

				// Component was added back
				UActorComponent* const* PossibleRecreatedComponent = Actor->GetInstanceComponents().FindByPredicate([&](UActorComponent* Component)
				{
					return Component->GetFName() == "InstancedRemovedFromActor";
				});
				if (!PossibleRecreatedComponent)
				{
					AddError(TEXT("Removed instanced component was not added back."));
					return;
				}
				UStaticMeshComponent* RecreatedComponent = Cast<UStaticMeshComponent>(*PossibleRecreatedComponent);
				TestEqual(TEXT("Recreated component has correct mesh"), ToRawPtr(RecreatedComponent->GetStaticMesh()), ToRawPtr(Actor->CylinderMesh));
				TestEqual(TEXT("Recreated component has correct relative location"), RecreatedComponent->GetRelativeLocation(), FVector(1, 2, 3));
				TestTrue(TEXT("Recreated component attach parent was restored"), RecreatedComponent->GetAttachParent() == Actor->GetMesh());
			});

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRecreatedActorHasAllComponents, "VirtualProduction.LevelSnapshots.Snapshot.Component.RecreatedActorHasAllComponents", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRecreatedActorHasAllComponents::RunTest(const FString& Parameters)
	{
		AActor* Actor = nullptr;
		UStaticMeshComponent* InstancedRemovedFromActor = nullptr;
		UStaticMeshComponent* NativeRemovedFromActor = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				FActorSpawnParameters SpawnParams;
				SpawnParams.Name = "TestActor";
				Actor = World->SpawnActor<AActor>(SpawnParams);

				InstancedRemovedFromActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "InstancedRemovedFromActor");
				Actor->AddInstanceComponent(InstancedRemovedFromActor);
				NativeRemovedFromActor = NewObject<UStaticMeshComponent>(Actor, UStaticMeshComponent::StaticClass(), "NativeRemovedFromActor");

				InstancedRemovedFromActor->SetRelativeLocation(FVector(10,20,30));
				NativeRemovedFromActor->SetRelativeLocation(FVector(11,22,33));
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				World->DestroyActor(Actor);
				Actor = nullptr;
			})
			.ApplySnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				for (TActorIterator<AActor> It(World); It; ++It)
				{
					if (It->GetName().Equals("TestActor"))
					{
						Actor = *It;
					}
				}

				if (!ensure(Actor))
				{
					return;
				}

				TInlineComponentArray<UActorComponent*> Components(Actor);
				UActorComponent** InstancedComp = Components.FindByPredicate([](UActorComponent* Component) { return Component->GetName().Equals("InstancedRemovedFromActor"); });
				UActorComponent** NativeComp = Components.FindByPredicate([](UActorComponent* Component) { return Component->GetName().Equals("NativeRemovedFromActor"); });

				TestTrue(TEXT("Instance Component was recreated"), InstancedComp != nullptr);
				TestTrue(TEXT("Native Component without property referenced was not recreated"), NativeComp == nullptr);
				if (!InstancedComp || !NativeComp)
				{
					return;
				}
			
				TestEqual(TEXT("Instance Component had properties restored"), Cast<UStaticMeshComponent>(*InstancedComp)->GetRelativeLocation(), FVector(10,20,30));
			});
	
		return true;
	}

	/**
	* Usually components have the owning actor as outer.
	* Some applications, such as USD, create components that have other components as outers. It may even happen that components have duplicate names (with different outers though).
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FComponentsHaveComponentsAsOuters, "VirtualProduction.LevelSnapshots.Snapshot.Component.ComponentsHaveComponentsAsOuters", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FComponentsHaveComponentsAsOuters::RunTest(const FString& Parameters)
	{
		struct Local
		{
			static FSnapshotTestRunner CreateDuplicateComponentNameTest(ASnapshotTestActor*& CorrectComponentOrder, ASnapshotTestActor*& OutOfOrder)
			{
				FSnapshotTestRunner Result;
				Result.ModifyWorld([&](UWorld* World)
				{
					CorrectComponentOrder = ASnapshotTestActor::Spawn(World, "CorrectComponentOrder");
					{
						UStaticMeshComponent* ComponentWithActorAsOuter = NewObject<UStaticMeshComponent>(CorrectComponentOrder, UStaticMeshComponent::StaticClass(), "DuplicateName");
						CorrectComponentOrder->AddInstanceComponent(ComponentWithActorAsOuter);
						ComponentWithActorAsOuter->SetupAttachment(CorrectComponentOrder->GetRootComponent());
						ComponentWithActorAsOuter->SetRelativeLocation(FVector(10.f));

						UStaticMeshComponent* ComponentWithComponentAsOuter = NewObject<UStaticMeshComponent>(ComponentWithActorAsOuter, UStaticMeshComponent::StaticClass(), "DuplicateName");
						CorrectComponentOrder->AddInstanceComponent(ComponentWithComponentAsOuter);
						ComponentWithComponentAsOuter->SetupAttachment(ComponentWithActorAsOuter);
						ComponentWithComponentAsOuter->SetRelativeLocation(FVector(20.f));

						// Makes finding easier in test
						CorrectComponentOrder->ObjectArray.Add(ComponentWithActorAsOuter);
						CorrectComponentOrder->ObjectArray.Add(ComponentWithComponentAsOuter);
					}
				
					OutOfOrder = ASnapshotTestActor::Spawn(World, "OutOfOrder");
					{
						UStaticMeshComponent* ComponentWithActorAsOuter = NewObject<UStaticMeshComponent>(OutOfOrder, UStaticMeshComponent::StaticClass(), "DuplicateName");
						ComponentWithActorAsOuter->SetupAttachment(OutOfOrder->GetRootComponent());
						ComponentWithActorAsOuter->SetRelativeLocation(FVector(10.f));

						UStaticMeshComponent* ComponentWithComponentAsOuter = NewObject<UStaticMeshComponent>(ComponentWithActorAsOuter, UStaticMeshComponent::StaticClass(), "DuplicateName");
						ComponentWithComponentAsOuter->SetupAttachment(ComponentWithActorAsOuter);
						ComponentWithComponentAsOuter->SetRelativeLocation(FVector(20.f));
					
						// Makes finding easier in test
						OutOfOrder->ObjectArray.Add(ComponentWithActorAsOuter);
						OutOfOrder->ObjectArray.Add(ComponentWithComponentAsOuter);

						OutOfOrder->RemoveOwnedComponent(ComponentWithActorAsOuter);
						OutOfOrder->RemoveOwnedComponent(ComponentWithComponentAsOuter);

						OutOfOrder->AddOwnedComponent(ComponentWithComponentAsOuter);
						OutOfOrder->AddOwnedComponent(ComponentWithActorAsOuter);
						OutOfOrder->AddInstanceComponent(ComponentWithComponentAsOuter);
						OutOfOrder->AddInstanceComponent(ComponentWithActorAsOuter);
					}
				});

				return MoveTemp(Result);
			}

			static void TestRecreatedActor(UWorld* World, FAutomationTestBase& Test)
			{
				ASnapshotTestActor* Actor = FindObject<ASnapshotTestActor>(World->PersistentLevel, TEXT("CorrectComponentOrder"));
				if (Actor)
				{
					const bool bObjectArrayWasRestored = Actor && Actor->ObjectArray.Num() == 2 && Actor->ObjectArray[0] != nullptr && Actor->ObjectArray[1] != nullptr && Actor->ObjectArray[0] != Actor->ObjectArray[1];
					if (!bObjectArrayWasRestored)
					{
						Test.AddError(TEXT("ObjectArray was not restored"));
						return;
					}
				
					UStaticMeshComponent* ComponentWithActorAsOuter = Cast<UStaticMeshComponent>(Actor->ObjectArray[0]);
					UStaticMeshComponent* ComponentWithComponentAsOuter = Cast<UStaticMeshComponent>(Actor->ObjectArray[1]);
					const bool bComponentsAreValid = ComponentWithActorAsOuter && ComponentWithComponentAsOuter;
					if (!bComponentsAreValid)
					{
						Test.AddError(TEXT("Components were not recreated"));
						return;
					}

					Test.TestTrue(TEXT("Component names are equal"), ComponentWithActorAsOuter->GetFName() == ComponentWithComponentAsOuter->GetFName());
					Test.TestTrue(TEXT("Component are different instances"), ComponentWithActorAsOuter != ComponentWithComponentAsOuter);

					Test.TestTrue(TEXT("ComponentWithActorAsOuter->GetAttachParent() == Actor->GetRootComponent()"), ComponentWithActorAsOuter->GetAttachParent() == Actor->GetRootComponent());
					Test.TestTrue(TEXT("ComponentWithComponentAsOuter->GetAttachParent() == ComponentWithActorAsOuter"), ComponentWithComponentAsOuter->GetAttachParent() == ComponentWithActorAsOuter);

					Test.TestEqual(TEXT("ComponentWithActorAsOuter->RelativeLocation"), ComponentWithActorAsOuter->GetRelativeLocation(), FVector(10.f));
					Test.TestEqual(TEXT("ComponentWithActorAsOuter->RelativeLocation"), ComponentWithComponentAsOuter->GetRelativeLocation(), FVector(20.f));
				}
				else
				{
					Test.AddError(TEXT("Actor was not recreated"));
				}
			}
		};
	
		ASnapshotTestActor* CorrectComponentOrder = nullptr;
		ASnapshotTestActor* OutOfOrder = nullptr;
	
		Local::CreateDuplicateComponentNameTest(CorrectComponentOrder, OutOfOrder)
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				World->DestroyActor(CorrectComponentOrder);
				World->DestroyActor(OutOfOrder);
				CorrectComponentOrder = nullptr;
				OutOfOrder = nullptr;
			})
			.ApplySnapshot()
			.ModifyWorld([this](UWorld* World)
			{
				Local::TestRecreatedActor(World, *this);
				Local::TestRecreatedActor(World, *this);
			});
	
		return true;
	}
}