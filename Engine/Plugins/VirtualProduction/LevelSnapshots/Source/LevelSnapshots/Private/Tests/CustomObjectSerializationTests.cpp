// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/CustomSubobjectRestorationInfo.h"
#include "Filtering/PropertySelectionMap.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "LevelSnapshotsModule.h"
#include "Params/ObjectSnapshotSerializationData.h"
#include "Types/SnapshotTestActor.h"
#include "Util/CustomSubobjectTestUtil.h"
#include "Util/SnapshotTestRunner.h"

#include "Engine/StaticMeshActor.h"
#include "EngineUtils.h"
#include "Misc/AutomationTest.h"
#include "Misc/ScopeExit.h"

namespace UE::LevelSnapshots::Private::Tests
{
	/**
	 * Tests all interface functions are called at the correct time.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreActorCustomSubobject, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.RestoreActorCustomSubobject", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreActorCustomSubobject::RunTest(const FString& Parameters)
	{
		enum class EFunctionCall
		{
			OnTakeSnapshot,
			FindOrRecreateSubobjectInSnapshotWorld,
			FindOrRecreateSubobjectInEditorWorld,
			FindSubobjectInEditorWorld,
			OnPostSerializeSnapshotSubobject,
			OnPostSerializeEditorSubobject,
			PreApplySnapshotProperties,
			PostApplySnapshotProperties
		};
	
		class FStub : public ICustomObjectSnapshotSerializer
		{
			FRestoreActorCustomSubobject& Test;
		public:

			FStub(FRestoreActorCustomSubobject& Test)
				: Test(Test)
			{}
		
			virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::OnTakeSnapshot);
			
				Test.TestTrue(TEXT("Correct editor object passed in"), EditorObject == TestActor);
				const int32 Index = DataStorage.AddSubobjectSnapshot(TestActor->InstancedOnlySubobject_DefaultSubobject);
				DataStorage.GetSubobjectMetaData(Index)->WriteObjectAnnotation(FObjectAnnotator::CreateLambda([](FArchive& Archive)
				{
					int32 TestSubobjectInfo = 42;
					Archive << TestSubobjectInfo;
				}));

				DataStorage.WriteObjectAnnotation(FObjectAnnotator::CreateLambda([](FArchive& Archive)
				{
					int32 TestActorInfo = 21;
					Archive << TestActorInfo;
				}));
			}

			virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::FindOrRecreateSubobjectInSnapshotWorld);

				if (ASnapshotTestActor* SnapshotActor = Cast<ASnapshotTestActor>(SnapshotObject))
				{
					return SnapshotActor->InstancedOnlySubobject_DefaultSubobject;
				}
			
				Test.AddError(TEXT("Expected SnapshotObject to be an instance of ASnapshotTestActor"));
				return nullptr;
			}
		
			virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::FindOrRecreateSubobjectInEditorWorld);
			
				Test.TestTrue(TEXT("Correct editor object passed in"), EditorObject == TestActor);
				return TestActor->InstancedOnlySubobject_DefaultSubobject;	
			}
		
			virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::FindSubobjectInEditorWorld);
			
				Test.TestTrue(TEXT("Correct editor object passed in"), EditorObject == TestActor);

				int32 TestSubobjectInfo = 0;
				ObjectData.ReadObjectAnnotation(FObjectAnnotator::CreateLambda([&TestSubobjectInfo](FArchive& Archive)
				{
					Archive << TestSubobjectInfo;
				}));
				Test.TestEqual(TEXT("Saved custom subobject data is correct"), TestSubobjectInfo, 42);

				int32 TestActorInfo = 0;
				DataStorage.ReadObjectAnnotation(FObjectAnnotator::CreateLambda([&TestActorInfo](FArchive& Archive)
				{
					Archive << TestActorInfo;
				}));
				Test.TestEqual(TEXT("Saved custom actor data is correct"), TestActorInfo, 21);
			
				return TestActor->InstancedOnlySubobject_DefaultSubobject;	
			}

		
			virtual void OnPostSerializeSnapshotSubobject(UObject* Subobject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::OnPostSerializeSnapshotSubobject);
			}
		
			virtual void OnPostSerializeEditorSubobject(UObject* Subobject, const ISnapshotSubobjectMetaData& ObjectData,const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::OnPostSerializeEditorSubobject);
			}
		
			virtual void PreApplyToSnapshotObject(UObject* OriginalObject, const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::PreApplySnapshotProperties);
			}
			virtual void PostApplyToSnapshotObject(UObject* OriginalObject, const ICustomSnapshotSerializationData& DataStorage) override
			{
				CallOrder.Add(EFunctionCall::PostApplySnapshotProperties);
			}
			virtual void PreApplyToEditorObject(UObject* OriginalObject, const ICustomSnapshotSerializationData& DataStorage, const FPropertySelectionMap& SelectionMap) override
			{
				CallOrder.Add(EFunctionCall::PreApplySnapshotProperties);
			}
			virtual void PostApplyToEditorObject(UObject* OriginalObject, const ICustomSnapshotSerializationData& DataStorage, const FPropertySelectionMap& SelectionMap) override
			{
				CallOrder.Add(EFunctionCall::PostApplySnapshotProperties);
			}

			TArray<EFunctionCall> CallOrder;
			ASnapshotTestActor* TestActor;
		};

		// Handle registering and unregistering of custom serializer
		TSharedRef<FStub> Stub = MakeShared<FStub>(*this);
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		Module.RegisterCustomObjectSerializer(ASnapshotTestActor::StaticClass(), Stub);
		DisallowSubobjectProperties();
		ON_SCOPE_EXIT
		{
			Module.UnregisterCustomObjectSerializer(ASnapshotTestActor::StaticClass());
			ReallowSubobjectProperties();
		};
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub->TestActor = ASnapshotTestActor::Spawn(World);;

				Stub->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 42.f;
				Stub->TestActor->InstancedOnlySubobject_DefaultSubobject->IntProperty = 21;
				Stub->TestActor->IntProperty = 42;
				Stub->TestActor->TestComponent->IntProperty = 42;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				Stub->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 420.f;
				Stub->TestActor->InstancedOnlySubobject_DefaultSubobject->IntProperty = 210;
				Stub->TestActor->IntProperty = 420;
				Stub->TestActor->TestComponent->IntProperty = 420;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				// Ignore FindObjectCounterparts... it does not matter
				const int32 TakeSnapshotIndex								= Stub->CallOrder.Find(EFunctionCall::OnTakeSnapshot);
				const int32 OnPostSerializeSnapshotSubobjectIndex			= Stub->CallOrder.Find(EFunctionCall::OnPostSerializeSnapshotSubobject);
				const int32 OnPostSerializeEditorSubobjectIndex				= Stub->CallOrder.Find(EFunctionCall::OnPostSerializeEditorSubobject);
				const int32 PreApplySnapshotPropertiesIndex					= Stub->CallOrder.Find(EFunctionCall::PreApplySnapshotProperties);
				const int32 PostApplySnapshotPropertiesIndex				= Stub->CallOrder.Find(EFunctionCall::PostApplySnapshotProperties);

				// Expected call order?
				TestTrue(TEXT("OnTakeSnapshot was called"), TakeSnapshotIndex != INDEX_NONE);
				TestTrue(TEXT("OnPostSerializeSnapshotSubobject was called"), OnPostSerializeSnapshotSubobjectIndex != INDEX_NONE);
				TestTrue(TEXT("OnPostSerializeEditorSubobject was called"), OnPostSerializeEditorSubobjectIndex != INDEX_NONE);
				TestTrue(TEXT("PreApplySnapshotProperties was called"), PreApplySnapshotPropertiesIndex != INDEX_NONE);
				TestTrue(TEXT("PostApplySnapshotProperties was called"), PostApplySnapshotPropertiesIndex != INDEX_NONE);
				TestTrue(TEXT("PreApplySnapshotProperties called before PostApplySnapshotPropertiesIndex"), PreApplySnapshotPropertiesIndex < OnPostSerializeEditorSubobjectIndex);

				// Custom subobject restored
				TestEqual(TEXT("FloatProperty restored"), Stub->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty, 42.f);
				TestEqual(TEXT("IntProperty restored"), Stub->TestActor->InstancedOnlySubobject_DefaultSubobject->IntProperty, 21);
				// Normal values are still restored
				TestEqual(TEXT("Actor property still restored"), Stub->TestActor->IntProperty, 42);
				TestEqual(TEXT("Component property still restored"), Stub->TestActor->TestComponent->IntProperty, 42);
			});
	
		return true;
	}

	/**
	* Makes sure that we can write
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSaveAndLoadObjectAnnotation, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.SaveAndLoadObjectAnnotation", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FSaveAndLoadObjectAnnotation::RunTest(const FString& Parameters)
	{
		class FStub : public ICustomObjectSnapshotSerializer
		{
			FSaveAndLoadObjectAnnotation& Test;
		public:

			FStub(FSaveAndLoadObjectAnnotation& Test)
				: Test(Test)
			{}
		
			virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
			{
				ASnapshotTestActor* Actor = Cast<ASnapshotTestActor>(EditorObject);
				const int32 Index = DataStorage.AddSubobjectSnapshot(Actor->NonReflectedSubobject.Get());
				DataStorage.WriteObjectAnnotation(FObjectAnnotator::CreateLambda([Actor](FArchive& Archive)
				{
					UObject* Object = Actor->NonReflectedObjectProperty.Get(); 
					Archive << Actor->NonReflectedName;
					Archive << Object;
					Archive << Actor->NonReflectedSoftPtr;
				}));
				DataStorage.GetSubobjectMetaData(Index)->WriteObjectAnnotation(FObjectAnnotator::CreateLambda([Actor](FArchive& Archive)
				{
					Archive << Actor->NonReflectedSubobject->NonReflectedName;
					Archive << Actor->NonReflectedSubobject->NonReflectedObjectProperty;
					Archive << Actor->NonReflectedSubobject->NonReflectedSoftPtr;
				}));
			}

			virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				ASnapshotTestActor* Actor = Cast<ASnapshotTestActor>(SnapshotObject);
				Actor->AllocateNonReflectedSubobject();
				return Actor->NonReflectedSubobject.Get();
			}
		
			virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				ASnapshotTestActor* Actor = Cast<ASnapshotTestActor>(EditorObject);
				Actor->AllocateNonReflectedSubobject();
				return Actor->NonReflectedSubobject.Get();
			}
		
			virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				ASnapshotTestActor* Actor = Cast<ASnapshotTestActor>(EditorObject);
				return Actor->NonReflectedSubobject.Get();
			}

			virtual void PostApplyToSnapshotObject(UObject* OriginalObject, const ICustomSnapshotSerializationData& DataStorage) override
			{
				ASnapshotTestActor* Actor = Cast<ASnapshotTestActor>(OriginalObject);
				DataStorage.ReadObjectAnnotation(FObjectAnnotator::CreateLambda([Actor](FArchive& Archive)
				{
					UObject* Object = nullptr; 
					Archive << Actor->NonReflectedName;
					Archive << Object;
					Archive << Actor->NonReflectedSoftPtr;
					Actor->NonReflectedObjectProperty.Reset(Object);
				}));

				DataStorage.GetSubobjectMetaData(0)->ReadObjectAnnotation(FObjectAnnotator::CreateLambda([Actor](FArchive& Archive)
				{
					Archive << Actor->NonReflectedSubobject->NonReflectedName;
					Archive << Actor->NonReflectedSubobject->NonReflectedObjectProperty;
					Archive << Actor->NonReflectedSubobject->NonReflectedSoftPtr;
				}));
			}
		};

		// Handle registering and unregistering of custom serializer
		TSharedRef<FStub> Stub = MakeShared<FStub>(*this);
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		Module.RegisterCustomObjectSerializer(ASnapshotTestActor::StaticClass(), Stub);
		DisallowSubobjectProperties();
		ON_SCOPE_EXIT
		{
			Module.UnregisterCustomObjectSerializer(ASnapshotTestActor::StaticClass());
			ReallowSubobjectProperties();
		};

		ASnapshotTestActor* TestActor = nullptr;
		// Do not use ASnapshotTestActor as class for referenced actors because we don't want them to be processed by FStub
		AStaticMeshActor* FirstReferencedActor = nullptr;
		AStaticMeshActor* SecondReferencedActor = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
				TestActor->AllocateNonReflectedSubobject();
				// Spawn at different locations to avoid them failing to spawn
				FirstReferencedActor = World->SpawnActor<AStaticMeshActor>(FVector(500.f), FRotator());
				SecondReferencedActor = World->SpawnActor<AStaticMeshActor>(FVector(-500.f), FRotator());
			
				TestActor->IntProperty = 1;
				TestActor->NonReflectedName = FName("TestNonReflectedName_OnActor");
				TestActor->NonReflectedObjectProperty.Reset(FirstReferencedActor);
				TestActor->NonReflectedSoftPtr = FirstReferencedActor;
			
				TestActor->NonReflectedSubobject->NonReflectedName = FName("TestNonReflectedName_OnSubobject");
				TestActor->NonReflectedSubobject->NonReflectedObjectProperty = SecondReferencedActor;
				TestActor->NonReflectedSubobject->NonReflectedSoftPtr = SecondReferencedActor;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				// This makes the actor show up as changed - otherwise nothing will get serialized because all other properties are not reflected
				TestActor->IntProperty = 2;

				TestActor->NonReflectedName = FName(NAME_None);
				TestActor->NonReflectedObjectProperty = nullptr;
				TestActor->NonReflectedSoftPtr = nullptr;
			
				TestActor->NonReflectedSubobject->NonReflectedName = FName(NAME_None);
				TestActor->NonReflectedSubobject->NonReflectedObjectProperty = nullptr;
				TestActor->NonReflectedSubobject->NonReflectedSoftPtr = nullptr;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("TestActor->NonReflectedName"), TestActor->NonReflectedName, FName("TestNonReflectedName_OnActor"));
				TestTrue(TEXT("TestActor->NonReflectedObjectProperty"), TestActor->NonReflectedObjectProperty.Get() == FirstReferencedActor);
				TestTrue(TEXT("TestActor->NonReflectedSoftPtr"), TestActor->NonReflectedSoftPtr == FirstReferencedActor);
			
				TestEqual(TEXT("TestActor->NonReflectedSubobject->NonReflectedName"), TestActor->NonReflectedSubobject->NonReflectedName, FName("TestNonReflectedName_OnSubobject"));
				TestTrue(TEXT("TestActor->NonReflectedSubobject->NonReflectedObjectProperty"), TestActor->NonReflectedSubobject->NonReflectedObjectProperty == SecondReferencedActor);
				TestTrue(TEXT("TestActor->NonReflectedSubobject->NonReflectedSoftPtr"), TestActor->NonReflectedSubobject->NonReflectedSoftPtr == SecondReferencedActor);
			
			});

		return true;
	}

	/**
	* Tests that custom serialization works when an actor adds a subobject dependency to a subobject that also has a custom serializer.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreNestedCustomSubobject, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.RestoreNestedCustomSubobject", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreNestedCustomSubobject::RunTest(const FString& Parameters)
	{
		class FActorSerializer : public ICustomObjectSnapshotSerializer
		{
		public:
		
			virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
			{
				DataStorage.AddSubobjectSnapshot(TestActor->InstancedOnlySubobject_DefaultSubobject);
			}

			virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				if (ASnapshotTestActor* SnapshotActor = Cast<ASnapshotTestActor>(SnapshotObject))
				{
					return SnapshotActor->InstancedOnlySubobject_DefaultSubobject;
				}

				checkNoEntry();
				return nullptr;
			}
		
			virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				return TestActor->InstancedOnlySubobject_DefaultSubobject;	
			}
		
			virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				return TestActor->InstancedOnlySubobject_DefaultSubobject;	
			}

			ASnapshotTestActor* TestActor;
		};
	
		class FSubobjectSerializer : public ICustomObjectSnapshotSerializer
		{
		public:
		
			virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
			{
				DataStorage.AddSubobjectSnapshot(TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild);
			}

			virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				if (USubobject* Subobject = Cast<USubobject>(SnapshotObject))
				{
					return Subobject->NestedChild;
				}

				checkNoEntry();
				return nullptr;
			}
		
			virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				return TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild;	
			}
		
			virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				return TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild;	
			}

			ASnapshotTestActor* TestActor;
		};
	
		// Handle registering and unregistering of custom serializer
		TSharedRef<FActorSerializer> ActorSerializer = MakeShared<FActorSerializer>();
		TSharedRef<FSubobjectSerializer> SubobjectSerializer = MakeShared<FSubobjectSerializer>();
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		Module.RegisterCustomObjectSerializer(ASnapshotTestActor::StaticClass(), ActorSerializer);
		Module.RegisterCustomObjectSerializer(USubobject::StaticClass(), SubobjectSerializer);
		DisallowSubobjectProperties();
		ON_SCOPE_EXIT
		{
			Module.UnregisterCustomObjectSerializer(ASnapshotTestActor::StaticClass());
			Module.UnregisterCustomObjectSerializer(USubobject::StaticClass());
			ReallowSubobjectProperties();
		};

		ASnapshotTestActor* TestActor = nullptr;

		// Change properties on subobject and it's subobject. Both restored.
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
				ActorSerializer->TestActor = TestActor;
				SubobjectSerializer->TestActor = TestActor;
			
				TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 42.f;
				TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild->FloatProperty = 21.f;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 420.f;
				TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild->FloatProperty = 210.f;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("Subobject property restored"), TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty, 42.f);
				TestEqual(TEXT("Nested subobject property restored"), TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild->FloatProperty, 21.f);
			});


		// Change properties on subobject's subobject only. Still restored.
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
				ActorSerializer->TestActor = TestActor;
				SubobjectSerializer->TestActor = TestActor;
			
				TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild->FloatProperty = 21.f;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild->FloatProperty = 210.f;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("Nested subobject property restored when outer had no changed properties"), TestActor->InstancedOnlySubobject_DefaultSubobject->NestedChild->FloatProperty, 21.f);
			});
	
		return true;
	}

	/**
	* Tests that the following properties are not restored if no callbacks are registered for it:
	*
	* class ABlah
	* {
	*		UPROPERTY(Instanced)
	*		UObject* Object;
	*
	*		UPROPERTY()
	*		UObject* OtherObject;
	* };
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNonEditableObjectPropertyNotRestoredByDefault, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.NonEditableObjectPropertyNotRestoredByDefault", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FNonEditableObjectPropertyNotRestoredByDefault::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* ChangedPropertiesActor = nullptr;
		ASnapshotTestActor* NulledActor = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ChangedPropertiesActor = ASnapshotTestActor::Spawn(World, "ChangedPropertiesActor");
				NulledActor = ASnapshotTestActor::Spawn(World, "NulledActor");

				ChangedPropertiesActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 42.f;
				ChangedPropertiesActor->NakedSubobject_DefaultSubobject->FloatProperty = 21.f;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				ChangedPropertiesActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 420.f;
				ChangedPropertiesActor->NakedSubobject_DefaultSubobject->FloatProperty = 210.f;

				NulledActor->InstancedOnlySubobject_DefaultSubobject = nullptr;
				NulledActor->NakedSubobject_DefaultSubobject = nullptr;
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				TestEqual(TEXT("Unsupported instanced property not restored"), ChangedPropertiesActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty, 420.f);
				TestEqual(TEXT("Unsupported naked property not restored"), ChangedPropertiesActor->NakedSubobject_DefaultSubobject->FloatProperty, 210.f);
			
				TestTrue(TEXT("Unsupported instanced property stays null"), NulledActor->InstancedOnlySubobject_DefaultSubobject == nullptr);
				TestTrue(TEXT("Unsupported naked property stays null"), NulledActor->NakedSubobject_DefaultSubobject == nullptr);
			});

		return true;
	}

	/**
	* Checks that changed properties on custom restored subobjects are discovered
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFilterForPropertiesOnSubobjects, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.FilterForPropertiesOnSubobjects", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FFilterForPropertiesOnSubobjects::RunTest(const FString& Parameters)
	{
		// Handle registering and unregistering of custom serializer
		const TCustomObjectSerializerContext<FInstancedOnlySubobjectCustomObjectSerializer> Stub =
			FInstancedOnlySubobjectCustomObjectSerializer::Make();
	
		// Modify subobject. Actor unchanged. Actor is in selection map.
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
			
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 42.f;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 420.f;
			})

			.FilterProperties(Stub.GetCustomSerializer()->TestActor, [&](const FPropertySelectionMap& PropertySelectionMap)
			{
				// Custom subobject properties
				UClass* SubobjectClass = USubobject::StaticClass();
				const FProperty* ChangedSubobjectProperty = SubobjectClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, FloatProperty));

				const FPropertySelection* SelectedSubobjectProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject).GetPropertySelection();
				const bool bSubobjectHasExpectedNumChangedProperties = SelectedSubobjectProperties && SelectedSubobjectProperties->GetSelectedLeafProperties().Num() == 1;
				TestTrue(TEXT("Subobject has changed properties"), bSubobjectHasExpectedNumChangedProperties);
				TestTrue(TEXT("Changed property on subobject contained"), bSubobjectHasExpectedNumChangedProperties && SelectedSubobjectProperties->IsPropertySelected(nullptr, ChangedSubobjectProperty));


				// Actor properties
				const FPropertySelection* SelectedActorProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor).GetPropertySelection();
				TestTrue(TEXT("Unchanged actor in selection map when subobject was changed"), SelectedActorProperties != nullptr);
			});
		

	
		// Modify nothing. No properties in selection map.
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
			})
			.TakeSnapshot()
			.FilterProperties(Stub.GetCustomSerializer()->TestActor, [&](const FPropertySelectionMap& PropertySelectionMap)
			{
				// Custom subobject properties
				const FPropertySelection* SelectedSubobjectProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject).GetPropertySelection();
				TestTrue(TEXT("Unchanged subobject not in selection map"), SelectedSubobjectProperties == nullptr || SelectedSubobjectProperties->IsEmpty());


				// Actor properties
				const FPropertySelection* SelectedActorProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor).GetPropertySelection();
				TestTrue(TEXT("Unchanged actor not in selection map when subobject was not changed"), SelectedActorProperties == nullptr || SelectedActorProperties->IsEmpty());
			});






	
		// Make sure normal properties show up too:
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
			
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 42.f;
				Stub.GetCustomSerializer()->TestActor->IntProperty = 42;
				Stub.GetCustomSerializer()->TestActor->TestComponent->IntProperty = 42;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 420.f;
				Stub.GetCustomSerializer()->TestActor->IntProperty = 420;
				Stub.GetCustomSerializer()->TestActor->TestComponent->IntProperty = 420;
			})

			.FilterProperties(Stub.GetCustomSerializer()->TestActor, [&](const FPropertySelectionMap& PropertySelectionMap)
			{
				// Custom subobject properties
				UClass* SubobjectClass = USubobject::StaticClass();
				const FProperty* ChangedSubobjectProperty = SubobjectClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, FloatProperty));

				const FPropertySelection* SelectedSubobjectProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject).GetPropertySelection();
				const bool bSubobjectHasExpectedNumChangedProperties = SelectedSubobjectProperties && SelectedSubobjectProperties->GetSelectedLeafProperties().Num() == 1;
				TestTrue(TEXT("Subobject has changed properties"), bSubobjectHasExpectedNumChangedProperties);
				TestTrue(TEXT("Changed property on subobject contained"), bSubobjectHasExpectedNumChangedProperties && SelectedSubobjectProperties->IsPropertySelected(nullptr, ChangedSubobjectProperty));


				// Actor properties
				UClass* TestActorClass = ASnapshotTestActor::StaticClass();
				const FProperty* ChangedActorProperty = TestActorClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, IntProperty));
			
				const FPropertySelection* SelectedActorProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor).GetPropertySelection();
				const bool bActorHasExpectedNumChangedProperties = SelectedActorProperties && SelectedActorProperties->GetSelectedLeafProperties().Num();
				TestTrue(TEXT("Actor has changed properties"), bActorHasExpectedNumChangedProperties);
				TestTrue(TEXT("Changed property on actor contained"), bActorHasExpectedNumChangedProperties && SelectedActorProperties->IsPropertySelected(nullptr, ChangedActorProperty));


				// Normal subobject properties, e.g. component
				UClass* TestComponentClass = USnapshotTestComponent::StaticClass();
				const FProperty* ChangedComponentProperty = TestComponentClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USnapshotTestComponent, IntProperty));
			
				const FPropertySelection* SelectedComponentProperties = PropertySelectionMap.GetObjectSelection(Stub.GetCustomSerializer()->TestActor->TestComponent).GetPropertySelection();
				const bool bComponentHasExpectedNumChangedProperties = SelectedComponentProperties && SelectedComponentProperties->GetSelectedLeafProperties().Num();
				TestTrue(TEXT("Component has changed properties"), bComponentHasExpectedNumChangedProperties);
				TestTrue(TEXT("Changed property on component contained"), bComponentHasExpectedNumChangedProperties && SelectedComponentProperties->IsPropertySelected(nullptr, ChangedComponentProperty));
			});
	
		return true;
	}

	/**
	 * Checks that restoring subobjects which are missing from the editor world are in fact restored.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreSubobjectsMissingFromEditorWorld, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.RestoreSubobjectsMissingFromEditorWorld", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreSubobjectsMissingFromEditorWorld::RunTest(const FString& Parameters)
	{
		class FStub : public ICustomObjectSnapshotSerializer
		{
		public:
		
			virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
			{
				if (TestActor->NonReflectedSubobject)
				{
					DataStorage.AddSubobjectSnapshot(TestActor->NonReflectedSubobject.Get());
				}
			}

			virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				if (ASnapshotTestActor* SnapshotActor = Cast<ASnapshotTestActor>(SnapshotObject))
				{
					SnapshotActor->AllocateNonReflectedSubobject();
					return SnapshotActor->NonReflectedSubobject.Get();
				}

				checkNoEntry();
				return nullptr;
			}
		
			virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				if (ASnapshotTestActor* EditorActor = Cast<ASnapshotTestActor>(EditorObject))
				{
					EditorActor->AllocateNonReflectedSubobject();
					return EditorActor->NonReflectedSubobject.Get();
				}

				checkNoEntry();
				return nullptr;	
			}
		
			virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
			{
				if (ASnapshotTestActor* EditorActor = Cast<ASnapshotTestActor>(EditorObject))
				{
					return EditorActor->NonReflectedSubobject.Get();
				}

				checkNoEntry();
				return nullptr;
			}

			ASnapshotTestActor* TestActor;
		};
	
		// Handle registering and unregistering of custom serializer
		TSharedRef<FStub> Stub = MakeShared<FStub>();
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		Module.RegisterCustomObjectSerializer(ASnapshotTestActor::StaticClass(), Stub);
		DisallowSubobjectProperties();
		ON_SCOPE_EXIT
		{
			Module.UnregisterCustomObjectSerializer(ASnapshotTestActor::StaticClass());
			ReallowSubobjectProperties();
		};

	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub->TestActor = ASnapshotTestActor::Spawn(World, "TestActor");
				Stub->TestActor->IntProperty = 21;
				Stub->TestActor->AllocateNonReflectedSubobject();
				Stub->TestActor->NonReflectedSubobject->FloatProperty = 42.f;
				Stub->TestActor->NonReflectedSubobject->IntProperty = 21;
			})
			.TakeSnapshot()

			.ModifyWorld([&](UWorld* World)
			{
				Stub->TestActor->DestroyNonReflectedSubobject();
				// Needed otherwise no changes will be detected by hash
				Stub->TestActor->IntProperty = 42;
			})
			.FilterProperties(Stub->TestActor, [&](const FPropertySelectionMap& SelectionMap)
			{
				const UE::LevelSnapshots::FRestorableObjectSelection ObjectSelection = SelectionMap.GetObjectSelection(Stub->TestActor);
				TestTrue(TEXT("No changed actor properties"), ObjectSelection.GetPropertySelection() && ObjectSelection.GetPropertySelection()->GetSelectedProperties().Num() == 1 && ObjectSelection.GetPropertySelection()->HasCustomSerializedSubobjects());
				TestTrue(TEXT("No component selection"), ObjectSelection.GetComponentSelection() == nullptr);
				TestTrue(TEXT("Needs to restore custom subobject"), ObjectSelection.GetCustomSubobjectSelection() && ObjectSelection.GetCustomSubobjectSelection()->CustomSnapshotSubobjectsToRestore.Num() == 1);
			})
			.ApplySnapshot()

			.RunTest([&]()
			{
				if (IsValid(Stub->TestActor->NonReflectedSubobject.Get()))
				{
					TestEqual(TEXT("Custom float restored"), Stub->TestActor->NonReflectedSubobject->FloatProperty, 42.f);
					TestEqual(TEXT("Custom float restored"), Stub->TestActor->NonReflectedSubobject->IntProperty, 21);
				}
				else
				{
					AddError(TEXT("Custom subobject was not restored"));
				}
			});
	
		return true;
	}

	/**
	 * When an actor is recreated, all of its custom subobjects are recreated as well.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoresCustomSubobjectWhenActorRecreated, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.RestoresCustomSubobjectWhenActorRecreated", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoresCustomSubobjectWhenActorRecreated::RunTest(const FString& Parameters)
	{
		// Handle registering and unregistering of custom serializer
		const TCustomObjectSerializerContext<FInstancedOnlySubobjectCustomObjectSerializer> Stub =
			FInstancedOnlySubobjectCustomObjectSerializer::Make();

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor = ASnapshotTestActor::Spawn(World, "RecreatedActor");
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 12345.f;
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 5.f;
				World->DestroyActor(Stub.GetCustomSerializer()->TestActor);
				Stub.GetCustomSerializer()->TestActor = nullptr;
			})
			.ApplySnapshot()
		
			.ModifyWorld([&](UWorld* World)
			{
				const TActorIterator<ASnapshotTestActor> ActorIterator(World);
				if (!ActorIterator)
				{
					AddError(TEXT("Actor not restored"));
					return;
				}

				const ASnapshotTestActor* RestoredActor = *ActorIterator;
				TestEqual(TEXT("Default Subobject was restored"), RestoredActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty, 12345.f);
			});
		
		return true;
	}

	/** Tests that class archetype data is saved for custom subobjects. That means if a snapshot is taken on an unmodified custom subobject and then changed, it should restore. Fix for UE-168414. */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreCustomSubobjectPropertiesEqualToArchetypeValue, "VirtualProduction.LevelSnapshots.Snapshot.CustomObjectSerialization.RestorePropertiesEqualToArchetypeValue", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreCustomSubobjectPropertiesEqualToArchetypeValue::RunTest(const FString& Parameters)
	{
		// Handle registering and unregistering of custom serializer
		const TCustomObjectSerializerContext<FInstancedOnlySubobjectCustomObjectSerializer> Stub =
			FInstancedOnlySubobjectCustomObjectSerializer::Make();

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor = ASnapshotTestActor::Spawn(World, "RecreatedActor");
			})
			.TakeSnapshot()
	
			.ModifyWorld([&](UWorld* World)
			{
				Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty = 5.f;
			})
			.ApplySnapshot()
		
			.ModifyWorld([&](UWorld* World)
			{
				TestEqual(TEXT("Restore property that was equal to archetype value at time of taking snapshot"),
					Stub.GetCustomSerializer()->TestActor->InstancedOnlySubobject_DefaultSubobject->FloatProperty,
					0.f);
			});
		
		return true;
	} 
}
