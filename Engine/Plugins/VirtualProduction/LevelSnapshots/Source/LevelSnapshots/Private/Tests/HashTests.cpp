// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Util/SnapshotTestRunner.h"
#include "Tests/Types/SnapshotTestActor.h"

#include "Engine/World.h"
#include "Misc/AutomationTest.h"

namespace UE::LevelSnapshots::Private::Tests
{
	/**
	* Regression test. Swapping indices in a TArray<USubobject*> would result in the same hash, e.g. { ObjectA, nullptr } to { nullptr, ObjectA }. 
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHashSubobjectArray, "VirtualProduction.LevelSnapshots.Snapshot.Hash.HashSubobjectArray", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FHashSubobjectArray::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* SwapElementWithNull = nullptr;
		USubobject* First = nullptr;
		USubobject* Third = nullptr;

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				SwapElementWithNull = ASnapshotTestActor::Spawn(World, "SwapElementWithNull");
			
				First = NewObject<USubobject>(SwapElementWithNull, USubobject::StaticClass(), TEXT("First"));
				Third = NewObject<USubobject>(SwapElementWithNull, USubobject::StaticClass(), TEXT("Third"));

				SwapElementWithNull->ObjectArray.Add(First);
				SwapElementWithNull->ObjectArray.Add(nullptr);
				SwapElementWithNull->ObjectArray.Add(Third);
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				SwapElementWithNull->ObjectArray[0] = nullptr;
				SwapElementWithNull->ObjectArray[1] = First;
			
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("SwapElementWithNull[0]"), SwapElementWithNull->ObjectArray[0] && SwapElementWithNull->ObjectArray[0] == First);
				TestTrue(TEXT("SwapElementWithNull[1]"), SwapElementWithNull->ObjectArray[1] == nullptr);
				TestTrue(TEXT("SwapElementWithNull[2]"), SwapElementWithNull->ObjectArray[2] && SwapElementWithNull->ObjectArray[2] == Third);
			});
	
		return true;
	}
}