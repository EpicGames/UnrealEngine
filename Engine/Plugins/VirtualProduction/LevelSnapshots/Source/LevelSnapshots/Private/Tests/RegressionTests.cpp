// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filtering/PropertySelectionMap.h"
#include "Util/SnapshotTestRunner.h"
#include "Types/SnapshotTestActor.h"
#include "Types/ActorWithReferencesInCDO.h"

#include "Engine/World.h"
#include "EngineUtils.h"
#include "Algo/Count.h"
#include "Misc/AutomationTest.h"

// Bug fixes should generally be tested. Put tests for bug fixes here.

namespace UE::LevelSnapshots::Private::Tests
{
	/**
	* FTakeClassDefaultObjectSnapshotArchive used to crash when a class CDO contained a collection of object references. Make sure it does not crash and restores.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersWithObjectReferencesInCDO, "VirtualProduction.LevelSnapshots.Snapshot.Regression.ContainersWithObjectReferencesInCDO", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FContainersWithObjectReferencesInCDO::RunTest(const FString& Parameters)
	{
		AActorWithReferencesInCDO* Actor = nullptr;

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = World->SpawnActor<AActorWithReferencesInCDO>();
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Actor->SetAllPropertiesTo(Actor->CylinderMesh);
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("Object properties restored correctly"), Actor->DoAllPropertiesPointTo(Actor->CubeMesh));
			});
	
		return true;
	}

	namespace RestoreCollision
	{
		static bool AreCollisionResponsesEqual(ASnapshotTestActor* Instance, ECollisionResponse Response, int32 NumChannels)
		{
			const FCollisionResponse& Responses = Instance->StaticMeshComponent->BodyInstance.GetCollisionResponse();
			for (int32 i = 0; i < NumChannels; ++i)
			{
				if (Responses.GetResponse(static_cast<ECollisionChannel>(i)) != Response)
				{
					return false;
				}
			}
			return true;
		}
		static bool WereAllCollisionResponsesRestored(ASnapshotTestActor* Instance, ECollisionResponse Response)
		{
			return AreCollisionResponsesEqual(Instance, Response, 32);
		}

		static bool WereCollisionProfileResponsesRestored(ASnapshotTestActor* Instance, ECollisionResponse Response)
		{
			return AreCollisionResponsesEqual(Instance, Response, static_cast<int32>(ECollisionChannel::ECC_EngineTraceChannel1));
		}
	}

	/**
	* Multiple bugs:
	*	- FBodyInstance::ObjectType, FBodyInstance::CollisionEnabled and FBodyInstance::CollisionResponses should not be diffed when UStaticMeshComponent::bUseDefaultCollision == true
	*	- Other FBodyInstance properties should still diff and restore correctly
	*	- After restoration, transient property FCollisionResponse::ResponseToChannels should contain the correct values
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreCollision, "VirtualProduction.LevelSnapshots.Snapshot.Regression.RestoreCollision", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreCollision::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* CustomBlockAllToOverlapAll = nullptr;
		ASnapshotTestActor* DefaultWithChangedIgnoredProperties = nullptr;
		ASnapshotTestActor* DefaultToCustom = nullptr;
		ASnapshotTestActor* CustomToDefault = nullptr;
		ASnapshotTestActor* ChangeCollisionProfile = nullptr;

		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				CustomBlockAllToOverlapAll = ASnapshotTestActor::Spawn(World, "CustomBlockAllToOverlapAll");
				DefaultWithChangedIgnoredProperties = ASnapshotTestActor::Spawn(World, "DefaultWithChangedIgnoredProperties");
				DefaultToCustom = ASnapshotTestActor::Spawn(World, "DefaultToCustom");
				CustomToDefault = ASnapshotTestActor::Spawn(World, "CustomToDefault");
				ChangeCollisionProfile = ASnapshotTestActor::Spawn(World, "ChangeCollisionProfile");

				CustomBlockAllToOverlapAll->StaticMeshComponent->bUseDefaultCollision = false;
				CustomBlockAllToOverlapAll->StaticMeshComponent->SetCollisionResponseToAllChannels(ECR_Block);
			
				DefaultWithChangedIgnoredProperties->StaticMeshComponent->BodyInstance.SetCollisionEnabled(ECollisionEnabled::NoCollision);
				DefaultWithChangedIgnoredProperties->StaticMeshComponent->BodyInstance.SetObjectType(ECC_WorldStatic);
				DefaultWithChangedIgnoredProperties->StaticMeshComponent->BodyInstance.SetResponseToAllChannels(ECR_Ignore);
				DefaultWithChangedIgnoredProperties->StaticMeshComponent->bUseDefaultCollision = true;
			
				DefaultToCustom->StaticMeshComponent->bUseDefaultCollision = true;
			
				CustomToDefault->StaticMeshComponent->bUseDefaultCollision = false;
				CustomToDefault->StaticMeshComponent->SetCollisionResponseToAllChannels(ECR_Ignore);

				ChangeCollisionProfile->StaticMeshComponent->SetCollisionProfileName(FName("OverlapAll"));
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				CustomBlockAllToOverlapAll->StaticMeshComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Overlap);

				DefaultWithChangedIgnoredProperties->StaticMeshComponent->BodyInstance.SetCollisionEnabled(ECollisionEnabled::PhysicsOnly);
				DefaultWithChangedIgnoredProperties->StaticMeshComponent->BodyInstance.SetObjectType(ECC_WorldDynamic);
				DefaultWithChangedIgnoredProperties->StaticMeshComponent->BodyInstance.SetResponseToAllChannels(ECR_Overlap);

				DefaultToCustom->StaticMeshComponent->SetCollisionProfileName(FName("OverlapAll"));

				CustomToDefault->StaticMeshComponent->bUseDefaultCollision = true;

				ChangeCollisionProfile->StaticMeshComponent->SetCollisionProfileName(FName("BlockAll"));
			})

			.FilterProperties(DefaultWithChangedIgnoredProperties, [&](const FPropertySelectionMap& SelectionMap)
			{
				TestTrue(TEXT("Collision properties ignored when default collision is used"), !SelectionMap.HasChanges(DefaultWithChangedIgnoredProperties));
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("CustomBlockAllToOverlapAll: SetResponseToAllChannels restored"), RestoreCollision::WereAllCollisionResponsesRestored(CustomBlockAllToOverlapAll, ECR_Block));
			
				TestTrue(TEXT("Default collision restored"), DefaultToCustom->StaticMeshComponent->bUseDefaultCollision);
			
				TestTrue(TEXT("CustomToDefault: SetResponseToAllChannels restored"), RestoreCollision::WereAllCollisionResponsesRestored(CustomToDefault, ECR_Ignore));
			
				TestEqual(TEXT("Profile name restored"), ChangeCollisionProfile->StaticMeshComponent->GetCollisionProfileName(), FName("OverlapAll"));
				TestTrue(TEXT("Collision profile responses restored"), RestoreCollision::WereCollisionProfileResponsesRestored(ChangeCollisionProfile, ECR_Overlap));
			});
	
		return true;
	}

	/**
	* Suppose snapshot contains Root > Child and now the hierarchy is Child > Root. This used to cause a crash.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUpdateAttachChildrenInfiniteLoop, "VirtualProduction.LevelSnapshots.Snapshot.Regression.UpdateAttachChildrenInfiniteLoop", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FUpdateAttachChildrenInfiniteLoop::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Root = nullptr;
		ASnapshotTestActor* Child = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Root = ASnapshotTestActor::Spawn(World, "Root");
				Child = ASnapshotTestActor::Spawn(World, "Child");

				Child->AttachToActor(Root, FAttachmentTransformRules::KeepRelativeTransform);
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Child->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
				Root->AttachToActor(Child, FAttachmentTransformRules::KeepRelativeTransform);
			})
			// If the issue remains, ApplySnapshot will now cause a stack overflow crash 
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("Root: attach component"), Root->GetRootComponent()->GetAttachParent() == nullptr);
				TestTrue(TEXT("Root: attach actor"), Root->GetAttachParentActor() == nullptr);
			
				TestTrue(TEXT("Child: attach component"), Child->GetRootComponent()->GetAttachParent() == Root->GetRootComponent());
				TestTrue(TEXT("Child: attach actor"), Child->GetAttachParentActor() == Root);
			});

		return true;
	}

	/**
	* Spawn naked AActor and add instanced components. RootComponent needs to be set.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreRootComponent, "VirtualProduction.LevelSnapshots.Snapshot.Regression.RestoreRootComponent", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreRootComponent::RunTest(const FString& Parameters)
	{
		AActor* Actor = nullptr;
		USceneComponent* InstancedComponent = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = World->SpawnActor<AActor>();

				InstancedComponent = NewObject<USceneComponent>(Actor, USceneComponent::StaticClass(), "InstancedComponent");
				Actor->AddInstanceComponent(InstancedComponent);
				Actor->SetRootComponent(InstancedComponent);
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				InstancedComponent->DestroyComponent();
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("RootComponent"), Actor->GetRootComponent() != nullptr && Actor->GetRootComponent()->GetFName().IsEqual("InstancedComponent"));
			});

		return true;
	}

	/**
	 * Verify that the actor label property is restored
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreActorLabel, "VirtualProduction.LevelSnapshots.Snapshot.Regression.ActorLabelRestores", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreActorLabel::RunTest(const FString& Parameters)
	{
		AActor* KeepActor = nullptr;
		AActor* RemovedActor = nullptr;
		const FString KeepActorLabel = TEXT("KeepActor");
		const FString RemovedActorLabel = TEXT("RemovedActor");
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				KeepActor = World->SpawnActor<AActor>();
				RemovedActor = World->SpawnActor<AActor>();;

				KeepActor->SetActorLabel(KeepActorLabel);
				RemovedActor->SetActorLabel(RemovedActorLabel);
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				KeepActor->SetActorLabel(TEXT("NewName"));
				RemovedActor->Destroy();
			})
			.ApplySnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				TestEqual(TEXT("Modified actor label is restored"), KeepActor->GetActorLabel(), KeepActorLabel);

				bool bRemovedActorHasCorrectLabel = false;
				for (TActorIterator<AActor> ActorIt(World); ActorIt && !bRemovedActorHasCorrectLabel; ++ActorIt)
				{
					bRemovedActorHasCorrectLabel = ActorIt->GetActorLabel() == RemovedActorLabel;
				}
				TestTrue(TEXT("Removed actor label is restored"), bRemovedActorHasCorrectLabel);
			});

		return true;
	}

#if WITH_EDITOR
	/** Tests that actors implementing AActor::CanDeleteSelectedActor and AActor::IsUserManaged are properly removed. */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUserManagedActors, "VirtualProduction.LevelSnapshots.Snapshot.Regression.UserManagedActors", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FUserManagedActors::RunTest(const FString& Parameters)
	{
		int32 NumActorsBeforeModify;
		FSnapshotTestRunner()
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				NumActorsBeforeModify = Algo::CountIf(World->PersistentLevel->Actors, [](AActor* Actor){ return Actor != nullptr; });
				
				ASnapshotTestActor* NotDeletableActor = ASnapshotTestActor::Spawn(World);
				ASnapshotTestActor* NotUserManagedActor = ASnapshotTestActor::Spawn(World);
				NotDeletableActor->DeleteSelectedActorMode = ASnapshotTestActor::ECanDeleteMode::No;
				NotUserManagedActor->bIsUserManaged = false;
			})
			.ApplySnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				const int32 NumActorsAfterRestore = Algo::CountIf(World->PersistentLevel->Actors, [](AActor* Actor){ return Actor != nullptr; });
				TestEqual(TEXT("CanDeleteSelectedActor & IsUserManaged"), NumActorsAfterRestore, NumActorsBeforeModify);
			});

		return true;
	}
#endif

	/**
	 * There used to be a bug in FApplySnapshotFilter::TrackPossibleMapSubobjectProperties would invoke FScriptSetHelper:.GetElementPtr and
	 * FScriptMapHelper::GetKeyPtr before converting logical indices to internal indices.
	 * The result was that the following would crash:
	 * 1. Add an actor that has a UPROPERTY(EditAnywhere, Instanced) TMap<FName, UObject*> property
	 * 2. Add 3 subobjects to that property
	 * 3. Remove the middle one
	 * 4. Diff the snapshot > Crash
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInstancedSubobjectRemovedFromTMap, "VirtualProduction.LevelSnapshots.Snapshot.Regression.InstancedSubobjectRemovedFromTMap", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FInstancedSubobjectRemovedFromTMap::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
		USubobject* Subobject1 = nullptr;
		USubobject* Subobject2 = nullptr;
		USubobject* Subobject3 = nullptr;
		const FName SubobjectName1 = TEXT("Subobject1");
		const FName SubobjectName2 = TEXT("Subobject2");
		const FName SubobjectName3 = TEXT("Subobject3");
		
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World);
				Subobject1 = NewObject<USubobject>(Actor, SubobjectName1);
				Subobject2 = NewObject<USubobject>(Actor, SubobjectName2);
				Subobject3 = NewObject<USubobject>(Actor, SubobjectName3);

				Subobject1->IntProperty = 1;
				Subobject2->IntProperty = 2;
				Subobject3->IntProperty = 3;
				
				Actor->EditableInstancedSubobjectMap_OptionalSubobject.Add(SubobjectName1, Subobject1);
				Actor->EditableInstancedSubobjectMap_OptionalSubobject.Add(SubobjectName2, Subobject2);
				Actor->EditableInstancedSubobjectMap_OptionalSubobject.Add(SubobjectName3, Subobject3);
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Actor->EditableInstancedSubobjectMap_OptionalSubobject.Remove(SubobjectName2);
			})
			.ApplySnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				const TObjectPtr<USubobject>* Restored1 = Actor->EditableInstancedSubobjectMap_OptionalSubobject.Find(SubobjectName1);
				const TObjectPtr<USubobject>* Restored2 = Actor->EditableInstancedSubobjectMap_OptionalSubobject.Find(SubobjectName2);
				const TObjectPtr<USubobject>* Restored3 = Actor->EditableInstancedSubobjectMap_OptionalSubobject.Find(SubobjectName3);

				if (!Restored1 || !Restored2 || !Restored3 || !*Restored1 || !*Restored2 || !*Restored3)
				{
					AddError(TEXT("Not all objects where restored"));
				}
				else
				{
					TestEqual(TEXT("1"), Restored1->Get()->IntProperty, 1);
					TestEqual(TEXT("2"), Restored2->Get()->IntProperty, 2);
					TestEqual(TEXT("3"), Restored3->Get()->IntProperty, 3);
				}
			});

		return true;
	}
}