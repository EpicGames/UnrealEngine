// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstantFilter.h"
#include "Filtering/PropertySelectionMap.h"
#include "ILevelSnapshotsModule.h"
#include "Interfaces/ISnapshotRestorabilityOverrider.h"
#include "Util/SnapshotTestRunner.h"
#include "Types/SnapshotTestActor.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/AutomationTest.h"
#include "Misc/ScopeExit.h"

namespace UE::LevelSnapshots::Private::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRestoreSubobjectProperties, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.RestoreSubobjectProperties", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FRestoreSubobjectProperties::RunTest(const FString& Parameters)
	{
		struct Local
		{
			static void SubobjectsStay_RestoreAllChangedProperties(FAutomationTestBase& Test)
			{
				ASnapshotTestActor* Actor = nullptr;
				FSnapshotTestRunner()
					.ModifyWorld([&](UWorld* World)
					{
						Actor = ASnapshotTestActor::Spawn(World, "TestActor");
						Actor->AllocateSubobjects();

						Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty					= 1;
						Actor->EditOnlySubobject_OptionalSubobject->IntProperty							= 2;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty		= 3;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty					= 4;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty	= 5;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty				= 6;
					})
					.TakeSnapshot()
					.ModifyWorld([&](UWorld*)
					{
						Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty					= 10;
						Actor->EditOnlySubobject_OptionalSubobject->IntProperty							= 20;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty		= 30;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty					= 40;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty	= 50;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty				= 60;
					})

					// Are only the expected properties in the selection set?
					.FilterProperties(Actor, [&](const FPropertySelectionMap& SelectionMap)
					{
						Test.TestEqual(TEXT("Selection map contains only actor and subobjects"), SelectionMap.GetKeyCount(), 7);
						const UE::LevelSnapshots::FRestorableObjectSelection ActorSelection = SelectionMap.GetObjectSelection(Actor);
						if (const FPropertySelection* ActorProperties = ActorSelection.GetPropertySelection())
						{
							Test.TestEqual(TEXT("Only subobject properties are selected"), ActorProperties->GetSelectedProperties().Num(), 6);
						
							const FProperty* EditableInstancedSubobject = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobject_DefaultSubobject));
							const FProperty* EditOnlySubobject = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobject_OptionalSubobject));
							const FProperty* EditableInstancedSubobjectArray = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobjectArray_OptionalSubobject));
							const FProperty* EditOnlySubobjectArray = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobjectArray_OptionalSubobject));
							const FProperty* EditableInstancedSubobjectMap = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobjectMap_OptionalSubobject));
							const FProperty* EditOnlySubobjectMap = ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobjectMap_OptionalSubobject));
						
							Test.TestTrue(TEXT("EditableInstancedSubobject property contained"), ActorProperties->IsPropertySelected(nullptr, EditableInstancedSubobject));
							Test.TestTrue(TEXT("EditOnlySubobject property contained"), ActorProperties->IsPropertySelected(nullptr, EditOnlySubobject));
							Test.TestTrue(TEXT("EditableInstancedSubobjectArray property contained"), ActorProperties->IsPropertySelected(nullptr, EditableInstancedSubobjectArray));
							Test.TestTrue(TEXT("EditOnlySubobjectArray property contained"), ActorProperties->IsPropertySelected(nullptr, EditOnlySubobjectArray));
							Test.TestTrue(TEXT("EditableInstancedSubobjectMap property contained"), ActorProperties->IsPropertySelected(nullptr, EditableInstancedSubobjectMap));
							Test.TestTrue(TEXT("EditOnlySubobjectMap property contained"), ActorProperties->IsPropertySelected(nullptr, EditOnlySubobjectMap));
						}
						else
						{
							Test.AddError(TEXT("Missing actor property selection"));
						}

						// Does every changed subobject show up with the changed property?
						const FProperty* IntProperty = USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty));
						const TArray<UE::LevelSnapshots::FRestorableObjectSelection> ChangedSubobjects = {
							SelectionMap.GetObjectSelection(Actor->EditableInstancedSubobject_DefaultSubobject),
							SelectionMap.GetObjectSelection(Actor->EditOnlySubobject_OptionalSubobject),
							SelectionMap.GetObjectSelection(Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]),
							SelectionMap.GetObjectSelection(Actor->EditOnlySubobjectArray_OptionalSubobject[0]),
							SelectionMap.GetObjectSelection(Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]),
							SelectionMap.GetObjectSelection(Actor->EditOnlySubobjectMap_OptionalSubobject["First"])
						};
						for (const UE::LevelSnapshots::FRestorableObjectSelection& ObjectSelection : ChangedSubobjects)
						{
							if (const FPropertySelection* PropertySelection = ObjectSelection.GetPropertySelection())
							{
								Test.TestTrue(TEXT("Contains int property"), PropertySelection->IsPropertySelected(nullptr, IntProperty));
								Test.TestEqual(TEXT("No other properties selected"), PropertySelection->GetSelectedProperties().Num(), 1);
							}
							else
							{
								Test.AddError(TEXT("Missing subobject property selection"));
							}
						}

						// Special case for subobjects: for collections, unchanged subobjects must show up with an empty set (see SnapshotUtil::Object::ResolveObjectDependencyForEditorWorld)
						const TArray<UE::LevelSnapshots::FRestorableObjectSelection> UnchangedSubobjects = {
							SelectionMap.GetObjectSelection(Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]),
							SelectionMap.GetObjectSelection(Actor->EditOnlySubobjectArray_OptionalSubobject[1]),
							SelectionMap.GetObjectSelection(Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]),
							SelectionMap.GetObjectSelection(Actor->EditOnlySubobjectMap_OptionalSubobject["Second"])
						};
						for (const UE::LevelSnapshots::FRestorableObjectSelection& ObjectSelection : UnchangedSubobjects)
						{
							if (const FPropertySelection* PropertySelection = ObjectSelection.GetPropertySelection())
							{
								Test.TestTrue(TEXT("Unchanged subobject contains no properties"), PropertySelection->IsEmpty());
							}
						}
					})

					// Were the subobjects correctly restored?
					.ApplySnapshot()
					.RunTest([&]()
					{
						Test.TestEqual(TEXT("Resolved UPROPERTY(EditAnywhere, Instanced) UObject*"), Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty,							1);
						Test.TestEqual(TEXT("Resolved UPROPERTY(EditAnywhere) UObject*"), Actor->EditOnlySubobject_OptionalSubobject->IntProperty,												2);
						Test.TestEqual(TEXT("Resolved UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty,			3);
						Test.TestEqual(TEXT("Resolved UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty,								4);
						Test.TestEqual(TEXT("Resolved UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty,	5);
						Test.TestEqual(TEXT("Resolved UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty,						6);
					});
			}

			static void SubobjectsStay_RestoreOnlyIntProperties(FAutomationTestBase& Test)
			{
				ASnapshotTestActor* Actor = nullptr;
			
				FSnapshotTestRunner()
					.ModifyWorld([&](UWorld* World)
					{
						Actor = ASnapshotTestActor::Spawn(World, "TestActor");
						Actor->AllocateSubobjects();

						Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty					= 1;
						Actor->EditableInstancedSubobject_DefaultSubobject->FloatProperty				= 1.f;
						Actor->EditOnlySubobject_OptionalSubobject->IntProperty							= 2;
						Actor->EditOnlySubobject_OptionalSubobject->FloatProperty							= 2.f;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty			= 3;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->FloatProperty		= 3.f;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]->IntProperty			= 3;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]->FloatProperty		= 3.f;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty					= 4;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->FloatProperty					= 4.f;
						Actor->EditOnlySubobjectArray_OptionalSubobject[1]->IntProperty					= 4;
						Actor->EditOnlySubobjectArray_OptionalSubobject[1]->FloatProperty					= 4.f;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty		= 5;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->FloatProperty	= 5.f;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]->IntProperty		= 5;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]->FloatProperty	= 5.f;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty				= 6;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->FloatProperty				= 6.f;
						Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]->IntProperty				= 6;
						Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]->FloatProperty			= 6.f;
					})
					.TakeSnapshot()
					.ModifyWorld([&](UWorld* World)
					{
						Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty					= 10;
						Actor->EditableInstancedSubobject_DefaultSubobject->FloatProperty				= 10.f;
						Actor->EditOnlySubobject_OptionalSubobject->IntProperty							= 20;
						Actor->EditOnlySubobject_OptionalSubobject->FloatProperty							= 20.f;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty			= 30;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->FloatProperty		= 30.f;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]->IntProperty			= 30;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]->FloatProperty		= 30.f;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty					= 40;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->FloatProperty					= 40.f;
						Actor->EditOnlySubobjectArray_OptionalSubobject[1]->IntProperty					= 40;
						Actor->EditOnlySubobjectArray_OptionalSubobject[1]->FloatProperty					= 40.f;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty		= 50;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->FloatProperty	= 50.f;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]->IntProperty		= 50;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]->FloatProperty	= 50.f;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty				= 60;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->FloatProperty				= 60.f;
						Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]->IntProperty				= 60;
						Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]->FloatProperty			= 60.f;
					})

					// Only restore IntProperty. Leave FloatProperty modified.
					.ApplySnapshot([&]()
					{
						FPropertySelectionMap SelectionMap;

						SelectionMap.AddObjectProperties(Actor->EditableInstancedSubobject_DefaultSubobject,					{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditOnlySubobject_OptionalSubobject,							{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditableInstancedSubobjectArray_OptionalSubobject[0],			{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditOnlySubobjectArray_OptionalSubobject[0],					{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"],		{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditOnlySubobjectMap_OptionalSubobject["First"],				{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor,
							{
								{
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobject_DefaultSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobject_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobjectArray_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobjectArray_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobjectMap_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobjectMap_OptionalSubobject))
								}
							});
						SelectionMap.MarkSubobjectForRestoringReferencesButSkipProperties(Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]);
						SelectionMap.MarkSubobjectForRestoringReferencesButSkipProperties(Actor->EditOnlySubobjectArray_OptionalSubobject[1]);
						SelectionMap.MarkSubobjectForRestoringReferencesButSkipProperties(Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]);
						SelectionMap.MarkSubobjectForRestoringReferencesButSkipProperties(Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]);
					
						return SelectionMap;
					})

					// Were only IntProperties restored?
					.RunTest([&]()
					{
						Test.TestEqual(TEXT("Resolved: restored IntProperty on UPROPERTY(EditAnywhere, Instanced) UObject*"), Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty,								1);
						Test.TestEqual(TEXT("Resolved: skipped FloatProperty on UPROPERTY(EditAnywhere, Instanced) UObject*"), Actor->EditableInstancedSubobject_DefaultSubobject->FloatProperty,								10.f);
					
						Test.TestEqual(TEXT("Resolved: restored IntProperty on UPROPERTY(EditAnywhere) UObject*"), Actor->EditOnlySubobject_OptionalSubobject->IntProperty,													2);
						Test.TestEqual(TEXT("Resolved: skipped FloatProperty on UPROPERTY(EditAnywhere) UObject*"), Actor->EditOnlySubobject_OptionalSubobject->FloatProperty,													20.f);
					
						Test.TestEqual(TEXT("Resolved: restored IntProperty on UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty,				3);
						Test.TestEqual(TEXT("Resolved: skipped FloatProperty on UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->FloatProperty,				30.f);
						Test.TestEqual(TEXT("Resolved: skipped int on unselected object on UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]->IntProperty,					30);
						Test.TestEqual(TEXT("Resolved: skipped float on unselected object FloatProperty on UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[1]->FloatProperty,	30.f);
					
						Test.TestEqual(TEXT("Resolved: restored IntProperty on UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty,									4);
						Test.TestEqual(TEXT("Resolved: skipped FloatProperty on UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->FloatProperty,									40.f);
						Test.TestEqual(TEXT("Resolved: skipped int on unselected object on UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[1]->IntProperty,						40);
						Test.TestEqual(TEXT("Resolved: skipped float on unselected object FloatProperty on UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[1]->FloatProperty,		40.f);
					
						Test.TestEqual(TEXT("Resolved: restored IntProperty on UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty,		5);
						Test.TestEqual(TEXT("Resolved: skipped FloatProperty on UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->FloatProperty,		50.f);
						Test.TestEqual(TEXT("Resolved: skipped int on unselected object on UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]->IntProperty,			50);
						Test.TestEqual(TEXT("Resolved: skipped float on unselected object on UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["Second"]->FloatProperty,		50.f);
					
						Test.TestEqual(TEXT("Resolved: restored IntProperty on UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty,							6);
						Test.TestEqual(TEXT("Resolved: skipped FloatProperty on UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->FloatProperty,							60.f);
						Test.TestEqual(TEXT("Resolved: skipped int on unselected object on UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]->IntProperty,								60);
						Test.TestEqual(TEXT("Resolved: skipped float on unselected object on UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["Second"]->FloatProperty,							60.f);
					});
			}

			static void DestroySubobjects_RestoreAllProperties(FAutomationTestBase& Test)
			{
				ASnapshotTestActor* Actor = nullptr;
				UObject* EditableInstancedSubobject_Destroyed = nullptr;
				UObject* EditOnlySubobject_Destroyed = nullptr;
				UObject* EditableInstancedSubobjectArray_Destroyed = nullptr;
				UObject* EditOnlySubobjectArray_Destroyed = nullptr;
				UObject* EditableInstancedSubobjectMap_Destroyed = nullptr;
				UObject* EditOnlySubobjectMap_Destroyed = nullptr;
			
				FSnapshotTestRunner()
					.ModifyWorld([&](UWorld* World)
					{
						Actor = ASnapshotTestActor::Spawn(World, "TestActor");
						Actor->AllocateSubobjects();

						Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty				= 1;
						Actor->EditOnlySubobject_OptionalSubobject->IntProperty						= 2;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty		= 3;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty				= 4;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty	= 5;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty			= 6;
					})
					.TakeSnapshot()

					// "Destroy" subobjects but keep them allocated...
					.ModifyWorld([&](UWorld* World)
					{
						Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty				= 10;
						Actor->EditOnlySubobject_OptionalSubobject->IntProperty						= 20;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty		= 30;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty				= 40;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty	= 50;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty			= 60;

						Actor->EditableInstancedSubobject_DefaultSubobject->MarkAsGarbage();
						Actor->EditOnlySubobject_OptionalSubobject->MarkAsGarbage();
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->MarkAsGarbage();
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->MarkAsGarbage();
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->MarkAsGarbage();
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->MarkAsGarbage();

						EditableInstancedSubobject_Destroyed		= Actor->EditableInstancedSubobject_DefaultSubobject;
						EditOnlySubobject_Destroyed					= Actor->EditOnlySubobject_OptionalSubobject;
						EditableInstancedSubobjectArray_Destroyed	= Actor->EditableInstancedSubobjectArray_OptionalSubobject[0];
						EditOnlySubobjectArray_Destroyed			= Actor->EditOnlySubobjectArray_OptionalSubobject[0];
						EditableInstancedSubobjectMap_Destroyed		= Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"];
						EditOnlySubobjectMap_Destroyed				= Actor->EditOnlySubobjectMap_OptionalSubobject["First"];
					
						Actor->EditableInstancedSubobject_DefaultSubobject				= nullptr;
						Actor->EditOnlySubobject_OptionalSubobject						= nullptr;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]		= nullptr;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]				= nullptr;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]	= nullptr;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]			= nullptr;
					})
					.ApplySnapshot()

					// 
					.RunTest([&]()
					{
						const bool bRestoredAllObjects = Actor->EditableInstancedSubobject_DefaultSubobject && Actor->EditOnlySubobject_OptionalSubobject && Actor->EditableInstancedSubobjectArray_OptionalSubobject[0] && Actor->EditOnlySubobjectArray_OptionalSubobject[0] && Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"] && Actor->EditOnlySubobjectMap_OptionalSubobject["First"];
						const bool bAllocatedNewInstances = EditableInstancedSubobject_Destroyed != Actor->EditableInstancedSubobject_DefaultSubobject && EditOnlySubobject_Destroyed != Actor->EditOnlySubobject_OptionalSubobject && EditableInstancedSubobjectArray_Destroyed && Actor->EditableInstancedSubobjectArray_OptionalSubobject[0] && EditOnlySubobjectArray_Destroyed != Actor->EditOnlySubobjectArray_OptionalSubobject[0] && EditableInstancedSubobjectMap_Destroyed != Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"] && EditOnlySubobjectMap_Destroyed != Actor->EditOnlySubobjectMap_OptionalSubobject["First"];
						Test.TestTrue(TEXT("All nulled references were restored"), bRestoredAllObjects);
						Test.TestTrue(TEXT("Allocated new object instances"), bAllocatedNewInstances);
						if (!bRestoredAllObjects)
						{
							return;
						}
					
						Test.TestEqual(TEXT("Recreated UPROPERTY(EditAnywhere, Instanced) UObject*"), Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty,							1);
						Test.TestEqual(TEXT("Recreated UPROPERTY(EditAnywhere) UObject*"), Actor->EditOnlySubobject_OptionalSubobject->IntProperty,												2);
						Test.TestEqual(TEXT("Recreated UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty,			3);
						Test.TestEqual(TEXT("Recreated UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty,								4);
						Test.TestEqual(TEXT("Recreated UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty,	5);
						Test.TestEqual(TEXT("Recreated UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty,						6);
					});
			}

			static void SubobjectsUnreferencedButRemainAlive_RestoreAllChangedProperties(FAutomationTestBase& Test)
			{
				ASnapshotTestActor* Actor = nullptr;
			
				FSnapshotTestRunner()
					.ModifyWorld([&](UWorld* World)
					{
						Actor = ASnapshotTestActor::Spawn(World, "TestActor");
						Actor->AllocateSubobjects();

						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty		= 1;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty				= 2;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty	= 3;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty			= 4;
					})
					.TakeSnapshot()
					.ModifyWorld([&](UWorld* World)
					{
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty		= 10;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty				= 20;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty	= 30;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty			= 40;

						Actor->EditableInstancedSubobjectArray_OptionalSubobject.Empty();
						Actor->EditOnlySubobjectArray_OptionalSubobject.Empty();
						Actor->EditableInstancedSubobjectMap_OptionalSubobject.Empty();
						Actor->EditOnlySubobjectMap_OptionalSubobject.Empty();
					})
					.ApplySnapshot()
					.RunTest([&]()
					{
						const bool bRestoredAllContainers = Actor->EditableInstancedSubobjectArray_OptionalSubobject.Num() == 2 && Actor->EditOnlySubobjectArray_OptionalSubobject.Num() == 2 && Actor->EditableInstancedSubobjectMap_OptionalSubobject.Contains("First") && Actor->EditOnlySubobjectMap_OptionalSubobject.Contains("First");
						Test.TestTrue(TEXT("All containers were restored (restore all)"), bRestoredAllContainers);
						if (!bRestoredAllContainers)
						{
							return;
						}
					
						Test.TestEqual(TEXT("Resized container UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty,			1);
						Test.TestEqual(TEXT("Resized container UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty,								2);
						Test.TestEqual(TEXT("Resized container UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty,	3);
						Test.TestEqual(TEXT("Resized container UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty,						4);
					});
			}

			static void SubobjectsUnreferencedButRemainAlive_RestoreOnlyIntProperties(FAutomationTestBase& Test)
			{
				ASnapshotTestActor* Actor = nullptr;
				FPropertySelectionMap SelectionMap;
			
				FSnapshotTestRunner()
					.ModifyWorld([&](UWorld* World)
					{
						Actor = ASnapshotTestActor::Spawn(World, "TestActor");
						Actor->AllocateSubobjects();

						Actor->ObjectReference											= Actor->EditableInstancedSubobjectArray_OptionalSubobject[0];
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty			= 1;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->FloatProperty		= 1.f;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty					= 2;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->FloatProperty					= 2.f;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty		= 3;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->FloatProperty	= 3.f;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty				= 4;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->FloatProperty				= 4.f;
					})
					.TakeSnapshot()
					.ModifyWorld([&](UWorld* World)
					{
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty			= 10;
						Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->FloatProperty		= 10.f;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty					= 20;
						Actor->EditOnlySubobjectArray_OptionalSubobject[0]->FloatProperty					= 20.f;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty		= 30;
						Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->FloatProperty	= 30.f;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty				= 40;
						Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->FloatProperty				= 40.f;

						SelectionMap.AddObjectProperties(Actor->EditableInstancedSubobjectArray_OptionalSubobject[0],		{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditOnlySubobjectArray_OptionalSubobject[0],				{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"], { USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor->EditOnlySubobjectMap_OptionalSubobject["First"],			{ USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, IntProperty)) });
						SelectionMap.AddObjectProperties(Actor,
							{
								{
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobjectArray_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobjectArray_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobjectMap_OptionalSubobject)),
									ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditOnlySubobjectMap_OptionalSubobject))
								}
							});

						Actor->EditableInstancedSubobjectArray_OptionalSubobject.Empty();
						Actor->EditOnlySubobjectArray_OptionalSubobject.Empty();
						Actor->EditableInstancedSubobjectMap_OptionalSubobject.Empty();
						Actor->EditOnlySubobjectMap_OptionalSubobject.Empty();
					})
					.ApplySnapshot([&]()
					{
						return SelectionMap;
					})
					.RunTest([&]()
					{
						const bool bRestoredAllContainers = Actor->EditableInstancedSubobjectArray_OptionalSubobject.Num() == 2 && Actor->EditOnlySubobjectArray_OptionalSubobject.Num() == 2 && Actor->EditableInstancedSubobjectMap_OptionalSubobject.Contains("First") && Actor->EditOnlySubobjectMap_OptionalSubobject.Contains("First");
						Test.TestTrue(TEXT("All containers were restored (restore only ints)"), bRestoredAllContainers);
						if (!bRestoredAllContainers)
						{
							return;
						}

						Test.TestTrue(TEXT("Resized container: ObjectReference == EditableInstancedSubobjectArray[0]"), Actor->ObjectReference == Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]);
						Test.TestEqual(TEXT("Resized container: restored IntProperty UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty,				1);
						Test.TestEqual(TEXT("Resized container: skipped FloatProperty UPROPERTY(EditAnywhere, Instanced) TArray<UObject*>"), Actor->EditableInstancedSubobjectArray_OptionalSubobject[0]->FloatProperty,			10.f);
						Test.TestEqual(TEXT("Resized container: restored IntProperty UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->IntProperty,									2);
						Test.TestEqual(TEXT("Resized container: skipped FloatProperty UPROPERTY(EditAnywhere) TArray<UObject*>"), Actor->EditOnlySubobjectArray_OptionalSubobject[0]->FloatProperty,								20.f);
						Test.TestEqual(TEXT("Resized container: restored IntProperty UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->IntProperty,		3);
						Test.TestEqual(TEXT("Resized container: skipped FloatProperty UPROPERTY(EditAnywhere, Instanced) TMap<FName,UObject*>"), Actor->EditableInstancedSubobjectMap_OptionalSubobject["First"]->FloatProperty,	30.f);
						Test.TestEqual(TEXT("Resized container: restored IntProperty UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->IntProperty,							4);
						Test.TestEqual(TEXT("Resized container: skipped FloatProperty UPROPERTY(EditAnywhere) TMap<FName,UObject*>"), Actor->EditOnlySubobjectMap_OptionalSubobject["First"]->FloatProperty,						40.f);
					});
			}
		};
	

		// Subobjects remain allocated and referenced:
		Local::SubobjectsStay_RestoreAllChangedProperties(*this);
		Local::SubobjectsStay_RestoreOnlyIntProperties(*this);
	
		// Remove subobjects and fully recreate them:
		Local::DestroySubobjects_RestoreAllProperties(*this);

		// Clear arrays and maps:
		Local::SubobjectsUnreferencedButRemainAlive_RestoreAllChangedProperties(*this);
		Local::SubobjectsUnreferencedButRemainAlive_RestoreOnlyIntProperties(*this);
	
		return true;
	}

	/**
	* Create material instance dynamic, assign it to a mesh, and compare snapshot to original:
	*  - generate no diff if materials have some properties
	*  - generate diff if a parameter is changed
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDynamicMaterialInstanceDiffCorrectly, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.DynamicMaterialInstanceDiffCorrectly", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FDynamicMaterialInstanceDiffCorrectly::RunTest(const FString& Parameters)
	{
		// TODO
		return true;
	}

	/**
	* If the softobject path saved for an object property still points to an object but it is pending kill, verify that a new instance is allocated.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPendingKillSubobjectIsReplacedWithNewInstance, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.PendingKillSubobjectIsReplacedWithNewInstance", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FPendingKillSubobjectIsReplacedWithNewInstance::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
		USubobject* Subobject = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World, "TestActor");
				Actor->AllocateSubobjects();

				Subobject = Actor->EditableInstancedSubobject_DefaultSubobject;
				Subobject->IntProperty = 1;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty = 10;
				Actor->EditableInstancedSubobject_DefaultSubobject->MarkAsGarbage();
				Actor->EditableInstancedSubobject_DefaultSubobject = nullptr;
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("Subobject reference is valid"), Actor->EditableInstancedSubobject_DefaultSubobject != nullptr);
				TestTrue(TEXT("New subobject was allocated"), Subobject != Actor->EditableInstancedSubobject_DefaultSubobject);
				TestEqual(TEXT("Subobject values restored correctly"), Actor->EditableInstancedSubobject_DefaultSubobject != nullptr && Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty, 1);
			});

		return true;
	}

	/**
	* Do nested subobjects restore correctly, i.e. a subobject of a subobject.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNestedSubobjectOuters, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.NestedSubobjectOuters", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FNestedSubobjectOuters::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* Actor = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				Actor = ASnapshotTestActor::Spawn(World, "TestActor");
				Actor->AllocateSubobjects();

				Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty = 21;
				Actor->EditableInstancedSubobject_DefaultSubobject->NestedChild->IntProperty = 42;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty = 210;
				Actor->EditableInstancedSubobject_DefaultSubobject->NestedChild->IntProperty = 420;
			
				Actor->EditableInstancedSubobject_DefaultSubobject->UneditableNestedChild = nullptr;
				Actor->EditableInstancedSubobject_DefaultSubobject->NestedChild = nullptr;
				Actor->EditableInstancedSubobject_DefaultSubobject = nullptr;;
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("Root subobject recreated"), Actor->EditableInstancedSubobject_DefaultSubobject != nullptr);
				TestTrue(TEXT("Nested editable subobject recreated"), Actor->EditableInstancedSubobject_DefaultSubobject && Actor->EditableInstancedSubobject_DefaultSubobject->NestedChild);
				TestTrue(TEXT("Nested uneditable subobject not recreated"), Actor->EditableInstancedSubobject_DefaultSubobject && Actor->EditableInstancedSubobject_DefaultSubobject->UneditableNestedChild == nullptr);
				if (Actor->EditableInstancedSubobject_DefaultSubobject && Actor->EditableInstancedSubobject_DefaultSubobject->NestedChild)
				{
					TestTrue(TEXT("Root subobject values restored"), Actor->EditableInstancedSubobject_DefaultSubobject->IntProperty == 21);
					TestTrue(TEXT("Root subobject values restored"), Actor->EditableInstancedSubobject_DefaultSubobject->NestedChild->IntProperty == 42);
				}
			});

		return true;
	}

	/**
	* Make subobject A reference subobject B and vice versa.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCircularSubobjectDependency, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.NestedSubobjectOuters", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FCircularSubobjectDependency::RunTest(const FString& Parameters)
	{
		struct Local
		{
			static void RunCircularSubobjectTest(FAutomationTestBase& Test, TFunction<void(UObject* Subobject)> SubobjectModifierFunction)
			{
				ASnapshotTestActor* ActorA = nullptr;
				ASnapshotTestActor* ActorB = nullptr;
	
				FSnapshotTestRunner()
					.ModifyWorld([&](UWorld* World)
					{
						ActorA = ASnapshotTestActor::Spawn(World, "ActorA");
						ActorA->AllocateSubobjects();
						ActorB = ASnapshotTestActor::Spawn(World, "ActorB");
						ActorB->AllocateSubobjects();

						ActorA->EditableInstancedSubobject_DefaultSubobject->IntProperty = 21;
						ActorB->EditableInstancedSubobject_DefaultSubobject->IntProperty = 42;

						ActorA->ObjectReference = ActorB->EditableInstancedSubobject_DefaultSubobject;
						ActorB->ObjectReference = ActorA->EditableInstancedSubobject_DefaultSubobject;
					})
					.TakeSnapshot()
					.ModifyWorld([&](UWorld* World)
					{
						ActorA->EditableInstancedSubobject_DefaultSubobject->IntProperty = 210;
						ActorB->EditableInstancedSubobject_DefaultSubobject->IntProperty = 420;

						SubobjectModifierFunction(ActorA->EditableInstancedSubobject_DefaultSubobject);
						SubobjectModifierFunction(ActorB->EditableInstancedSubobject_DefaultSubobject);
						SubobjectModifierFunction(ActorA->ObjectReference);
						SubobjectModifierFunction(ActorB->ObjectReference);
					
						ActorA->EditableInstancedSubobject_DefaultSubobject = nullptr;
						ActorB->EditableInstancedSubobject_DefaultSubobject = nullptr;
						ActorA->ObjectReference = nullptr;
						ActorB->ObjectReference = nullptr;
					})
					.ApplySnapshot()
					.RunTest([&]()
					{
						const bool bInstancedPropertiesWereRestored = ActorA->EditableInstancedSubobject_DefaultSubobject && ActorB->EditableInstancedSubobject_DefaultSubobject; 
						Test.TestTrue(TEXT("Subobject properties restored"), bInstancedPropertiesWereRestored);
						if (!bInstancedPropertiesWereRestored)
						{
							return;
						}
			
						Test.TestEqual(TEXT("ActorA restored"), ActorA->EditableInstancedSubobject_DefaultSubobject->IntProperty, 21);
						Test.TestEqual(TEXT("ActorB restored"), ActorB->EditableInstancedSubobject_DefaultSubobject->IntProperty, 42);

						Test.TestTrue(TEXT("ActorA points to ActorB's subobject"), ActorA->ObjectReference == ActorB->EditableInstancedSubobject_DefaultSubobject);
						Test.TestTrue(TEXT("ActorB points to ActorA's subobject"), ActorB->ObjectReference == ActorA->EditableInstancedSubobject_DefaultSubobject);
					});
			}
		};
	
		Local::RunCircularSubobjectTest(*this, [](UObject* Subobject){});
		Local::RunCircularSubobjectTest(*this, [](UObject* Subobject)
		{
			Subobject->MarkAsGarbage();
		});

		return true;
	}

	/**
	* Test that if a reference resolved to a component that is dead but still exists in memory does not end up getting referenced.
	* 
	* 1. Actor A references another actor B's component.
	* 2. Take snapshot
	* 3. Delete actor B and set actor's reference to nullptr
	* 4. Restore snapshot
	*	4.1. Recreate actor > A reference component of actor B
	*	4.2. Do not recreate actor > A reference component is nullptr (i.e. does not point to dead component which still exists in memory)
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSkipDeadComponentReferences, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.SkipDeadComponentReferences", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FSkipDeadComponentReferences::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* ActorA = nullptr;
		ASnapshotTestActor* ActorB = nullptr;

		// Recreate the acto
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ActorA = ASnapshotTestActor::Spawn(World, "ActorA");
				ActorB = ASnapshotTestActor::Spawn(World, "ActorB");

				ActorA->ObjectReference = ActorB->TestComponent;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				ActorA->ObjectReference = nullptr;
				World->DestroyActor(ActorB);
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("ObjectReference was restored"), ActorA->ObjectReference != nullptr && !ActorA->ObjectReference->IsIn(ActorA));
			});
	
		// Do not recreate actor
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ActorA = ASnapshotTestActor::Spawn(World, "ActorA");
				ActorB = ASnapshotTestActor::Spawn(World, "ActorB");

				ActorA->ObjectReference = ActorB->TestComponent;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				ActorA->ObjectReference = nullptr;
				World->DestroyActor(ActorB);
			})
			.ApplySnapshot([]()
			{
				UConstantFilter* ContantFilter = NewObject<UConstantFilter>();
				ContantFilter->IsDeletedActorValidResult = EFilterResult::Exclude;
				return ContantFilter;
			})
			.RunTest([&]()
			{
				TestTrue(TEXT("ObjectReference was not set to dead component"), ActorA->ObjectReference == nullptr);
			});

		return true;
	}

	/**
	* Example:
	*
	* class AMyActor
	* {
	*		UPROPERTY(EditAnywhere, Instanced)
	*		UMySubobject* Subobject;
	* }
	*
	* If Subobject is an unsupported class, see FSnapshotRestorability, then this property should not generate any diff.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSkippedSubobjectsDoNotDiff, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.SkippedSubobjectsDoNotDiff", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FSkippedSubobjectsDoNotDiff::RunTest(const FString& Parameters)
	{
		class FDisableSubobjectClassSupport : public ISnapshotRestorabilityOverrider
		{
		public:
			virtual ERestorabilityOverride IsComponentDesirableForCapture(const UActorComponent* Component) override
			{
				return Component->GetClass() != USnapshotTestComponent::StaticClass() ? ERestorabilityOverride::DoNotCare : ERestorabilityOverride::Disallow;
			}
		};

		TSharedRef<FDisableSubobjectClassSupport> Support = MakeShared<FDisableSubobjectClassSupport>();
		ILevelSnapshotsModule::Get().RegisterRestorabilityOverrider(Support);
		ILevelSnapshotsModule::Get().AddSkippedSubobjectClasses({ USubobject::StaticClass() });
		ON_SCOPE_EXIT
		{
			ILevelSnapshotsModule::Get().UnregisterRestorabilityOverrider(Support);
			ILevelSnapshotsModule::Get().RemoveSkippedSubobjectClasses({ USubobject::StaticClass() });
		};
	
		ASnapshotTestActor* ActorA = nullptr;
		ASnapshotTestActor* ActorB = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ActorA = ASnapshotTestActor::Spawn(World, "ActorA");
				ActorA->AllocateSubobjects();
				ActorB = ASnapshotTestActor::Spawn(World, "ActorB");
				ActorB->AllocateSubobjects();

				ActorA->EditableInstancedSubobject_DefaultSubobject->IntProperty			= 21;
				ActorA->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty	= 42;
				ActorA->TestComponent->IntProperty											= 84;

				ActorB->AddObjectReference(ActorB->EditOnlySubobject_OptionalSubobject, "MapKey");					
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				ActorA->EditableInstancedSubobject_DefaultSubobject->IntProperty			= 210;
				ActorA->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty	= 420;
				ActorA->TestComponent->IntProperty											= 840;
			
				ActorB->ClearObjectReferences();
				ActorB->EditableInstancedSubobject_DefaultSubobject				= nullptr;
				ActorB->EditableInstancedSubobjectArray_OptionalSubobject[0]	= nullptr;
				ActorB->TestComponent											= nullptr;
			})

			.FilterProperties(ActorA, [&](const FPropertySelectionMap& PropertySelection)
			{
				TestEqual(TEXT("Actor A has no diffed properties"), PropertySelection.GetKeyCount(), 0);
			})

			// Soft paths properties are a special case: we treat them as strings:
				// - Check that soft object properties DO show up as changed
				// - Check that all other object properties do not show up as changed
			.FilterProperties(ActorB, [&](const FPropertySelectionMap& PropertySelection)
			{
				TestEqual(TEXT("Actor B only has soft references as diffed properties"), PropertySelection.GetKeyCount(), 1);
			
				const FPropertySelection* ActorSelection = PropertySelection.GetObjectSelection(ActorB).GetPropertySelection();
				if (!ActorSelection)
				{
					AddError(TEXT("Missing property selection for ActorB"));
					return;
				}

				// Expected to have changed because soft object pointers count as strings
				TArray<FProperty*> SoftObjectProperties = {
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftPath)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftPathArray)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftPathSet)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftPathMap)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftObjectPtr)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftObjectPtrArray)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftObjectPtrSet)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, SoftObjectPtrMap)),
				};
				for (FProperty* Property : SoftObjectProperties)
				{
					TestTrue(TEXT("Soft object property"), ActorSelection->IsPropertySelected(nullptr, Property));
				}
			
				// See AccessSnapshot below for why
				TArray<FProperty*> EmptySnapshotCollections = {
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, ObjectArray)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, ObjectSet)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, ObjectMap)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, WeakObjectPtrArray)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, WeakObjectPtrSet)),
					ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, WeakObjectPtrMap)),
				};
				for (FProperty* Property : EmptySnapshotCollections)
				{
					TestTrue(TEXT("Empty snapshot collection"), ActorSelection->IsPropertySelected(nullptr, Property));
				}

				// + 2 because FSoftObjectPath has 4 properties; they are implicitly added because ASnapshotTestActor::SoftPath is technically a struct
				const int32 ExpectedNumProperties = SoftObjectProperties.Num() + 4 + EmptySnapshotCollections.Num();
				const TArray<FLevelSnapshotPropertyChain>& SelectedProperties = ActorSelection->GetSelectedProperties();
				TestEqual(TEXT("ActorB only has soft object paths in selection set"), SelectedProperties.Num(), ExpectedNumProperties);
			})

			// Because USubobject is skipped, the collections should contain null
			// However, the original does not contain any objects because we cleared it above
			// Since they collections are non-equal sizes, they're expected to diff.
			.AccessSnapshot([&](ULevelSnapshot* Snapshot)
			{
				const TOptional<TNonNullPtr<AActor>> OptionalActorB_Snapshot = Snapshot->GetDeserializedActor(ActorB);
				if (!ensure(OptionalActorB_Snapshot))
				{
					return;
				}
			
				ASnapshotTestActor* ActorB_Snapshot = Cast<ASnapshotTestActor>(OptionalActorB_Snapshot->Get());
				TestTrue(TEXT("ObjectArray contains null"), ActorB_Snapshot->ObjectArray.Num() == 1 && ActorB_Snapshot->ObjectArray[0] == nullptr);
				TestTrue(TEXT("ObjectSet contains null"), ActorB_Snapshot->ObjectSet.Num() == 1 && *ActorB_Snapshot->ObjectSet.FindArbitraryElement() == nullptr);
				TestTrue(TEXT("ObjectMap contains null"), ActorB_Snapshot->ObjectMap.Num() == 1 && ActorB_Snapshot->ObjectMap.Find("MapKey") && ActorB_Snapshot->ObjectMap["MapKey"] == nullptr);
				TestTrue(TEXT("WeakObjectPtrArray contains null"), ActorB_Snapshot->WeakObjectPtrArray.Num() == 1 && ActorB_Snapshot->WeakObjectPtrArray[0] == nullptr);
				TestTrue(TEXT("WeakObjectPtrSet contains null"), ActorB_Snapshot->WeakObjectPtrSet.Num() == 1 && *ActorB_Snapshot->WeakObjectPtrSet.FindArbitraryElement() == nullptr);
				TestTrue(TEXT("WeakObjectPtrMap contains null"), ActorB_Snapshot->WeakObjectPtrMap.Num() == 1 && ActorB_Snapshot->WeakObjectPtrMap.Find("MapKey") && ActorB_Snapshot->WeakObjectPtrMap["MapKey"] == nullptr);
			})

			// Check whether no skipped subobjects were restored
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestEqual(TEXT("Simple reference property not restored"), ActorA->EditableInstancedSubobject_DefaultSubobject->IntProperty, 210);
				TestEqual(TEXT("Array reference property not restored"), ActorA->EditableInstancedSubobjectArray_OptionalSubobject[0]->IntProperty, 420);
				TestEqual(TEXT("Component reference property not restored"), ActorA->TestComponent->IntProperty, 840);

				TestTrue(TEXT("All object references stay null"), !ActorB->HasAnyValidHardObjectReference());
				TestTrue(TEXT("Simple reference property stays null"), ActorB->EditableInstancedSubobject_DefaultSubobject == nullptr);
				TestTrue(TEXT("Array reference property stays null"), ActorB->EditableInstancedSubobjectArray_OptionalSubobject[0] == nullptr);
				TestTrue(TEXT("Component reference property stays null"), ActorB->TestComponent == nullptr);
			});

		return true;
	}

	/**
	* Things like changing array / set / map element order, such as { A, null } to { null, A }, is detected and restored correctly.
	*
	* Differs from FReorderReferenceCollections because different runs for subobjects.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FReorderSubobjectCollections, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.ReorderSubobjectCollections", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FReorderSubobjectCollections::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* SwapElementOrder = nullptr;
		ASnapshotTestActor* SwapElementWithNull = nullptr;
		ASnapshotTestActor* ReplaceElementWithNull = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				FActorSpawnParameters SwapElementOrderParams;
			
			
				SwapElementOrder = ASnapshotTestActor::Spawn(World, "SwapElementOrder");
				SwapElementWithNull = ASnapshotTestActor::Spawn(World, "SwapElementWithNull");
				ReplaceElementWithNull = ASnapshotTestActor::Spawn(World, "ReplaceElementWithNull");

				{
					USubobject* First = NewObject<USubobject>(SwapElementOrder, USubobject::StaticClass(), TEXT("First"));
					USubobject* Second = NewObject<USubobject>(SwapElementOrder, USubobject::StaticClass(), TEXT("Second"));
					First->IntProperty = 1;
					Second->IntProperty = 2;
				
					SwapElementOrder->ObjectArray.Add(First);
					SwapElementOrder->ObjectArray.Add(Second);
					SwapElementOrder->ObjectMap.Add("First", First);
					SwapElementOrder->ObjectMap.Add("Second", Second);
				}
			
				{
					USubobject* First = NewObject<USubobject>(SwapElementWithNull, USubobject::StaticClass(), TEXT("First"));
					USubobject* Third = NewObject<USubobject>(SwapElementWithNull, USubobject::StaticClass(), TEXT("Third"));
					First->IntProperty = 10;
					Third->IntProperty = 30;

					SwapElementWithNull->ObjectArray.Add(First);
					SwapElementWithNull->ObjectArray.Add(nullptr);
					SwapElementWithNull->ObjectArray.Add(Third);
				}
			
				{
					USubobject* First = NewObject<USubobject>(ReplaceElementWithNull, USubobject::StaticClass(), TEXT("First"));
					USubobject* Second = NewObject<USubobject>(ReplaceElementWithNull, USubobject::StaticClass(), TEXT("Second"));
					USubobject* Third = NewObject<USubobject>(ReplaceElementWithNull, USubobject::StaticClass(), TEXT("Third"));
					First->IntProperty = 100;
					Second->IntProperty = 200;
					Third->IntProperty = 300;
				
					ReplaceElementWithNull->ObjectArray.Add(First);
					ReplaceElementWithNull->ObjectArray.Add(Second);
					ReplaceElementWithNull->ObjectArray.Add(Third); 
				}
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				SwapElementOrder->ObjectMap["Second"] = SwapElementOrder->ObjectArray[0];
				SwapElementOrder->ObjectMap["First"] = SwapElementOrder->ObjectArray[1];
				UObject* Temp = SwapElementOrder->ObjectArray[0];
				SwapElementOrder->ObjectArray[0] = SwapElementOrder->ObjectArray[1];
				SwapElementOrder->ObjectArray[1] = Temp;

				SwapElementWithNull->ObjectArray[1] = SwapElementWithNull->ObjectArray[0];
				SwapElementWithNull->ObjectArray[0] = nullptr;
			
				ReplaceElementWithNull->ObjectArray[1] = nullptr;
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				TestTrue(TEXT("SwapElementOrder[0]"), SwapElementOrder->ObjectArray[0]->GetName() == TEXT("First"));
				TestTrue(TEXT("SwapElementOrder[1]"), SwapElementOrder->ObjectArray[1]->GetName() == TEXT("Second"));
				TestTrue(TEXT("SwapElementOrder[First]"), SwapElementOrder->ObjectMap["First"]->GetName() == TEXT("First"));
				TestTrue(TEXT("SwapElementOrder[Second]"), SwapElementOrder->ObjectMap["Second"]->GetName() == TEXT("Second"));
			
				TestTrue(TEXT("SwapElementWithNull[0]"), SwapElementWithNull->ObjectArray[0] && SwapElementWithNull->ObjectArray[0]->GetName() == TEXT("First"));
				TestTrue(TEXT("SwapElementWithNull[1]"), SwapElementWithNull->ObjectArray[1] == nullptr);
				TestTrue(TEXT("SwapElementWithNull[2]"), SwapElementWithNull->ObjectArray[2] && SwapElementWithNull->ObjectArray[2]->GetName() == TEXT("Third"));
			
				TestTrue(TEXT("ReplaceElementWithNull[0]"), ReplaceElementWithNull->ObjectArray[0] && ReplaceElementWithNull->ObjectArray[0]->GetName() == TEXT("First"));
				TestTrue(TEXT("ReplaceElementWithNull[1]"), ReplaceElementWithNull->ObjectArray[1] && ReplaceElementWithNull->ObjectArray[1]->GetName() == TEXT("Second"));
				TestTrue(TEXT("ReplaceElementWithNull[2]"), ReplaceElementWithNull->ObjectArray[2] && ReplaceElementWithNull->ObjectArray[2]->GetName() == TEXT("Third"));
			});

		return true;
	}

	/**
	* Actors referencing other actor's subobjects restore correctly.
	*/
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FReferenceSubobjectsFromOtherActors, "VirtualProduction.LevelSnapshots.Snapshot.Subobject.ReferenceSubobjectsFromOtherActors", (EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter));
	bool FReferenceSubobjectsFromOtherActors::RunTest(const FString& Parameters)
	{
		ASnapshotTestActor* ReferenceExternalSubobjects = nullptr;
		ASnapshotTestActor* ReferencedObject = nullptr;
	
		USubobject* DestroyedReferencedSubobject = nullptr;
		USubobject* DestroyedUnreferencedSubobject = nullptr;
	
		FSnapshotTestRunner()
			.ModifyWorld([&](UWorld* World)
			{
				ReferenceExternalSubobjects = ASnapshotTestActor::Spawn(World, "ReferenceExternalSubobjects");
				ReferenceExternalSubobjects->AllocateSubobjects();
			
				ReferencedObject = ASnapshotTestActor::Spawn(World, "ReferencedObject");
				ReferencedObject->AllocateSubobjects();
				DestroyedReferencedSubobject = NewObject<USubobject>(ReferencedObject, USubobject::StaticClass(), TEXT("DestroyedReferencedSubobject"));
				DestroyedReferencedSubobject->FloatProperty = 42.f;
				DestroyedUnreferencedSubobject = NewObject<USubobject>(ReferencedObject, USubobject::StaticClass(), TEXT("DestroyedUnreferencedSubobject"));
				DestroyedUnreferencedSubobject->FloatProperty = 21.f;
			
				ReferenceExternalSubobjects->ObjectArray.Add(ReferencedObject->EditableInstancedSubobject_DefaultSubobject);
				ReferenceExternalSubobjects->ObjectArray.Add(ReferencedObject->EditOnlySubobject_OptionalSubobject);
				ReferenceExternalSubobjects->ObjectArray.Add(DestroyedReferencedSubobject);
				ReferenceExternalSubobjects->ObjectArray.Add(DestroyedUnreferencedSubobject);
			
				ReferenceExternalSubobjects->ObjectSet.Add(ReferencedObject->EditableInstancedSubobject_DefaultSubobject);
				ReferenceExternalSubobjects->ObjectSet.Add(ReferencedObject->EditOnlySubobject_OptionalSubobject);
			
				ReferenceExternalSubobjects->ObjectMap.Add("EditableInstancedSubobject", ReferencedObject->EditableInstancedSubobject_DefaultSubobject);
				ReferenceExternalSubobjects->ObjectMap.Add("EditOnlySubobject", ReferencedObject->EditOnlySubobject_OptionalSubobject);
				ReferenceExternalSubobjects->ObjectMap.Add("DestroyedReferencedSubobject", DestroyedReferencedSubobject);
				ReferenceExternalSubobjects->ObjectMap.Add("DestroyedUnreferencedSubobject", DestroyedUnreferencedSubobject);

				ReferenceExternalSubobjects->TestComponent = ReferencedObject->TestComponent;
			
			
				ReferencedObject->ObjectReference = DestroyedReferencedSubobject;
			})
			.TakeSnapshot()
			.ModifyWorld([&](UWorld* World)
			{
				DestroyedReferencedSubobject->MarkAsGarbage();
				DestroyedUnreferencedSubobject->MarkAsGarbage();
			
				ReferenceExternalSubobjects->ObjectArray.Empty();
				ReferenceExternalSubobjects->ObjectSet.Empty();
				ReferenceExternalSubobjects->ObjectMap.Empty();
				ReferenceExternalSubobjects->TestComponent = ReferenceExternalSubobjects->FindComponentByClass<USnapshotTestComponent>();
			})
			.ApplySnapshot()
			.RunTest([&]()
			{
				if (ensure(ReferenceExternalSubobjects->ObjectArray.Num() == 4))
				{
					const bool bIsDestroyedReferencedSubobjectValid = ReferenceExternalSubobjects->ObjectArray[2]
						&& ReferenceExternalSubobjects->ObjectArray[2]->GetOuter() == ReferencedObject
						&& ReferenceExternalSubobjects->ObjectArray[2]->GetName() == TEXT("DestroyedReferencedSubobject")
						&& Cast<USubobject>(ReferenceExternalSubobjects->ObjectArray[2])->FloatProperty == 42.f;
					const bool bDestroyedUnreferencedSubobjectValid = ReferenceExternalSubobjects->ObjectArray[3]
						&& ReferenceExternalSubobjects->ObjectArray[3]->GetOuter() == ReferencedObject
						&& ReferenceExternalSubobjects->ObjectArray[3]->GetName() == TEXT("DestroyedUnreferencedSubobject")
						&& Cast<USubobject>(ReferenceExternalSubobjects->ObjectArray[3])->FloatProperty == 21.f;;
				
					TestTrue(TEXT("Array: EditableInstancedSubobject"), ReferenceExternalSubobjects->ObjectArray[0] == ReferencedObject->EditableInstancedSubobject_DefaultSubobject);
					TestTrue(TEXT("Array: EditOnlySubobject"), ReferenceExternalSubobjects->ObjectArray[1] == ReferencedObject->EditOnlySubobject_OptionalSubobject);
					TestTrue(TEXT("Array: DestroyedReferencedSubobject"), bIsDestroyedReferencedSubobjectValid);
					TestTrue(TEXT("Array: DestroyedUnreferencedSubobject"), bDestroyedUnreferencedSubobjectValid);
				}

				if (ensure(ReferenceExternalSubobjects->ObjectSet.Num() == 2))
				{
					TestTrue(TEXT("Set: EditableInstancedSubobject"), ReferenceExternalSubobjects->ObjectSet.Contains(ReferencedObject->EditableInstancedSubobject_DefaultSubobject));
					TestTrue(TEXT("Set: EditOnlySubobject"), ReferenceExternalSubobjects->ObjectArray.Contains(ReferencedObject->EditOnlySubobject_OptionalSubobject));
				}

				if (ensure(ReferenceExternalSubobjects->ObjectMap.Num() == 4))
				{
					UObject* EditableInstancedSubobject = ReferenceExternalSubobjects->ObjectMap.Find("EditableInstancedSubobject") ? *ReferenceExternalSubobjects->ObjectMap.Find("EditableInstancedSubobject") : nullptr;
					UObject* EditOnlySubobject = ReferenceExternalSubobjects->ObjectMap.Find("EditOnlySubobject")  ? *ReferenceExternalSubobjects->ObjectMap.Find("EditOnlySubobject")  : nullptr;
					UObject* DestroyedReferencedSubobject = ReferenceExternalSubobjects->ObjectMap.Find("DestroyedReferencedSubobject") ? *ReferenceExternalSubobjects->ObjectMap.Find("DestroyedReferencedSubobject") : nullptr;
					UObject* DestroyedUnreferencedSubobject = ReferenceExternalSubobjects->ObjectMap.Find("DestroyedUnreferencedSubobject") ? *ReferenceExternalSubobjects->ObjectMap.Find("DestroyedUnreferencedSubobject") : nullptr;

					const bool bIsDestroyedReferencedSubobjectValid = DestroyedReferencedSubobject
						&& DestroyedReferencedSubobject->GetOuter() == ReferencedObject 
						&& DestroyedReferencedSubobject->GetName() == TEXT("DestroyedReferencedSubobject")
						&& Cast<USubobject>(DestroyedReferencedSubobject)->FloatProperty == 42.f;
					const bool bIsDestroyedUnreferencedSubobjectValid = DestroyedUnreferencedSubobject
						&& DestroyedUnreferencedSubobject->GetOuter() == ReferencedObject
						&& DestroyedUnreferencedSubobject->GetName() == TEXT("DestroyedUnreferencedSubobject")
						&& Cast<USubobject>(DestroyedUnreferencedSubobject)->FloatProperty == 21.f;
				
					TestTrue(TEXT("Map: EditableInstancedSubobject"), EditableInstancedSubobject == ReferencedObject->EditableInstancedSubobject_DefaultSubobject);
					TestTrue(TEXT("Map: EditOnlySubobject"), EditOnlySubobject == ReferencedObject->EditOnlySubobject_OptionalSubobject);
					TestTrue(TEXT("Map: DestroyedReferencedSubobject"), bIsDestroyedReferencedSubobjectValid);
					TestTrue(TEXT("Map: DestroyedUnreferencedSubobject"), bIsDestroyedUnreferencedSubobjectValid);
				}
			
				TestTrue(TEXT("Component reference was restored"), ReferenceExternalSubobjects->TestComponent == ReferencedObject->TestComponent);
			});

		return true;
	}
}