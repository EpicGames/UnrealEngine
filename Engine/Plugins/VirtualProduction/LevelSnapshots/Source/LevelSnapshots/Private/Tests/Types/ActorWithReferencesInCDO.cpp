// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorWithReferencesInCDO.h"

#include "UObject/ConstructorHelpers.h"

AActorWithReferencesInCDO::AActorWithReferencesInCDO()
{
	ConstructorHelpers::FObjectFinder<UStaticMesh> CubeFinder(TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'"), LOAD_Quiet | LOAD_NoWarn);
	if (CubeFinder.Succeeded())
	{
		CubeMesh = CubeFinder.Object;
	}
	ConstructorHelpers::FObjectFinder<UStaticMesh> CylinderFinder(TEXT("StaticMesh'/Engine/BasicShapes/Cylinder.Cylinder'"), LOAD_Quiet | LOAD_NoWarn);
	if (CylinderFinder.Succeeded())
	{
		CylinderMesh = CylinderFinder.Object;
	}

	SetAllPropertiesTo(CubeMesh);
}

void AActorWithReferencesInCDO::SetAllPropertiesTo(UObject* Object)
{
	Array.Empty();
	Set.Empty();
	IntKeyMap.Empty();
	IntValueMap.Empty();

	Array.Add(Object);
	Set.Add(Object);
	IntKeyMap.Add(42, Object);
	IntValueMap.Add(Object, 42);
}

bool AActorWithReferencesInCDO::DoAllPropertiesPointTo(UObject* TestReference)
{
	return Array.Num() == 1
		&& Set.Num() == 1
		&& IntKeyMap.Num() == 1
		&& IntValueMap.Num() == 1
		&& Array[0].Object == TestReference
		&& Set.Contains(TestReference)
		&& IntKeyMap.Contains(42) && IntKeyMap.Find(42)->Object == TestReference
		&& IntValueMap.Contains(TestReference) && *IntValueMap.Find(TestReference) == 42;
}
