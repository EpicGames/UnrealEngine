// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "ActorWithReferencesInCDO.generated.h"

class UStaticMesh;

USTRUCT()
struct FExternalReferenceDummy
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UObject> Object;

	friend uint32 GetTypeHash(const FExternalReferenceDummy& Struct)
	{
		return ::PointerHash(Struct.Object);
	}

	friend bool operator==(const FExternalReferenceDummy& Lhs, const FExternalReferenceDummy& RHS)
	{
		return Lhs.Object == RHS.Object;
	}

	FExternalReferenceDummy(UObject* Object = nullptr)
		:
		Object(Object)
	{}
};

UCLASS()
class AActorWithReferencesInCDO : public AActor
{
	GENERATED_BODY()
public:

	AActorWithReferencesInCDO();

	void SetAllPropertiesTo(UObject* Object);
	bool DoAllPropertiesPointTo(UObject* TestReference);

	/******************** Properties  ********************/

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<FExternalReferenceDummy> Array;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSet<FExternalReferenceDummy> Set;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<int32, FExternalReferenceDummy> IntKeyMap;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<FExternalReferenceDummy, int32> IntValueMap;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	FExternalReferenceDummy Struct;
	
	
	/******************** External references  ********************/
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UStaticMesh> CubeMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UStaticMesh> CylinderMesh = nullptr;
};
