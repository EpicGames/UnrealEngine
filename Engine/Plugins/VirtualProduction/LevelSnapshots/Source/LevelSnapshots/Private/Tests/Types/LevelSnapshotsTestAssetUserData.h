// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "LevelSnapshotsTestAssetUserData.generated.h"

// These classes do nothing special. See UActorComponent::AddAssetUserData: it only allows one instance per unique class... this is why we must create a few additional classes.

UCLASS()
class LEVELSNAPSHOTS_API ULevelSnapshotsTestAssetUserData_Persistent : public UAssetUserData
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 Value = 0;
};

UCLASS()
class LEVELSNAPSHOTS_API ULevelSnapshotsTestAssetUserData_MarkedTransient : public UAssetUserData
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 Value = 0;
};

UCLASS()
class LEVELSNAPSHOTS_API ULevelSnapshotsTestAssetUserData_TransientPackage : public UAssetUserData
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 Value = 0;
};
