// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotTestActor.h"

#include "Engine/World.h"
#include "UObject/ConstructorHelpers.h"

USubobject::USubobject()
{
	NestedChild = CreateDefaultSubobject<USubSubobject>(TEXT("NestedChild"));
	UneditableNestedChild = CreateDefaultSubobject<USubSubobject>(TEXT("UneditableNestedChild"));;
}

USnapshotTestComponent::USnapshotTestComponent()
{
	Subobject = CreateDefaultSubobject<USubobject>(TEXT("Subobject"));
}

ASnapshotTestActor* ASnapshotTestActor::Spawn(UWorld* World, FName Name)
{
	FActorSpawnParameters Params;
	Params.Name = Name;
	Params.bNoFail = true;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	Params.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
	ASnapshotTestActor* Result = World->SpawnActor<ASnapshotTestActor>(Params);

	// Editor code sets an actor's label after adding it to the world. Note that GetActorLabel sets a label (see default param bCreateIfNone = true)
	// Without this call, the actor would not have any label. When the unit test diffs properties, GetActorLabel will have been called, giving the actor a label.
	// The transient snapshot actor would not have a label. This would generate an unexpected diff which a unit test is not even testing for
	// Summary: Just give all actors a label so they do not generate false diffs.
	Result->SetActorLabel(Name.ToString());
	
	return Result;
}

ASnapshotTestActor::ASnapshotTestActor()
{
	ConstructorHelpers::FObjectFinder<UMaterialInterface> GradientLinearMaterialFinder(TEXT("Material'/Engine/MaterialTemplates/Gradients/Gradient_Linear.Gradient_Linear'"), LOAD_Quiet | LOAD_NoWarn);
	if (GradientLinearMaterialFinder.Succeeded())
	{
		GradientLinearMaterial = GradientLinearMaterialFinder.Object;
	}

	ConstructorHelpers::FObjectFinder<UMaterialInterface> GradientRadialMaterialFinder(TEXT("Material'/Engine/MaterialTemplates/Gradients/Gradient_Radial.Gradient_Radial'"), LOAD_Quiet | LOAD_NoWarn);
	if (GradientRadialMaterialFinder.Succeeded())
	{
		GradientRadialMaterial = GradientRadialMaterialFinder.Object;
	}
	
	ConstructorHelpers::FObjectFinder<UStaticMesh> CubeFinder(TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'"), LOAD_Quiet | LOAD_NoWarn);
	if (CubeFinder.Succeeded())
	{
		CubeMesh = CubeFinder.Object;
	}
	ConstructorHelpers::FObjectFinder<UStaticMesh> CylinderFinder(TEXT("StaticMesh'/Engine/BasicShapes/Cylinder.Cylinder'"), LOAD_Quiet | LOAD_NoWarn);
	if (CylinderFinder.Succeeded())
	{
		CylinderMesh = CylinderFinder.Object;
	}

	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("StaticMeshComponent"));
	StaticMeshComponent->SetupAttachment(RootComponent);
	StaticMeshComponent->SetStaticMesh(CubeMesh);
	
	InstancedMeshComponent = CreateDefaultSubobject<UInstancedStaticMeshComponent>(TEXT("InstancedMeshComponent"));
	InstancedMeshComponent->SetupAttachment(RootComponent);

	PointLightComponent = CreateDefaultSubobject<UPointLightComponent>(TEXT("PointLightComponent"));
	PointLightComponent->SetupAttachment(InstancedMeshComponent);

	TestComponent = CreateDefaultSubobject<USnapshotTestComponent>(TEXT("TestComponent"));
}

bool ASnapshotTestActor::HasAnyValidHardObjectReference() const
{
	const bool bHasSingleReference = !ObjectReference || !WeakObjectPtr.IsValid();

	const bool bAnyCollectionHasReference = ObjectArray.Num() != 0
		|| ObjectSet.Num() != 0
		|| WeakObjectPtrArray.Num() != 0
		|| WeakObjectPtrSet.Num() != 0
		|| ObjectMap.Num() != 0
		|| WeakObjectPtrMap.Num() != 0;

	return !bHasSingleReference && !bAnyCollectionHasReference;
}

bool ASnapshotTestActor::HasObjectReference(UObject* Object, bool bOnlyCheckCollections, FName MapKey) const
{
	const bool bReferencesEqual = (bOnlyCheckCollections
			|| (ObjectReference == Object && SoftPath == Object && SoftObjectPtr == Object && WeakObjectPtr == Object));

	const bool bCollectionsEqual = ObjectArray.Contains(Object)
		&& ObjectSet.Contains(Object)
		&& SoftPathArray.Contains(Object)
		&& SoftPathSet.Contains(Object)
		&& SoftObjectPtrArray.Contains(Object)
		&& SoftObjectPtrSet.Contains(Object)
		&& WeakObjectPtrArray.Contains(Object)
		&& WeakObjectPtrSet.Contains(Object)
		&& ObjectMap.Find(MapKey) && *ObjectMap.Find(MapKey) == Object
		&& SoftPathMap.Find(MapKey) && *SoftPathMap.Find(MapKey) == Object
		&& SoftObjectPtrMap.Find(MapKey) && *SoftObjectPtrMap.Find(MapKey) == Object
		&& WeakObjectPtrMap.Find(MapKey) && *WeakObjectPtrMap.Find(MapKey) == Object;

	return bReferencesEqual && bCollectionsEqual;
}

void ASnapshotTestActor::SetObjectReference(UObject* Object, FName MapKey)
{
	ClearObjectReferences();
	AddObjectReference(Object, MapKey);
}

void ASnapshotTestActor::AddObjectReference(UObject* Object, FName MapKey)
{
	ObjectReference = Object;
	ObjectArray.Add(Object);
	ObjectSet.Add(Object);
	ObjectMap.Add(MapKey, Object);

	SoftPath = Object;
	SoftPathArray.Add(Object);
	SoftPathSet.Add(Object);
	SoftPathMap.Add(MapKey, Object);

	SoftObjectPtr = Object;
	SoftObjectPtrArray.Add(Object);
	SoftObjectPtrSet.Add(Object);
	SoftObjectPtrMap.Add(MapKey, Object);

	WeakObjectPtr = Object;
	WeakObjectPtrArray.Add(Object);
	WeakObjectPtrSet.Add(Object);
	WeakObjectPtrMap.Add(MapKey, Object);
}

void ASnapshotTestActor::ClearObjectReferences()
{
	ObjectReference = nullptr;
	ObjectArray.Reset();
	ObjectSet.Reset();
	ObjectMap.Reset();

	SoftPath = nullptr;
	SoftPathArray.Reset();
	SoftPathSet.Reset();
	SoftPathMap.Reset();

	SoftObjectPtr = nullptr;
	SoftObjectPtrArray.Reset();
	SoftObjectPtrSet.Reset();
	SoftObjectPtrMap.Reset();

	WeakObjectPtr = nullptr;
	WeakObjectPtrArray.Reset();
	WeakObjectPtrSet.Reset();
	WeakObjectPtrMap.Reset();
}

void ASnapshotTestActor::AllocateSubobjects()
{
	EditOnlySubobject_OptionalSubobject = NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditOnlySubobject"));

	EditableInstancedSubobjectArray_OptionalSubobject.Add(NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditableInstancedSubobjectArray_1")));
	EditableInstancedSubobjectArray_OptionalSubobject.Add(NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditableInstancedSubobjectArray_2")));

	EditableInstancedSubobjectMap_OptionalSubobject.Add("First", NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditableInstancedSubobjectMap_1")));
	EditableInstancedSubobjectMap_OptionalSubobject.Add("Second", NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditableInstancedSubobjectMap_2")));
	
	EditOnlySubobjectArray_OptionalSubobject.Add(NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditOnlySubobjectArray_1")));
	EditOnlySubobjectArray_OptionalSubobject.Add(NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditOnlySubobjectArray_2")));
	
	EditOnlySubobjectMap_OptionalSubobject.Add("First", NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditOnlySubobjectMap_1")));
	EditOnlySubobjectMap_OptionalSubobject.Add("Second", NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditOnlySubobjectMap_2")));
}

void ASnapshotTestActor::AllocateNonReflectedSubobject()
{
	if (!NonReflectedSubobject)
	{
		NonReflectedSubobject.Reset(NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("NonReflectedSubobject")));
	}
}

void ASnapshotTestActor::DestroyNonReflectedSubobject()
{
	NonReflectedSubobject.Reset();
}

void ASnapshotTestActor::PostInitProperties()
{
	Super::PostInitProperties();

	EditableInstancedSubobject_DefaultSubobject = NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("EditableInstancedSubobject"));
	InstancedOnlySubobject_DefaultSubobject = NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("InstancedSubobject"));
	NakedSubobject_DefaultSubobject = NewObject<USubobject>(this, USubobject::StaticClass(), TEXT("NakedSubobject"));
}
