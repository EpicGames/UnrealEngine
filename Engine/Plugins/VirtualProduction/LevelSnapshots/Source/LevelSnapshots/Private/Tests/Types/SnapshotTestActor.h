// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/PointLightComponent.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Character.h"
#include "Materials/MaterialInterface.h"
#include "SnapshotTestActor.generated.h"

UCLASS()
class USubSubobject : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 IntProperty = 0;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	float FloatProperty = 0;
	
};

UCLASS()
class USubobject : public UObject
{
	GENERATED_BODY()
public:

	USubobject();
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 IntProperty = 0;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	float FloatProperty = 0;

	UPROPERTY(EditAnywhere, Instanced, Category = "Level Snapshots")
	TObjectPtr<USubSubobject> NestedChild;

	UPROPERTY(Instanced)
	TObjectPtr<USubSubobject> UneditableNestedChild;

	FName NonReflectedName;
	UObject* NonReflectedObjectProperty;
	TSoftObjectPtr<UObject> NonReflectedSoftPtr;
};

UCLASS()
class USnapshotTestComponent : public UActorComponent
{
	GENERATED_BODY()
public:

	USnapshotTestComponent();
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 IntProperty = 0;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	float FloatProperty = 0;

	UPROPERTY(Instanced)
	TObjectPtr<USubobject> Subobject;
};

UCLASS()
class ASnapshotTestActor : public ACharacter
{
	GENERATED_BODY()
public:

	static ASnapshotTestActor* Spawn(UWorld* World, FName Name = TEXT("TestActor"));
	
	ASnapshotTestActor();

	bool HasAnyValidHardObjectReference() const;
	bool HasObjectReference(UObject* Object, bool bOnlyCheckCollections = false, FName MapKey = NAME_Name) const;
	
	void SetObjectReference(UObject* Object, FName MapKey = NAME_Name);
	void AddObjectReference(UObject* Object, FName MapKey = NAME_Name);
	void ClearObjectReferences();

	void AllocateSubobjects();

	void AllocateNonReflectedSubobject();
	void DestroyNonReflectedSubobject();
	
	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

#if WITH_EDITOR
	enum class ECanDeleteMode
	{
		CallSuper,
		Yes,
		No
	} DeleteSelectedActorMode = ECanDeleteMode::CallSuper;
	bool bIsUserManaged = true;
	
	virtual bool CanDeleteSelectedActor(FText& OutReason) const override
	{
		switch (DeleteSelectedActorMode)
		{
		case ECanDeleteMode::Yes: return true;
		case ECanDeleteMode::No: return false;
		default:
			return Super::CanDeleteSelectedActor(OutReason);
		}
	}
	virtual bool IsUserManaged() const override  { return bIsUserManaged; }
#endif
	
	
	/******************** Skipped properties  ********************/
	
	UPROPERTY()
	int32 DeprecatedProperty_DEPRECATED = 100;

	UPROPERTY(EditAnywhere, Transient, Category = "Level Snapshots")
	int32 TransientProperty = 200;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	int32 IntProperty = 0;


	
	/******************** Raw references  ********************/
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UObject> ObjectReference;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<TObjectPtr<UObject>> ObjectArray;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSet<TObjectPtr<UObject>> ObjectSet;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<FName, TObjectPtr<UObject>> ObjectMap;



	/******************** FSoftObjectPath  ********************/

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	FSoftObjectPath SoftPath;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<FSoftObjectPath> SoftPathArray;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSet<FSoftObjectPath> SoftPathSet;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<FName, FSoftObjectPath> SoftPathMap;

	

	
	/******************** TSoftObjectPtr  ********************/

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSoftObjectPtr<UObject> SoftObjectPtr;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<TSoftObjectPtr<UObject>> SoftObjectPtrArray;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSet<TSoftObjectPtr<UObject>> SoftObjectPtrSet;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<FName, TSoftObjectPtr<UObject>> SoftObjectPtrMap;



	
	/******************** TWeakObjectPtr  ********************/

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TWeakObjectPtr<UObject> WeakObjectPtr;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<TWeakObjectPtr<UObject>> WeakObjectPtrArray;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSet<TWeakObjectPtr<UObject>> WeakObjectPtrSet;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<FName, TWeakObjectPtr<UObject>> WeakObjectPtrMap;

	

	/******************** External component references  ********************/
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UActorComponent> ExternalComponentReference;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UObject> ExternalComponentReferenceAsUObject;



	
	/******************** External references  ********************/
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UMaterialInterface> GradientLinearMaterial;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UMaterialInterface> GradientRadialMaterial;
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UStaticMesh> CubeMesh;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UStaticMesh> CylinderMesh;



	
	/******************** Subobject Component references  ********************/

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent;
	
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UInstancedStaticMeshComponent> InstancedMeshComponent;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<UPointLightComponent> PointLightComponent;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<USnapshotTestComponent> TestComponent;

	
	/******************** Subobject references  ********************/

	UPROPERTY(EditAnywhere, Instanced, Category = "Level Snapshots")
	TObjectPtr<USubobject> EditableInstancedSubobject_DefaultSubobject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobject> InstancedOnlySubobject_DefaultSubobject;

	UPROPERTY()
	TObjectPtr<USubobject> NakedSubobject_DefaultSubobject;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TObjectPtr<USubobject> EditOnlySubobject_OptionalSubobject;
	
	
	UPROPERTY(EditAnywhere, Instanced, Category = "Level Snapshots")
	TArray<TObjectPtr<USubobject>> EditableInstancedSubobjectArray_OptionalSubobject;

	UPROPERTY(EditAnywhere, Instanced, Category = "Level Snapshots")
	TMap<FName, TObjectPtr<USubobject>> EditableInstancedSubobjectMap_OptionalSubobject;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TArray<TObjectPtr<USubobject>> EditOnlySubobjectArray_OptionalSubobject;

	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TMap<FName, TObjectPtr<USubobject>> EditOnlySubobjectMap_OptionalSubobject;
	
	TStrongObjectPtr<USubobject> NonReflectedSubobject;

	FName NonReflectedName;
	TStrongObjectPtr<UObject> NonReflectedObjectProperty;
	TSoftObjectPtr<UObject> NonReflectedSoftPtr;
};
