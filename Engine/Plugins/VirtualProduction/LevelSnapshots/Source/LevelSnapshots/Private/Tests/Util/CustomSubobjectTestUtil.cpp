// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomSubobjectTestUtil.h"

#include "LevelSnapshotsModule.h"
#include "Params/ObjectSnapshotSerializationData.h"
#include "Tests/Types/SnapshotTestActor.h"

namespace UE::LevelSnapshots::Private::Tests
{
	TCustomObjectSerializerContext<FInstancedOnlySubobjectCustomObjectSerializer> FInstancedOnlySubobjectCustomObjectSerializer::Make()
	{
		return TCustomObjectSerializerContext<FInstancedOnlySubobjectCustomObjectSerializer>();
	}

	void FInstancedOnlySubobjectCustomObjectSerializer::OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage)
	{
		DataStorage.AddSubobjectSnapshot(TestActor->InstancedOnlySubobject_DefaultSubobject);
	}

	UObject* FInstancedOnlySubobjectCustomObjectSerializer::FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage)
	{
		if (ASnapshotTestActor* SnapshotActor = Cast<ASnapshotTestActor>(SnapshotObject))
		{
			check(SnapshotActor->InstancedOnlySubobject_DefaultSubobject);
			return SnapshotActor->InstancedOnlySubobject_DefaultSubobject;
		}

		checkNoEntry();
		return nullptr;
	}

	UObject* FInstancedOnlySubobjectCustomObjectSerializer::FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage)
	{
		if (ASnapshotTestActor* SnapshotActor =Cast<ASnapshotTestActor>(EditorObject))
		{
			check(SnapshotActor->InstancedOnlySubobject_DefaultSubobject);
			return SnapshotActor->InstancedOnlySubobject_DefaultSubobject;
		}

		checkNoEntry();
		return nullptr;
	}

	UObject* FInstancedOnlySubobjectCustomObjectSerializer::FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage)
	{
		if (ASnapshotTestActor* SnapshotActor = Cast<ASnapshotTestActor>(EditorObject))
		{
			check(SnapshotActor->InstancedOnlySubobject_DefaultSubobject);
			return SnapshotActor->InstancedOnlySubobject_DefaultSubobject;
		}

		checkNoEntry();
		return nullptr;
	}

	TSet<const FProperty*> GetSubobjectProperties()
	{
		TSet<const FProperty*> Properties;
			
		Properties.Add(ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, EditableInstancedSubobject_DefaultSubobject)));
		Properties.Add(ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, InstancedOnlySubobject_DefaultSubobject)));
		Properties.Add(ASnapshotTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ASnapshotTestActor, NakedSubobject_DefaultSubobject)));
		Properties.Add(USubobject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USubobject, NestedChild)));

		return Properties;
	}
		
	void DisallowSubobjectProperties()
	{
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<Private::FLevelSnapshotsModule>("LevelSnapshots");
		Module.AddExplicitlyUnsupportedProperties(GetSubobjectProperties());
	}

	void ReallowSubobjectProperties()
	{
		FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
		Module.RemoveExplicitlyUnsupportedProperties(GetSubobjectProperties());
	}
}


