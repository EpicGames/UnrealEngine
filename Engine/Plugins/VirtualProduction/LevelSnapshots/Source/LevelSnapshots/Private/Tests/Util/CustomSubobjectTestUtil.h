// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotsModule.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "Tests/Types/SnapshotTestActor.h"

class ASnapshotTestActor;

namespace UE::LevelSnapshots::Private::Tests
{
	TSet<const FProperty*> GetSubobjectProperties();
	void DisallowSubobjectProperties();
	void ReallowSubobjectProperties();
	
	template<typename TSerializerType>
	class TCustomObjectSerializerContext
	{
		TSharedRef<TSerializerType> CustomSerializer;
	public:
		
		template<typename ...T>
		TCustomObjectSerializerContext(T&&... Arg)
			: CustomSerializer(MakeShared<TSerializerType>(Forward<T>(Arg)...))
		{
			FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
			Module.RegisterCustomObjectSerializer(ASnapshotTestActor::StaticClass(), CustomSerializer);
			DisallowSubobjectProperties();
		}
		
		~TCustomObjectSerializerContext()
		{
			FLevelSnapshotsModule& Module = FModuleManager::Get().GetModuleChecked<FLevelSnapshotsModule>("LevelSnapshots");
			Module.UnregisterCustomObjectSerializer(ASnapshotTestActor::StaticClass());
			ReallowSubobjectProperties();
		}
		
		TSharedRef<TSerializerType> GetCustomSerializer() const { return CustomSerializer; }
	};
	
	class FInstancedOnlySubobjectCustomObjectSerializer : public ICustomObjectSnapshotSerializer
	{
	public:

		static TCustomObjectSerializerContext<FInstancedOnlySubobjectCustomObjectSerializer> Make();
		
		virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override;
		virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override;
		virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override;
		virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override;

		ASnapshotTestActor* TestActor;
	};
}
