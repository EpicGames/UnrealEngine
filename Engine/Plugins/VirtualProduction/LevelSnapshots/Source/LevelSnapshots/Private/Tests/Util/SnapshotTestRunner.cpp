// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnapshotTestRunner.h"

#include "LevelSnapshotsFilteringLibrary.h"
#include "Data/LevelSnapshot.h"
#include "LevelSnapshotsFunctionLibrary.h"

#include "PreviewScene.h"
#include "UObject/Package.h"

FName UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::DefaultSnapshotId = FName("DefaultSnapshotId");

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::FSnapshotTestRunner(ETestFlags Flags)
	: Flags(Flags)
{
	TestWorld = MakeShared<FPreviewScene>(
		FPreviewScene::ConstructionValues()
			.SetEditor(true)
			);
	
	// Some tests rely on the fact that the test takes place in a non-transient world
	if (EnumHasAnyFlags(Flags, ETestFlags::NonTransientWorld))
	{
		UPackage* Package = CreatePackage(*FString::Printf(TEXT("/Temp/LevelSnapshots/Test%s"), *FGuid::NewGuid().ToString(EGuidFormats::Base36Encoded)));
		// Executed tests might (indirectly) trigger a manual garbage collection 
		Package->AddToRoot();
		TestWorld->GetWorld()->Rename(nullptr, Package);
	}
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::FSnapshotTestRunner(UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner&& Other)
{
	TestWorld = Other.TestWorld;
	Snapshots = Other.Snapshots;
	Flags = Other.Flags;
	// Prevent Other's destructor from calling RemoveFromRoot
	Other.Snapshots.Empty();
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::~FSnapshotTestRunner()
{
	if (EnumHasAnyFlags(Flags, ETestFlags::NonTransientWorld))
	{
		UPackage* Package = TestWorld->GetWorld()->GetPackage();
		Package->RemoveFromRoot();
		// If you run a test, and then try to close the engine, the package would show as pending save
		Package->SetFlags(RF_Transient);
	}
	
	for (auto SnapshotIt = Snapshots.CreateIterator(); SnapshotIt; ++SnapshotIt)
	{
		SnapshotIt->Value->ConditionalBeginDestroy();
		SnapshotIt->Value->RemoveFromRoot();
		// ULevelSnapshot::BeginDestroy marks it as RF_Standalone again
		SnapshotIt->Value->ClearFlags(RF_Standalone);
	}
	
	if (EnumHasAnyFlags(Flags, ETestFlags::NonTransientWorld))
	{
		// Otherwise other tests will log a warning "CleanupWorld was called twice"
		CollectGarbage(RF_Standalone);
	}
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::ModifyWorld(TFunction<void(UWorld*)> Callback)
{
	Callback(TestWorld->GetWorld());
	return *this;
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::TakeSnapshot(FName SnapshotId)
{
	if (ULevelSnapshot** ExistingSnapshot = Snapshots.Find(SnapshotId))
	{
		(*ExistingSnapshot)->SnapshotWorld(TestWorld->GetWorld());
	}
	else
	{
		ULevelSnapshot* NewSnapshot = ULevelSnapshotsFunctionLibrary::TakeLevelSnapshot(TestWorld->GetWorld(), SnapshotId);
		// Executed tests might (indirectly) trigger a manual garbage collection 
		NewSnapshot->AddToRoot();
		
		Snapshots.Add(
			SnapshotId,
			NewSnapshot
			);
	}
	
	return *this;
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::AccessSnapshot(TFunction<void (ULevelSnapshot*)> Callback, FName SnapshotId)
{
	if (ULevelSnapshot** ExistingSnapshot = Snapshots.Find(SnapshotId))
	{
		Callback(*ExistingSnapshot);
	}
	else
	{
		checkNoEntry();
	}

	return *this;
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::AccessSnapshotAndWorld(TFunction<void(ULevelSnapshot*, UWorld*)> Callback, FName SnapshotId)
{
	return AccessSnapshot([this, &Callback](ULevelSnapshot* Snapshot)
	{
		Callback(Snapshot, TestWorld->GetWorld());
	}, SnapshotId);
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::ApplySnapshot(TFunction<ULevelSnapshotFilter*()> Callback, FName SnapshotId)
{
	return ApplySnapshot(Callback(), SnapshotId);
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::ApplySnapshot(ULevelSnapshotFilter* Filter, FName SnapshotId)
{
	if (ULevelSnapshot** ExistingSnapshot = Snapshots.Find(SnapshotId))
	{
		ULevelSnapshotsFunctionLibrary::ApplySnapshotToWorld(TestWorld->GetWorld(), *ExistingSnapshot, Filter);
	}
	else
	{
		checkNoEntry();
	}
	
	return *this;
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::ApplySnapshot(TFunction<FPropertySelectionMap()> Callback, FName SnapshotId)
{
	return ApplySnapshot(Callback(), SnapshotId);
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::ApplySnapshot(const FPropertySelectionMap& SelectionSet, FName SnapshotId)
{
	if (ULevelSnapshot** ExistingSnapshot = Snapshots.Find(SnapshotId))
	{
		(*ExistingSnapshot)->ApplySnapshotToWorld(TestWorld->GetWorld(), SelectionSet);
	}
	else
	{
		checkNoEntry();
	}
	
	return *this;
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::FilterProperties(AActor* OriginalActor, TFunction<void(const FPropertySelectionMap&)> Callback, const ULevelSnapshotFilter* Filter, FName SnapshotId)
{
	return AccessSnapshot([OriginalActor, &Callback, Filter](ULevelSnapshot* Snapshot)
	{
		const FSoftObjectPath PathToOriginalActor(OriginalActor);
		const TOptional<TNonNullPtr<AActor>> SnapshotCounterpart = Snapshot->GetDeserializedActor(PathToOriginalActor);
		if (ensure(SnapshotCounterpart))
		{
			FPropertySelectionMap SelectedProperties;
			ULevelSnapshotsFilteringLibrary::ApplyFilterToFindSelectedProperties(Snapshot, SelectedProperties, OriginalActor, SnapshotCounterpart.GetValue(), Filter);
			Callback(SelectedProperties);
		}
	}, SnapshotId);
}

UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner& UE::LevelSnapshots::Private::Tests::FSnapshotTestRunner::RunTest(TFunction<void()> Callback)
{
	Callback();
	return *this;
}
