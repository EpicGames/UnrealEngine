// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "LevelSnapshot.h"
#include "LevelSnapshotFilters.h"

#include "Engine/World.h"

class FPreviewScene;

namespace UE::LevelSnapshots::Private::Tests
{
	enum class ETestFlags
	{
		None,
		/** The UWorld should be created in a non-transient package */
		NonTransientWorld = 1 << 0
	};
	ENUM_CLASS_FLAGS(ETestFlags);
	
	/**
	* Utility for executing tests
	*/
	class FSnapshotTestRunner
	{
	public:

		static FName DefaultSnapshotId;

		FSnapshotTestRunner(ETestFlags Flags = ETestFlags::None);
		FSnapshotTestRunner(const FSnapshotTestRunner&) = delete;
		FSnapshotTestRunner(FSnapshotTestRunner&& Other);
		~FSnapshotTestRunner();

		FSnapshotTestRunner& ModifyWorld(TFunction<void(UWorld*)> Callback);
	
		FSnapshotTestRunner& TakeSnapshot(FName SnapshotId = DefaultSnapshotId);
		FSnapshotTestRunner& AccessSnapshot(TFunction<void(ULevelSnapshot*)> Callback, FName SnapshotId = DefaultSnapshotId);
		FSnapshotTestRunner& AccessSnapshotAndWorld(TFunction<void(ULevelSnapshot*, UWorld*)> Callback, FName SnapshotId = DefaultSnapshotId);
	
		FSnapshotTestRunner& ApplySnapshot(TFunction<ULevelSnapshotFilter*()> Callback, FName SnapshotId = DefaultSnapshotId);
		FSnapshotTestRunner& ApplySnapshot(ULevelSnapshotFilter* Filter = nullptr, FName SnapshotId = DefaultSnapshotId);
	
		FSnapshotTestRunner& ApplySnapshot(TFunction<FPropertySelectionMap()> Callback, FName SnapshotId = DefaultSnapshotId);
		FSnapshotTestRunner& ApplySnapshot(const FPropertySelectionMap& SelectionSet, FName SnapshotId = DefaultSnapshotId);

		FSnapshotTestRunner& FilterProperties(AActor* OriginalActor, TFunction<void(const FPropertySelectionMap&)> Callback, const ULevelSnapshotFilter* Filter = nullptr, FName SnapshotId = DefaultSnapshotId);

		/* Just calls Callback. Existing for better readability in tests.  */
		FSnapshotTestRunner& RunTest(TFunction<void()> Callback);

	private:

		ETestFlags Flags;

		TSharedPtr<FPreviewScene> TestWorld;

		TMap<FName, ULevelSnapshot*> Snapshots;
	};
}
