// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Util/TestUtil.h"

#include "Engine/World.h"

AActor* UE::LevelSnapshots::Private::Tests::Spawn(UWorld* World, FName Name, UClass* Class)
{
	FActorSpawnParameters Params;
	Params.Name = Name;
	Params.bNoFail = true;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	return World->SpawnActor(Class, {}, {}, Params);
}
