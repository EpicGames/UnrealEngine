// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

namespace UE::LevelSnapshots::Private::Tests
{
	template<typename T>
	static T* Spawn(UWorld* World, FName Name)
	{
		return Cast<T>(Tests::Spawn(World, Name, T::StaticClass()));
	}

	static AActor* Spawn(UWorld* World, FName Name, UClass* Class);
};
