// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyIterator.h"

#include "UObject/UnrealType.h"

UE::LevelSnapshots::Private::FPropertyIterator::FPropertyIterator(UStruct* RootStruct, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback, EPropertyFlags SkipFlags)
	: SkipFlags(SkipFlags)
{
	IterateProperties(RootStruct, PropertyCallback, StructCallback);
}

void UE::LevelSnapshots::Private::FPropertyIterator::IterateProperties(UStruct* Struct, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback)
{
	if (VisitedSet.Contains(Struct))
	{
		return;
	}
	VisitedSet.Add(Struct);

	StructCallback(Struct);
	for (TFieldIterator<FProperty> PropertyIt(Struct); PropertyIt; ++PropertyIt)
	{
		if (PropertyIt->HasAnyPropertyFlags(SkipFlags))
		{
			continue;
		}
			
		PropertyCallback(*PropertyIt);
			
		if (!HandleStructProperty(*PropertyIt, PropertyCallback, StructCallback))
		{
			HandleCollectionProperties(*PropertyIt, PropertyCallback, StructCallback);
		}
	}
}

bool UE::LevelSnapshots::Private::FPropertyIterator::HandleStructProperty(FProperty* Property, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		IterateProperties(StructProperty->Struct, PropertyCallback, StructCallback);
		return true;
	}
	return false;
}

void UE::LevelSnapshots::Private::FPropertyIterator::HandleCollectionProperties(FProperty* Property, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback)
{
	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		HandleStructProperty(ArrayProperty->Inner, PropertyCallback, StructCallback);
	}
	else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	{
		HandleStructProperty(SetProperty->ElementProp, PropertyCallback, StructCallback);
	}
	else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		HandleStructProperty(MapProperty->KeyProp, PropertyCallback, StructCallback);
		HandleStructProperty(MapProperty->ValueProp, PropertyCallback, StructCallback);
	}
}