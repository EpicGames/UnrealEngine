// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

namespace UE::LevelSnapshots::Private
{
	/**
	* Utility class for iterating all properties in a struct, including properties of sub-structs and collections containing structs.
	*/
	class FPropertyIterator
	{
		TSet<UStruct*> VisitedSet;
		const EPropertyFlags SkipFlags;
	public:

		FPropertyIterator(UStruct* RootStruct, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback, EPropertyFlags SkipFlags = CPF_Transient | CPF_Deprecated);

	private:
	
		void IterateProperties(UStruct* Struct, TFunctionRef<void(FProperty*)> Callback, TFunctionRef<void(UStruct*)> StructCallback);
		bool HandleStructProperty(FProperty* Property, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback);
		void HandleCollectionProperties(FProperty* Property, TFunctionRef<void(FProperty*)> PropertyCallback, TFunctionRef<void(UStruct*)> StructCallback);
	};

}

