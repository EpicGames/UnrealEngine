// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftObjectPathUtil.h"

#include "ActorSnapshotData.h"
#include "LevelSnapshotsLog.h"

#include "GameFramework/Actor.h"

namespace UE::LevelSnapshots::Private
{
	FSoftObjectPath ExtractPathWithoutSubobjects(const FSoftObjectPath& ObjectPath)
	{
		int32 ColonIndex = INDEX_NONE;
		const FString Path = ObjectPath.ToString();
		Path.FindChar(':', ColonIndex);
		return ColonIndex == INDEX_NONE
			? Path
			: Path.Left(ColonIndex);
	}

	FString ExtractLastSubobjectName(const FSoftObjectPath& ObjectPath)
	{
		const FString& SubPathString = ObjectPath.GetSubPathString();
		const int32 LastDotIndex = SubPathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (LastDotIndex == INDEX_NONE)
		{
			return SubPathString;
		}

		return SubPathString.RightChop(LastDotIndex + 1);
	}

	TOptional<FSoftObjectPath> ExtractActorFromPath(const FSoftObjectPath& OriginalObjectPath, bool& bIsPathToActorSubobject)
	{
		const static FString PersistentLevelString("PersistentLevel.");
		const int32 PersistentLevelStringLength = PersistentLevelString.Len();
		// /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent becomes PersistentLevel.StaticMeshActor_42.StaticMeshComponent
		const FString& SubPathString = OriginalObjectPath.GetSubPathString();
		const int32 IndexOfPersistentLevelInfo = SubPathString.Find(PersistentLevelString, ESearchCase::CaseSensitive);
		if (IndexOfPersistentLevelInfo == INDEX_NONE)
		{
			return {};
		}

		// Example: /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent
		const int32 DotAfterActorNameIndex = SubPathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, IndexOfPersistentLevelInfo + PersistentLevelStringLength);
		const bool bPathIsToActor = DotAfterActorNameIndex == INDEX_NONE;
		
		bIsPathToActorSubobject = !bPathIsToActor;
		return bPathIsToActor ?
			// Example /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42
			OriginalObjectPath
			:
			// Converts /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent to /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42
			FSoftObjectPath(OriginalObjectPath.GetAssetPath(), SubPathString.Left(DotAfterActorNameIndex));
	}

	bool IsPathToWorldObject(const FSoftObjectPath& OriginalObjectPath)
	{
		bool bDummy;
		return ExtractActorFromPath(OriginalObjectPath, bDummy).IsSet();
	}

	TOptional<int32> FindDotAfterActorName(const FSoftObjectPath& OriginalObjectPath)
	{
		const static FString PersistentLevelString("PersistentLevel.");
		const int32 PersistentLevelStringLength = PersistentLevelString.Len();
		// /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent becomes PersistentLevel.StaticMeshActor_42.StaticMeshComponent
		const FString& SubPathString = OriginalObjectPath.GetSubPathString();
		const int32 IndexOfPersistentLevelInfo = SubPathString.Find(PersistentLevelString, ESearchCase::CaseSensitive);
		if (IndexOfPersistentLevelInfo == INDEX_NONE)
		{
			return {};
		}

		// Example: /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent
		const int32 DotAfterActorNameIndex = SubPathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, IndexOfPersistentLevelInfo + PersistentLevelStringLength);
		return DotAfterActorNameIndex != INDEX_NONE ? DotAfterActorNameIndex : TOptional<int32>();
	}

	TOptional<TNonNullPtr<FActorSnapshotData>> FindSavedActorDataUsingObjectPath(TMap<FSoftObjectPath, FActorSnapshotData>& ActorData, const FSoftObjectPath& OriginalObjectPath, bool& bIsPathToActorSubobject)
	{
		const TOptional<FSoftObjectPath> PathToActor = UE::LevelSnapshots::Private::ExtractActorFromPath(OriginalObjectPath, bIsPathToActorSubobject);
		if (!PathToActor.IsSet())
		{
			return {};	
		}
	
		FActorSnapshotData* Result = ActorData.Find(*PathToActor);
		UE_CLOGF(Result == nullptr, LogLevelSnapshots, Warning, "Path %ls looks like an actor path but no data was saved for it. Maybe it was a reference to an auto-generated actor, e.g. a brush or volume present in all worlds by default?", *OriginalObjectPath.ToString());
		return Result ? TOptional<TNonNullPtr<FActorSnapshotData>>(Result) : TOptional<TNonNullPtr<FActorSnapshotData>>();  
	}
	
	FSoftObjectPath SetActorInPath(AActor* NewActor, const FSoftObjectPath& OriginalObjectPath)
	{
		const static FString PersistentLevelString("PersistentLevel.");
		const int32 PersistentLevelStringLength = PersistentLevelString.Len();
		const FString& SubPathString = OriginalObjectPath.GetSubPathString();
		const int32 IndexOfPersistentLevelInfo = SubPathString.Find(PersistentLevelString, ESearchCase::CaseSensitive);
		if (IndexOfPersistentLevelInfo == INDEX_NONE)
		{
			return {};
		}

		const int32 DotAfterActorNameIndex = SubPathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, IndexOfPersistentLevelInfo + PersistentLevelStringLength);

		const FSoftObjectPath PathToNewActor(NewActor);
		// PersistentLevel.StaticMeshActor_42.StaticMeshComponent becomes .StaticMeshComponent
		const FString PathAfterOriginalActor = SubPathString.Right(SubPathString.Len() - DotAfterActorNameIndex); 
		return FSoftObjectPath(PathToNewActor.GetAssetPath(), PathToNewActor.GetSubPathString() + PathAfterOriginalActor);
	}
}