// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/NonNullPointer.h"

class AActor;
struct FActorSnapshotData;

namespace UE::LevelSnapshots::Private
{
	/** Given e.g. "/Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.SomeComponent" returns everything before the colon, i.e. "/Game/MapName.MapName"*/
	FSoftObjectPath ExtractPathWithoutSubobjects(const FSoftObjectPath& ObjectPath);
	
	/** Finds the last subobject name in the path */
	FString ExtractLastSubobjectName(const FSoftObjectPath& ObjectPath);

	/** If path contains an actor, return subobject path to that actor. */
	TOptional<FSoftObjectPath> ExtractActorFromPath(const FSoftObjectPath& OriginalObjectPath, bool& bIsPathToActorSubobject);

	/** Checks whether this is a soft object path to an object in an editor world */
	bool IsPathToWorldObject(const FSoftObjectPath& OriginalObjectPath);

	/** Given a path such as "/Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.SomeComponent" returns the index of "S" in "SomeComponent" */
	TOptional<int32> FindDotAfterActorName(const FSoftObjectPath& OriginalObjectPath);

	/**
	 * If Path contains a path to an actor, returns that actor.
	 * Example: /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent returns StaticMeshActor_42's data
	 */
	TOptional<TNonNullPtr<FActorSnapshotData>> FindSavedActorDataUsingObjectPath(TMap<FSoftObjectPath, FActorSnapshotData>& ActorData, const FSoftObjectPath& OriginalObjectPath, bool& bIsPathToActorSubobject);

	/**
	 * Takes an existing path to an actor's subobjects and replaces the actor bit with the path to another actor.
	 *
	 * E.g. /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent could become /Game/MapName.MapName:PersistentLevel.SomeOtherActor.StaticMeshComponent
	 */
	FSoftObjectPath SetActorInPath(AActor* NewActor, const FSoftObjectPath& OriginalObjectPath);
};
