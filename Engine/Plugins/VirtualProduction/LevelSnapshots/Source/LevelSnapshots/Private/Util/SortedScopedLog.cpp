// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/SortedScopedLog.h"

#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "Templates/Greater.h"

void FConditionalSortedScopedLog::FFormatLog::operator()(FOutputDevice& OutputDevice, const FString& DisplayUnitsString, const FString& Title, const TArray<FLoggedItem>& SortedItems)
{
	if (!Title.IsEmpty())
	{
		OutputDevice.Logf(ELogVerbosity::Log, TEXT("%s"), *Title);
	}
		
	for (const FLoggedItem& Item : SortedItems)
	{
		OutputDevice.Logf(ELogVerbosity::Log, TEXT( "\t%s - %6.3f %s" ), *Item.Value, Item.Key, *DisplayUnitsString);
	}
}

namespace
{
	FString CreateCSVRow(const FString& String)
	{
		return FString::Printf(TEXT("\"%s\""), *String);
	}

	FString CreateCSVRow(const FString& String, double Time, const FString& DisplayUnitsString)
	{
		return FString::Printf(TEXT("\"%32s\",\"%6.3f%s\""), *String, Time, *DisplayUnitsString);
	}
}

void FConditionalSortedScopedLog::FFormatCSV::operator()(FOutputDevice& OutputDevice, const FString& DisplayUnitsString, const FString& Title, const TArray<FLoggedItem>& SortedItems)
{
	FString Result;

	if (!Title.IsEmpty())
	{
		Result += CreateCSVRow(Title);

		if (SortedItems.Num())
		{
			Result += "\n";
		}
	}

	const int32 Num = SortedItems.Num();
	for (int32 i = 0; i < Num - 1; ++i)
	{
		const FLoggedItem& Item = SortedItems[i];
		Result += CreateCSVRow(Item.Value, Item.Key, DisplayUnitsString) + "\n";
	}

	if (Num)
	{
		const FLoggedItem& LastItem = SortedItems[Num - 1];
		Result += CreateCSVRow(LastItem.Value, LastItem.Key, DisplayUnitsString);
	}
	
	OutputDevice.Logf(ELogVerbosity::Log, TEXT("%s"), *Result);
}

FConditionalSortedScopedLog::FConditionalSortedScopedLog(bool bCondition, FString Title, FLogFormatter Formatter, FOutputDevice* OutputDevice, bool bLargeTimesFirst, bool bLogTotalTime, EScopeLogTimeUnits Units)
	: bCondition(bCondition)
	, LogTitle(Title)
	, bLargeTimesFirst(bLargeTimesFirst)
	, StartTime(bLogTotalTime ? FPlatformTime::Seconds() : 0.f)
	, Units(Units)
	, Formatter(Formatter)
	, Output(OutputDevice)
{}

FConditionalSortedScopedLog::~FConditionalSortedScopedLog()
{
	if (bCondition)
	{
		if (bLargeTimesFirst)
		{
			LoggedItems.Sort(TGreater<FLoggedItem>());
		}
		else
		{
			LoggedItems.Sort(TLess<FLoggedItem>());
		}

		const bool bLogTotalTime = StartTime != 0.0;
		if (bLogTotalTime)
		{
			AddCustomMeasuredLogItem(TEXT("Total Time"), FPlatformTime::Seconds() - StartTime);
		}
		
		Formatter(Output ? *Output : *GLog, GetDisplayUnitsString(), LogTitle, LoggedItems);
	}
}

FScopedLogItem FConditionalSortedScopedLog::AddScopedLogItem(FString Name, bool bItemLogCondition)
{
	return FScopedLogItem(*this, bItemLogCondition, Name);
}

void FConditionalSortedScopedLog::AddCustomMeasuredLogItem(FString Name, double Time)
{
	LoggedItems.Add(FLoggedItem(GetDisplayScopedTime(Time), Name));
}

double FConditionalSortedScopedLog::GetDisplayScopedTime(double InScopedTime) const
{
	switch(Units)
	{
		case EScopeLogTimeUnits::Seconds: return InScopedTime;
		case EScopeLogTimeUnits::Milliseconds:
		default:
			return InScopedTime * 1000.0f;
	}
}

FString FConditionalSortedScopedLog::GetDisplayUnitsString() const
{
	switch (Units)
	{
		case EScopeLogTimeUnits::Seconds: return TEXT("s");
		case EScopeLogTimeUnits::Milliseconds:
		default:
			return TEXT("ms");
	}
}

FScopedLogItem::FScopedLogItem(FConditionalSortedScopedLog& Container, bool bCondition, FString InName)
	: Container(Container)
	, bCondition(bCondition)
	, StartTime(bCondition ? FPlatformTime::Seconds() : 0.0)
	, Name(InName)
{}

FScopedLogItem::~FScopedLogItem()
{
	if (bCondition && Container.bCondition)
	{
		const double ScopedTime = FPlatformTime::Seconds() - StartTime;
		Container.AddCustomMeasuredLogItem(Name, ScopedTime);
	}
}