// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Templates/UnrealTemplate.h"

class FScopedLogItem;

/**
 * Measures and sorts computing times.
 * Use as follows:
 *
 * void Foo()
 * {
 *		const bool bShouldLog = ...
 *		FConditionalSortedScopedLog Container(bShouldLog);
 *
 *		for (...)
 *		{
 *			FScopedLogItem Log = Container.AddLogItem("Item_N")
 *			...
 *		}
 * }
 *
 * This will print something like this:
 * LogStats:                  NDC_NantStage_2 - 42.983 ms
 * LogStats:       BP_OffRoadVechicleStatic_4 - 14.094 ms
 * LogStats:                      LightCard_2 - 1.795 ms
 */
class FConditionalSortedScopedLog : public FNoncopyable
{
	friend FScopedLogItem;
public:
	
	using FLoggedItem = TPair<double, FString>;
	using FLogFormatter = TFunction<void(FOutputDevice& OutputDevice, const FString DisplayUnitsString, const FString& Title, const TArray<FLoggedItem>& SortedItems)>;

	/** Simply logs to output device */
	struct FFormatLog
	{
		void operator()(FOutputDevice& OutputDevice, const FString& DisplayUnitsString, const FString& Title, const TArray<FLoggedItem>& SortedItems);
	};
	/** Logs in CSV format */
	struct FFormatCSV
	{
		void operator()(FOutputDevice& OutputDevice, const FString& DisplayUnitsString, const FString& Title, const TArray<FLoggedItem>& SortedItems);
	};

	enum class EScopeLogTimeUnits
	{
		Milliseconds,
		Seconds
	};

	/**
	 * @param bCondition Determines whether anything will be printed or not. Typical use case: this condition depends on a console variable.
	 * @param Title This will be printed before the sorted list of items
	 * @param bLargeTimesFirst Whether to sort descending or ascending
	 * @param bLogTotalTime Whether to log the total time taken when this log is destroyed
	 * @param Units Whether to use seconds or milliseconds when printing time information
	 * @param Formatter Receives the final data and logs it. Useful for logging in a special format, such as
	 * @param OutputDevice Where to print to
	 */
	FConditionalSortedScopedLog(
		bool bCondition,
		FString Title = FString(),
		FLogFormatter Formatter = FFormatLog(),
		FOutputDevice* OutputDevice = GLog,
		bool bLargeTimesFirst = true,
		bool bLogTotalTime = true,
		EScopeLogTimeUnits Units = EScopeLogTimeUnits::Milliseconds);
	~FConditionalSortedScopedLog();

	
	/** Adds a log item. When it goes out of scope, its time is automatically added to LoggedItems. */
	[[nodiscard]] FScopedLogItem AddScopedLogItem(FString Name, bool bItemLogCondition = true);

	void AddCustomMeasuredLogItem(bool bLogCondition, FString Name, double Time) { if (bLogCondition) { AddCustomMeasuredLogItem(Name, Time); } }
	/** Adds a time you measured yourself*/
	void AddCustomMeasuredLogItem(FString Name, double Time);

private:
	
	double GetDisplayScopedTime(double InScopedTime) const;
	FString GetDisplayUnitsString() const;
	
	bool bCondition;
	
	FString LogTitle;
	bool bLargeTimesFirst;
	const double StartTime; 
	EScopeLogTimeUnits Units;
	FLogFormatter Formatter;
	FOutputDevice* Output;
	
	/** Will be sorted later */
	TArray<FLoggedItem> LoggedItems;
};

/** Tracks time for this item to be destroyed and logs it to the owning container. */
class FScopedLogItem
{
public:
	
	FScopedLogItem(FConditionalSortedScopedLog& Container, bool bCondition, FString InName);
	~FScopedLogItem();
	
	FScopedLogItem(const FScopedLogItem&) = delete;
	FScopedLogItem(FScopedLogItem&&) = default;
	FScopedLogItem& operator=(const FScopedLogItem&) = delete;
	FScopedLogItem& operator=(FScopedLogItem&&) = delete;

private:
	
	FConditionalSortedScopedLog& Container;
	const bool bCondition;
	const double StartTime;
	const FString Name;
};

/** Version of FConditionalSortedScopedLog that always logs. */
class FSortedScopedLog : public FConditionalSortedScopedLog
{
public:

	FSortedScopedLog(FString Name = FString(),
		FLogFormatter Formatter = FFormatLog(),
		FOutputDevice* OutputDevice = GLog,
		bool bLargeTimesFirst = true, 
		bool bLogTotalTime = true,
		EScopeLogTimeUnits Units = EScopeLogTimeUnits::Milliseconds
		)
		: FConditionalSortedScopedLog(true, Name, Formatter, OutputDevice, bLargeTimesFirst, bLogTotalTime, Units)
	{}
};