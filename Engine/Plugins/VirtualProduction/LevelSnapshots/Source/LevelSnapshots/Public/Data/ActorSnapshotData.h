// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentSnapshotData.h"
#include "CustomSerializationData.h"
#include "Hashing/ActorSnapshotHash.h"
#include "ObjectSnapshotData.h"
#include "SnapshotUtilTypes.h"
#include "ActorSnapshotData.generated.h"

/** Holds saved actor data. See ActorUtil for operations. */
USTRUCT()
struct LEVELSNAPSHOTS_API FActorSnapshotData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	/** The label of the actor when it was saved. */
	UPROPERTY()
	FString ActorLabel;
#endif
	

	/** The class the actor had when it was saved. */
	UPROPERTY()
	FSoftClassPath ActorClass_DEPRECATED;

	/** Valid index to FWorldSnapshotData::ClassData. Use to lookup class and archetype data. */
	UPROPERTY()
	int32 ClassIndex = INDEX_NONE;

	/** The actor's serialized data */
	UPROPERTY()
	FObjectSnapshotData SerializedActorData;
	

	
	/** Data that was generated by some ICustomObjectSnapshotSerializer. Needed to restore custom subobjects. */
	UPROPERTY()
	FCustomSerializationData CustomActorSerializationData;


	
	/** Additional component recreation info */
	UPROPERTY()
	TMap<int32, FComponentSnapshotData> ComponentData;

	/** Tracks all non-component subobjects. Valid index to FWorldSnapshotData::SerializedObjectReferences. */
	UPROPERTY()
	TArray<int32> OwnedSubobjects;

	

	/** Used to detect changes without loading actor into memory. */
	UPROPERTY()
	FActorSnapshotHash Hash;

	// "Type safety" for better self-documenting code
	UE::LevelSnapshots::FClassDataIndex GetClassDataIndex() const { return ClassIndex; }
};
