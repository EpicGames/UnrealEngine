// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ObjectSnapshotData.h"
#include "ClassDefaultObjectSnapshotData.generated.h"

/** We save the CDO of every object to save space. This holds the CDO's saved data. */
USTRUCT()
struct LEVELSNAPSHOTS_API FClassDefaultObjectSnapshotData : public FObjectSnapshotData
{
	GENERATED_BODY()

	/** Whether no data was saved because somebody called ILevelSnapshotsModule::AddSkippedClassDefault */
	UPROPERTY()
	bool bSerializationSkippedCDO = false;
};

