// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ObjectSnapshotData.h"
#include "ClassSnapshotData.generated.h"

UENUM()
enum class ESnapshotClassFlags : uint8
{
	NoFlags						= 0x00000000,
	
	/**
	 * There is no data available for the archetype because Level Snapshots was configured so, e.g. ILevelSnapshotsModule::AddSkippedClassDefault was called.
	 */
	SerializationSkippedArchetypeData	= 0x00000001
};

ENUM_CLASS_FLAGS(ESnapshotClassFlags)

/**
 * Holds class information for a specific use of a class. Usually it holds the class archetype.
 * Actor classes have exactly one instance of FClassSnapshotData.
 * Subobjects can have multiple instances of FClassSnapshotData; in actors each subobject has its own archetype.
 */
USTRUCT()
struct LEVELSNAPSHOTS_API FClassSnapshotData : public FObjectSnapshotData
{
	GENERATED_BODY()

	/** The class getting saved. */
	UPROPERTY()
	FSoftClassPath ClassPath;

	/** Flags of ClassPath, i.e. the class whose archetype was serialized */
	UPROPERTY()
	uint64 ClassFlags {};
	
	/** Special flags about this class */
	UPROPERTY()
	ESnapshotClassFlags SnapshotFlags = ESnapshotClassFlags::NoFlags;
	
	EObjectFlags GetClassFlags() const { return static_cast<EObjectFlags>(ClassFlags); }
};

