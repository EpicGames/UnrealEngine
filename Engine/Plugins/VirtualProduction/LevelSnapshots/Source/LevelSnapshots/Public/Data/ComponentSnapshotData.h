// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentInstanceDataCache.h"
#include "ComponentSnapshotData.generated.h"

USTRUCT()
struct LEVELSNAPSHOTS_API FComponentSnapshotData
{
	GENERATED_BODY()
	
	/** Describes how the component was created */
	UPROPERTY()
	EComponentCreationMethod CreationMethod {};
};