// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Data/ObjectSnapshotData.h"
#include "CustomSerializationData.generated.h"

USTRUCT()
struct LEVELSNAPSHOTS_API FCustomSubbjectSerializationData : public FObjectSnapshotData
{
	GENERATED_BODY()

	/* Valid index to FWorldSnapshotData::SerializedObjectReferences */
	UPROPERTY()
	int32 ObjectPathIndex = INDEX_NONE;

	/** Valid index to FWorldSnapshotData::ClassData. Use to lookup class and archetype data. */
	UPROPERTY()
	int32 ClassIndex = INDEX_NONE;

	/* Additional custom data saved by ICustomSnapshotSerializationData */
	UPROPERTY()
	TArray<uint8> SubobjectAnnotationData;
};

USTRUCT()
struct LEVELSNAPSHOTS_API FCustomSerializationData
{
	GENERATED_BODY()

	/* Additional custom data saved by ICustomSnapshotSerializationData */
	UPROPERTY()
	TArray<uint8> RootAnnotationData;

	/* Data for all subobjects */
	UPROPERTY()
	TArray<FCustomSubbjectSerializationData> Subobjects;
};