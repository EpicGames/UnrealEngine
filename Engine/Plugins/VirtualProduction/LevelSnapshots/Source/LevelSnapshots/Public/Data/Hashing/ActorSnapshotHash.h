// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"
#include "ActorSnapshotHash.generated.h"

/** Every actor is hashed when it is saved so we can quickly check whether an actor has changed. */
USTRUCT()
struct LEVELSNAPSHOTS_API FActorSnapshotHash
{
	GENERATED_BODY()

	/**
	 * How many micro seconds it took to compute the actor CRC32 during saving. Used when loading actors.
	 * If the hash time is excessively high, it is more efficient simply to load the actor. Configured in project settings.
	 */
	UPROPERTY()
	double MicroSecondsForCrc = -1.0;

	/** How many micro seconds it took to compute the MD5 hash. */
	UPROPERTY()
	double MicroSecondsForMD5 = -1.0;
	
	/** How many bytes of data were in the data were used for CRC32. Used to avoid computing hash. */
	UPROPERTY()
	int32 Crc32DataLength {};
	
	/** How many bytes of data were in the data were used for MD5. Used to avoid computing hash. */
	UPROPERTY()
	int32 MD5DataLength {};

	/**
	 * Crc32 hash of actor when it was snapshot. Used to check for changes without loading actor. Sometimes MD5 is faster.
	 * Cost is 0.0003s on "average" actors, like world settings.
	 */
	UPROPERTY()
	uint32 Crc32 {};

	/**
	 * MD5 hash of actor when it was snapshot. Used to check for changes without loading actor. Sometimes CRC32 is faster.
	 * Cost is 0.0003s on "average" actors, like world settings. 
	 * CRC32 is generally to be preferred but we do MD5 to future proof cases where there may be collisions. Users can configure this in project settings.
	 */
	FMD5Hash MD5;
	
	bool HasCrc32() const { return MicroSecondsForCrc > 0.0; }
	bool HasMD5() const
	{
		return MicroSecondsForMD5 > 0.0
			// Data saved before 5.2 can have 0.0 > MicroSecondsForMD5 but invalid MD5 because Serialize was not implemented correctly (see FSnapshotCustomVersion::FixActorHashSerialize)
			&& MD5.IsValid();
	}

	bool Serialize(FArchive& Archive);
};

template<>
struct TStructOpsTypeTraits<FActorSnapshotHash> : public TStructOpsTypeTraitsBase2<FActorSnapshotHash>
{
	enum 
	{ 
		WithSerializer = true
	};
};