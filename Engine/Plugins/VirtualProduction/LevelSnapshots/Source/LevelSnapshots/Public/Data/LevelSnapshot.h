// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SnapshotDataCache.h"
#include "WorldSnapshotData.h"
#include "LevelSnapshot.generated.h"

struct FPropertySelectionMap;

UENUM()
enum class ECachedDiffResult : uint8
{
	/** The actor was not yet analysed */
	NotInitialized,
	/** Actor was analysed and had changes */
	HadChanges,
	/** Actor was analysed and had no changes */
	HadNoChanges
};


/* Holds the state of a world at a given time. This asset can be used to rollback certain properties in a UWorld. */
UCLASS(BlueprintType)
class LEVELSNAPSHOTS_API ULevelSnapshot : public UObject
{
	GENERATED_BODY()
public:

	DECLARE_DELEGATE_OneParam(FActorPathConsumer, const FSoftObjectPath& /*OriginalActorPath*/);
	DECLARE_DELEGATE_OneParam(FActorConsumer, AActor* /*WorldActor*/);
	DECLARE_MULTICAST_DELEGATE(FSnapshotEvent);
	
	/* Captures the current state of the given world. */
	bool SnapshotWorld(UWorld* TargetWorld);
	
	/* Applies this snapshot to the given world. We assume the world matches. SelectionSet specifies which properties to roll back. */
	void ApplySnapshotToWorld(UWorld* TargetWorld, const FPropertySelectionMap& SelectionSet);


	/**
	 * Checks whether the given actor has changes to the snapshot version. First compares hashes and then proceeds comparing
	 * property values.
	 *
	 * In most cases, this function is faster than HasOriginalChangedPropertiesSinceSnapshotWasTaken because it tries to
	 * slow calls to GetDeserializedActor by comparing hashes first.
	 */
	bool HasChangedSinceSnapshotWasTaken(AActor* WorldActor);

	/**
	 * Checks whether the original actor has any properties that changed since the snapshot was taken by comparing properties.
	 *
	 * Use this function instead of HasChangedSinceSnapshotWasTaken if you've already called GetDeserializedActor. Otherwise
	 * HasChangedSinceSnapshotWasTaken should be faster in most cases.
	 */
	bool HasOriginalChangedPropertiesSinceSnapshotWasTaken(AActor* SnapshotActor, AActor* WorldActor);

	/** Gets the display label of the path of the actor. This is useful for the LS UI to display actor names, especially of removed actors. */
	FString GetActorLabel(const FSoftObjectPath& OriginalActorPath) const;
	
	/** Given an actor path in the world, gets the equivalent actor from the snapshot. */
	TOptional<TNonNullPtr<AActor>> GetDeserializedActor(const FSoftObjectPath& OriginalActorPath);

	/** Given an actor path in the world, gets its class from the snapshot. */
	TOptional<TNonNullPtr<UClass>> GetActorClass(const FSoftObjectPath& OriginalActorPath);

	/** Gets the number of actors saved by this snapshot. */
	int32 GetNumSavedActors() const;
	
	/**
	 * Compares this snapshot to the world and calls the appropriate callbacks:
	 * @param World to check in
	 *	@param HandleMatchedActor Actor exists both in world and snapshot. Receives the original actor path.
	 *	@param HandleRemovedActor Actor exists in snapshot but not in world. Receives the original actor path.
	 *	@param HandleAddedActor Actor exists in world but not in snapshot. Receives reference to world actor.
	 */
	UE_DEPRECATED(5.1, "Use DiffWorld that accepts FActorConsumer for HandleMatchedActor instead.")
	void DiffWorld(UWorld* World, FActorPathConsumer HandleMatchedActor, FActorPathConsumer HandleRemovedActor, FActorConsumer HandleAddedActor);
	void DiffWorld(UWorld* World, FActorConsumer HandleMatchedActor, FActorPathConsumer HandleRemovedActor, FActorConsumer HandleAddedActor);

	
	/* Sets the name of this snapshot. */
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	void SetSnapshotName(const FName& InSnapshotName);
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	void SetSnapshotDescription(const FString& InSnapshotDescription);

	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	FSoftObjectPath GetMapPath() const { return MapPath; }
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	FDateTime GetCaptureTime() const { return CaptureTime; }
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	FName GetSnapshotName() const { return SnapshotName; }
	UFUNCTION(BlueprintPure, Category = "Level Snapshots")
	FString GetSnapshotDescription() const { return SnapshotDescription; }

	const FWorldSnapshotData& GetSerializedData() const { return SerializedData; }
	const FSnapshotDataCache& GetCache() const { return Cache; }

	FSnapshotEvent& OnPreApplySnapshot() { return OnPreApplySnapshotDelegate; }
	FSnapshotEvent& OnPostApplySnapshot() { return OnPostApplySnapshotDelegate; }

#if !WITH_EDITOR
	/**
	 * Indicates that user code is aware that ClearCachedDiffFlag must be called manually.
	 *
	 * In editor builds, the transaction system calls
	 * ClearCachedDiffFlag but in non-editor builds the transaction system does not exist.
	 * 
	 * Enables the use of CachedDiffedActors. 
	 */
	void EnableDiffCacheSupport() { bIsDiffCacheEnabled = true; }
#endif
	/** Clears the diff state of the actor. Designed to be called by the transaction system (or manually in non-editor builds if EnableDiffCacheSupport has been called). */
	void ResetDiffCacheToUninitialized(TArrayView<AActor*> ModifiedActors);
	void ResetDiffCacheToUninitialized(UObject* ModifiedObject); 
	
	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface
	
private:

	FString GenerateDebugLogInfo() const;

	void EnsureWorldInitialised();
	void DestroyWorld();
	void CleanUpWorld();
	void ClearCache();
	
	void RecreateSnapshotWorld();
	
	/** Callback to destroy our world when editor (editor build) or play (game builds) world is destroyed. */
	FDelegateHandle Handle;
	/** Callback to when an object is modified. Clears FActorSnapshotData::bHasBeenDiffed */
	FDelegateHandle OnObjectModifiedHandle;


#if WITH_EDITORONLY_DATA
	/** Transient package that contains the world. Exists so calls to MarkPackageDirty, called e.g. by AActor::Modify, do not mark the snapshot dirty. */
	UPROPERTY(Transient)
	TObjectPtr<UPackage> TransientWorldPackage;
#endif
	/** The root world, equivalent to the persistent world, we will be adding temporary actors to */
	UPROPERTY(Transient)
	TObjectPtr<UWorld> RootSnapshotWorld;
	/** Sublevels of RootSnapshotWorld */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UWorld>> SnapshotSublevels;
	
	/** The saved snapshot data */
	UPROPERTY()
	FWorldSnapshotData SerializedData;

	/** Holds all loaded objects*/
	UPROPERTY(Transient)
	FSnapshotDataCache Cache;
	
	/**
	 * Only used in non-editor builds (should be wrapped in !WITH_EDITORONLY_DATA but UHT does not allow it).
	 * Indicates that user code is aware that ClearCachedDiffFlag must be called manually.
	 *
	 * In editor builds, the transaction system calls
	 * ClearCachedDiffFlag but in non-editor builds the transaction system does not exist.
	 * 
	 * Enables the use of CachedDiffedActors. 
	 */
	UPROPERTY(Transient)
	bool bIsDiffCacheEnabled = false;

	/** Caches whether an actor was diffed already */	
	UPROPERTY(Transient)
	TMap<TWeakObjectPtr<AActor>, ECachedDiffResult> CachedDiffedActors;

	
	/** Path of the map that the snapshot was taken in */
	UPROPERTY(VisibleAnywhere, BlueprintGetter = "GetMapPath", AssetRegistrySearchable, Category = "Level Snapshots")
	FSoftObjectPath MapPath;
	
	/** UTC Time that the snapshot was taken */
	UPROPERTY(AssetRegistrySearchable, BlueprintGetter = "GetCaptureTime", VisibleAnywhere, Category = "Level Snapshots")
	FDateTime CaptureTime;

	/** User defined name for the snapshot, can differ from the actual asset name. */
	UPROPERTY(AssetRegistrySearchable, BlueprintGetter = "GetSnapshotName", EditAnywhere, Category = "Level Snapshots")
	FName SnapshotName;
	
	/** User defined description of the snapshot */
	UPROPERTY(AssetRegistrySearchable, BlueprintGetter = "GetSnapshotDescription", EditAnywhere, Category = "Level Snapshots")
	FString SnapshotDescription;

	FSnapshotEvent OnPreApplySnapshotDelegate;
	FSnapshotEvent OnPostApplySnapshotDelegate;
};
