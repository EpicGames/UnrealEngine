// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ObjectSnapshotData.generated.h"

/** Shared data all saved objects in Level Snapshots have. */
USTRUCT()
struct LEVELSNAPSHOTS_API FObjectSnapshotData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<uint8> SerializedData;

	/** Flags of the object that was serialized */
	UPROPERTY()
	uint64 ObjectFlags = RF_Transactional;

	EObjectFlags GetObjectFlags() const { return static_cast<EObjectFlags>(ObjectFlags); }
};