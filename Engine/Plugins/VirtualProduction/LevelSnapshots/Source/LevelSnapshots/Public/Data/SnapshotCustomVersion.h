// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

namespace UE::LevelSnapshots
{
	struct LEVELSNAPSHOTS_API FSnapshotCustomVersion
	{
		enum Type
		{
			/** Before any version changes were made in the plugin */
			BeforeCustomVersionWasAdded = 0,

			/** When subobject support was added. Specifically, USceneComponent::AttachParent were not captured. */
			SubobjectSupport = 1,

			/** FSnapshotActorData now stores actor hash data to facilitate checking whether an actor has changed without loading the actor */
			ActorHash = 2,

			/** FWorldSnapshotData::ClassDefaults was replaced by FWorldSnapshotData::ClassData. */
			ClassArchetypeRefactor = 3,

			/** FWorldSnapshotData now compresses data using Oodle before it is saved to disk */
			OoddleCompression = 4,

			/** Before this soft object references were saved by FCustomSerializationDataWriter using FObjectAndNameAsStringProxyArchive - if you moved the asset our data would get corrupted. */
			CustomSubobjectSoftObjectPathRefactor = 5,

			/**
			 * When serializing a class archetype, we now skip properties that contain subobjects by discovering reference properties with a reference collector archive.
			 * No migration needs to take place. This entry exists only for future proofing (in case something breaks).
			 */
			SkipSubobjectReferencesWhenSerializingArchetypesRefactor = 6,

			/**
			 * If FComponentEditorUtils::CanEditComponentInstance returned false, the snapshot would not be captured.
			 * Now, if FComponentEditorUtils::CanEditComponentInstance returns false the component will still be captured if it is a root component; otherwise the actor's transform cannot be restored.
			 * No migration needs to take place. This entry exists only for future proofing (in case something breaks).
			 */
			NonEditableComponentsAreCaptured = 7,

			/** FActorSnapshotHash::Serialize was not implemented correctly (typo with TStructOpsTypeTraits using WithSerialize instead of WithSerializer). */
			FixActorHashSerialize = 8,

			/**
			 * Change 21419024 of 16/08/2022 (which introduced CustomSubobjectSoftObjectPathRefactor above) causes
			 * UE::LevelSnapshots::Foliage::Private::FInstancedFoliageActorData to save foliage data in a bad format.
			 *
			 * Any foliage data from 16/08/2022 to 30/03/2023 is unusable.
			 * That means that 5.0 is not affected, 5.1 is broken, and 5.2 is fixed.
			 *
			 * The issue is that FInstancedFoliageActorData::Save, called by FFoliageSupport::OnTakeSnapshot, called
			 * AInstancedFoliageActor::ForEachFoliageInfo and saved the foliage types sequentially. The issue is that
			 * we differentiate between instanced subobject and asset foliage types; they were saved interleaved without
			 * tagging the data, which makes it impossible to read (e.g. subject1 > asset1 > asset2 > subject 2, etc.).
			 */
			FoliageTypesUnreadable = 9,

			// -----<new versions can be added above this line>-------------------------------------------------
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};

		const static FGuid GUID;

		private:
		FSnapshotCustomVersion() = delete;
	};
}
