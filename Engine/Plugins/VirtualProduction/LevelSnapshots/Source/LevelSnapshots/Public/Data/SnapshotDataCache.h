// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"
#include "WorldSnapshotData.h"
#include "SnapshotDataCache.generated.h"

USTRUCT()
struct FActorSnapshotCache
{
	GENERATED_BODY()
	
	UPROPERTY()
	TWeakObjectPtr<AActor> CachedSnapshotActor = nullptr;

	/**
	 * Whether we already serialised the snapshot data into the actor.
	 * 
	 * This exists because sometimes we need to preallocate an actor without serialisation.
	 * Example: When serializing another actor which referenced this actor.
	 */
	UPROPERTY()
	bool bReceivedSerialisation = false;

	/**
	 * Stores all object dependencies. Only valid if bReceivedSerialisation == true.
	 */
	UPROPERTY()
	TArray<int32> ObjectDependencies;
};

USTRUCT()
struct FSubobjectSnapshotCache
{
	GENERATED_BODY()
	
	/** Allocated in snapshot world */
	UPROPERTY()
	TObjectPtr<UObject> SnapshotObject = nullptr;

	/** Allocated in editor world */
	UPROPERTY()
	TWeakObjectPtr<UObject> EditorObject = nullptr;
};

USTRUCT()
struct FClassDefaultSnapshotCache
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UObject> CachedLoadedClassDefault = nullptr;
};

/** Caches data for re-use. */
USTRUCT()
struct FSnapshotDataCache
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FSoftObjectPath, FActorSnapshotCache> ActorCache;

	UPROPERTY()
	TMap<FSoftObjectPath, FSubobjectSnapshotCache> SubobjectCache;

	/**
	 * Caches each archetype object for faster lookup.
	 * 
	 * Equal length as FWorldSnapshotData::ClassData and the indices correspond to another.
	 * Elements can be null in which case there is no cache, yet.
	 */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ArchetypeObjects;

	/**
	 * Fallback for ClassData. Equal length as ClassData and the indices correspond to another.
	 * When we fail to archetype data in ClassData, we look for the most appropriate archetype and cache it here.
	 * 
	 * Archetype data may be missing because this snapshot is old (was not saved) or because it was explicitly skipped
	 * by user config.
	 */
	TArray<TOptional<FClassSnapshotData>> FallbackArchetypeData;

	void InitFor(const FWorldSnapshotData& SnapshotData)
	{
		const int32 Size = SnapshotData.ClassData.Num();
		ArchetypeObjects.SetNum(Size);
		FallbackArchetypeData.SetNum(Size);
	}

	void Reset()
	{
		ActorCache.Reset();
		SubobjectCache.Reset();
		ArchetypeObjects.Reset();
		FallbackArchetypeData.Reset();
	}
};
