// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::LevelSnapshots
{
	/** An index to FWorldSnapshotData::ClassData. */
	using FClassDataIndex = uint32;
}