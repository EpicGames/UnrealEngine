// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SnapshotVersion.generated.h"

class FEngineVersion;
struct FCustomVersion;


/** Holds file version information */
USTRUCT()
struct LEVELSNAPSHOTS_API FSnapshotFileVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the compiled in data */
	void Initialize();

	FString ToString() const;

	/* UE4 File version */
	UPROPERTY()
	int32 FileVersionUE4 = VER_LATEST_ENGINE_UE4;

	/* UE5 File version */
	UPROPERTY()
	int32 FileVersionUE5 = 0;
	
	/* Licensee file version */
	UPROPERTY()
	int32 FileVersionLicensee = 0;
};

/** Holds engine version information */
USTRUCT()
struct LEVELSNAPSHOTS_API FSnapshotEngineVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the given version */
	void Initialize(const FEngineVersion& InVersion);

	FString ToString() const;

	/** Major version number */
	UPROPERTY()
	uint16 Major = 0;

	/** Minor version number */
	UPROPERTY()
	uint16 Minor = 0;

	/** Patch version number */
	UPROPERTY()
	uint16 Patch = 0;

	/** Changelist number. This is used to arbitrate when Major/Minor/Patch version numbers match */
	UPROPERTY()
	uint32 Changelist = 0;
};

/** Holds custom version information */
USTRUCT()
struct LEVELSNAPSHOTS_API FSnapshotCustomVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the given version */
	void Initialize(const FCustomVersion& InVersion);

	friend FArchive& operator<<(FArchive& Archive, FSnapshotCustomVersionInfo& VersionInfo)
	{
		Archive << VersionInfo.FriendlyName;
		return Archive;
	}
	
	/** Friendly name of the version */
	UPROPERTY()
	FName FriendlyName;

	/** Unique custom key */
	UPROPERTY()
	FGuid Key;

	/** Custom version */
	UPROPERTY()
	int32 Version = 0;
};

/** Holds version information for a session */
USTRUCT()
struct LEVELSNAPSHOTS_API FSnapshotVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the compiled in data */
	void Initialize(bool bWithoutSnapshotVersion = false);
	bool IsInitialized() const;

	void ApplyToArchive(FArchive& Archive) const;

	FString ToString() const;
	
	/** @return The saved version for this plugin or -1 if none is available. */
	int32 GetSnapshotCustomVersion() const;
	
	/** File version info */
	UPROPERTY()
	FSnapshotFileVersionInfo FileVersion;

	/** Engine version info */
	UPROPERTY()
	FSnapshotEngineVersionInfo EngineVersion;

	/** Custom version info */
	UPROPERTY()
	TArray<FSnapshotCustomVersionInfo> CustomVersions;
};
