// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ObjectSnapshotData.h"
#include "SnapshotUtilTypes.h"
#include "SubobjectSnapshotData.generated.h"

/** Data saved for subobjects, such as components. */
USTRUCT()
struct LEVELSNAPSHOTS_API FSubobjectSnapshotData : public FObjectSnapshotData
{
	GENERATED_BODY()
	
	static FSubobjectSnapshotData MakeSkippedSubobjectData()
	{
		FSubobjectSnapshotData Result;
		Result.bWasSkippedClass = true;
		return Result;
	}

	/** Index to FWorldSnapshotData::SerializedObjectReferences */
	UPROPERTY()
	int32 OuterIndex = INDEX_NONE;

	UPROPERTY()
	FSoftClassPath Class_DEPRECATED;
	
	/** Valid index to FWorldSnapshotData::ClassData. Use to lookup class and archetype data. */
    UPROPERTY()
    int32 ClassIndex = INDEX_NONE;

	/** Whether this object was marked as unsupported when the snapshot was taken */
	UPROPERTY()
	bool bWasSkippedClass = false;
	
	// "Type safety" for better self-documenting code
	UE::LevelSnapshots::FClassDataIndex GetClassDataIndex() const { return ClassIndex; }
};