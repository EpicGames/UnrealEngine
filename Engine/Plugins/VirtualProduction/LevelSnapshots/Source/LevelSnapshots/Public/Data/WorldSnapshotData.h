// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorSnapshotData.h"
#include "ComponentSnapshotData.h"
#include "ClassDefaultObjectSnapshotData.h"
#include "ClassSnapshotData.h"
#include "CustomSerializationData.h"
#include "SnapshotVersion.h"
#include "SubobjectSnapshotData.h"
#include "Templates/NonNullPointer.h"
#include "UObject/Class.h"
#include "WorldSnapshotData.generated.h"

class AActor;
class FSnapshotArchive;
struct FPropertySelectionMap;
struct FActorSnapshotData;

/** Holds saved world data. See WorldDataUtil for operations. */
USTRUCT()
struct LEVELSNAPSHOTS_API FWorldSnapshotData
{
	GENERATED_BODY()

	void ForEachOriginalActor(TFunctionRef<void(const FSoftObjectPath& ActorPath, const FActorSnapshotData& SavedData)> HandleOriginalActorPath) const;
	bool HasMatchingSavedActor(const FSoftObjectPath& OriginalObjectPath) const;
	
	//~ Begin TStructOpsTypeTraits Interface
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
	//~ End TStructOpsTypeTraits Interface
	
	/* The root world we will be adding deserialized snapshots actors to */
	UPROPERTY(Transient)
	TWeakObjectPtr<UWorld> SnapshotWorld;
	/** Sublevels of SnapshotWorld; excludes SnapshotWorld->PersistentLevel */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UWorld>> SnapshotSublevels;

	/**
	 * Stores versioning information we inject into archives.
	 * This is to support asset migration, like FArchive::UsingCustomVersion.
	 */
	UPROPERTY()
	FSnapshotVersionInfo SnapshotVersionInfo;
	
	
	
	/**
	 * We only save properties with values different from their CDO counterpart.
	 * Because of this, we need to save class defaults in the snapshot.
	 */
	UPROPERTY()
	TMap<FSoftClassPath, FClassDefaultObjectSnapshotData> ClassDefaults_DEPRECATED;

	/**
	 * Saves class info, such as archetype data, for every object's class.
	 * 
	 * Actor classes have exactly one entry FClassSnapshotData::FClassSnapshotData.
	 * For other classes, e.g. components, there may be multiple entries for the same class. This is because
	 * subobjects can have their own archtetypes (you can see this by looking a the reset to default button in the details panel).
	 *
	 * Added in 5.1.
	 */
	UPROPERTY()
	TArray<FClassSnapshotData> ClassData;


	
	/**
	 * Holds serialized actor data.
	 * Maps the original actor's path to its serialized data.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, FActorSnapshotData> ActorData;

	

	
	/** Whenever an object needs to serialize a name, we add it to this array and serialize an index to this array. */
	UPROPERTY()
	TArray<FName> SerializedNames;

	/**
	 * Whenever an object needs to serialize an object reference, we keep the object path here and serialize an index to this array.
	 * 
	 * External references, e.g. UDataAssets or UMaterials, are easily handled.
	 * Example: UStaticMesh /Game/Australia/StaticMeshes/MegaScans/Nature_Rock_vbhtdixga/vbhtdixga_LOD0.vbhtdixga_LOD0
	 * 
	 * Internal references, e.g. to subobjects and to other actors in the world, are a bit tricky.
	 * For internal references, we need to do some translation:
	 * Example original: UStaticMeshActor::StaticMeshComponent /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent
	 * Example translated: UStaticMeshActor::StaticMeshComponent /Engine/Transient.World_21:PersistentLevel.StaticMeshActor_42.StaticMeshComponent
	 */
	UPROPERTY()
	TArray<FSoftObjectPath> SerializedObjectReferences;
	
	/**
	 * Key: A valid index to SerializedObjectReferences.
	 * Value: Subobject information for the associated entry in SerializedObjectReferences.
	 * There is only an entry if the associated object is in fact a subobject. Actors and assets in particular do not get any entry.
	 *
	 * Added in 5.0.
	 */
	UPROPERTY()
	TMap<int32, FSubobjectSnapshotData> Subobjects;

	/**
	 * Key: A valid index to SerializedObjectReferences
	 * Value: Data that was generated by some ICustomObjectSnapshotSerializer.
	 * 
	 * Added in 5.0.
	 */
	UPROPERTY()
	TMap<int32, FCustomSerializationData> CustomSubobjectSerializationData;

	

	/** Binds every entry in SerializedNames to its index. Speeds up adding unique names. */
	UPROPERTY(Transient)
	TMap<FName, int32> NameToIndex;
	
	/** Binds every entry in SerializedObjectReferences to its index. Speeds up adding unique references. */
    UPROPERTY(Transient)
    TMap<FSoftObjectPath, int32> ReferenceToIndex;

	/** Binds every archetype object for which we saved the class data in ClassData to its index. Speeds up adding unique classes. */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UObject>, uint32> ArchetypeToClassDataIndex;
};

template<>
struct TStructOpsTypeTraits<FWorldSnapshotData> : public TStructOpsTypeTraitsBase2<FWorldSnapshotData>
{
	enum 
	{
		WithSerializer = true,
		WithPostSerialize = true
	};
};

