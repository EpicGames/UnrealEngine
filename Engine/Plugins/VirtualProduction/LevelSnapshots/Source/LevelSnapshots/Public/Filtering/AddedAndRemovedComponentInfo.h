// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class UActorComponent;

namespace UE::LevelSnapshots
{
	/* Holds all properties that should be restored for an object. */
	struct LEVELSNAPSHOTS_API FAddedAndRemovedComponentInfo
	{
		/**
		* New components added to the actor after the snapshot was taken. Remove them from the original actor.
		* Point to instances in the editor world.
		*/
		TSet<TWeakObjectPtr<UActorComponent>> EditorWorldComponentsToRemove;

		/**
		* Components that were removed from the actor after the snapshot was taken. Add them to the original actor.
		* Point to instances in the snapshot world.
		*/
		TSet<TWeakObjectPtr<UActorComponent>> SnapshotComponentsToAdd;
	};
}	