// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::LevelSnapshots
{
	/* Holds all properties that should be restored for an object. */
	struct LEVELSNAPSHOTS_API FCustomSubobjectRestorationInfo
	{
		/**
		* Subobjects from snapshot world which were discovered using ICustomObjectSnapshotSerializer.
		* Recreate the equivalent subobject in the editor world.
		*/
		TSet<TWeakObjectPtr<UObject>> CustomSnapshotSubobjectsToRestore; 
	};
}