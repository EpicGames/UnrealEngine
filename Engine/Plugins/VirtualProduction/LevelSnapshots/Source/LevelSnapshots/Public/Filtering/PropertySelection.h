// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/UnrealType.h"

/* Uniquely identifies a property across structs.
 * 
 * Primarily this is used by serialisation code hence we pass in FArchiveSerializedPropertyChain often.
 * However, inheritance from FArchiveSerializedPropertyChain is an implementation detail.
 */
struct LEVELSNAPSHOTS_API FLevelSnapshotPropertyChain : FArchiveSerializedPropertyChain
{
	using Super = FArchiveSerializedPropertyChain;
	friend struct FPropertySelection;

	using Super::GetPropertyFromStack;
	using Super::GetPropertyFromRoot;
	using Super::GetNumProperties;

	FLevelSnapshotPropertyChain() = default;
	FLevelSnapshotPropertyChain(const FProperty* RootProperty);
	FLevelSnapshotPropertyChain(const FArchiveSerializedPropertyChain* ChainToLeaf, const FProperty* LeafProperty);
	
	FLevelSnapshotPropertyChain MakeAppended(const FProperty* Property) const;
	void AppendInline(const FProperty* Property);

	/** Returns an FLevelSnapshotPropertyChain given a leaf property and a UScriptStruct or UClass. */
	static TOptional<FLevelSnapshotPropertyChain> FindPathToProperty(
		const FProperty* InLeafProperty, const UStruct* InStructToSearch, const bool bIncludeLeafPropertyInChain = true);

	FString ToString() const;

	/**
	 * Checks whether a given property being serialized corresponds to this chain.
	 *
	 * @param ContainerChain The chain of properties to the most nested owning struct: See FArchive::GetSerializedPropertyChain.
	 * @param LeafProperty The leaf property in the struct
	 */
	bool EqualsSerializedProperty(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const;

	bool IsEmpty() const;
	bool operator==(const FLevelSnapshotPropertyChain& InPropertyChain) const;
};

/* Holds all properties that should be restored for an object. */
struct LEVELSNAPSHOTS_API FPropertySelection
{
	FPropertySelection() = default;
	FPropertySelection(const FProperty* SingleProperty);
	/* Note: all properties must be root properties (i.e. not within structs or collections). Check yourself, this is not checked. */
	FPropertySelection(const TSet<const FProperty*>& Properties);
	
	/** 
	 * Checks whether the given property should be serialized. It should be serialized if:
	 * - IsPropertySelected() returns true on the property
	 * - The property is inside of a collection for which IsPropertySelected() returns true
	 * - The property is not part of a struct for which IsPropertySelected() returns true; this happens when a struct implements a custom serializer and pushes other structs.
	 *
	 * As performance optimisation, we assume this function is called by FArchive::ShouldSkipProperty,
	 * i.e. ShouldSerializeProperty would return true on the elements of ContainerChain.
	 *
	 * @param ContainerChain The chain of properties to the most nested owning struct: See FArchive::GetSerializedPropertyChain.
	 * @param LeafProperty The leaf property in the struct
	 */
	bool ShouldSerializeProperty(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const;
	/**
	 * Checks whether the given property is in this selection. When serializing, you probably want to be using ShouldSerializeProperty.
	 *
	 * @param ContainerChain The chain of properties to the most nested owning struct: See FArchive::GetSerializedPropertyChain.
	 * @param LeafProperty The leaf property in the struct
	 */
	bool IsPropertySelected(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const;

	bool IsEmpty() const;

	void SetHasCustomSerializedSubobjects(bool bValue) { bHasCustomSerializedSubobjects = bValue; }
	bool HasCustomSerializedSubobjects() const { return bHasCustomSerializedSubobjects; }

	void AddProperty(const FLevelSnapshotPropertyChain& SelectedProperty);
	void RemoveProperty(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty);

	void RemoveProperty(FArchiveSerializedPropertyChain* ContainerChain);

	/* Gets a flat list of all selected properties. The result contains no information what nested struct a property came from. */
	const TArray<TFieldPath<FProperty>>& GetSelectedLeafProperties() const;

	const TArray<FLevelSnapshotPropertyChain>& GetSelectedProperties() const;

private:

	int32 FindPropertyChain(const FArchiveSerializedPropertyChain* ContainerChain, const FProperty* LeafProperty) const;

	/** Whether some ICustomObjectSnapshotSerializer has changed subobjects */
	bool bHasCustomSerializedSubobjects = false;
	
	/* Duplicate version of SelectedProperties with the struct-path leading to the property left out. Needed to build UI more easily. */
	TArray<TFieldPath<FProperty>> SelectedLeafProperties;

	/* These are the properies that need to be restored.
	 * 
	 * Key: First property name of property in chain.
	 * Value: All chains associated that have the Key property has first property.
	 * Mapping it out like this allows us to search less properties.
	 */
	TArray<FLevelSnapshotPropertyChain> SelectedProperties;
};
		
		

		