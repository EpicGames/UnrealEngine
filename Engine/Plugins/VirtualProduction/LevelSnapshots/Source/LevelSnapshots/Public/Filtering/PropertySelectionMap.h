// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Filtering/AddedAndRemovedComponentInfo.h"
#include "Filtering/CustomSubobjectRestorationInfo.h"
#include "Filtering/PropertySelection.h"
#include "Filtering/RestorableObjectSelection.h"
#include "GameFramework/Actor.h"

class UActorComponent;

struct FPropertySelectionMap;
	
/* Binds an object to its selected properties */
struct LEVELSNAPSHOTS_API FPropertySelectionMap
{
    friend UE::LevelSnapshots::FRestorableObjectSelection;

    /*************** Common features ***************/
    
    /* Respawn the actor from the data in the snapshot. */
    void AddDeletedActorToRespawn(const FSoftObjectPath& Original);
    void RemoveDeletedActorToRespawn(const FSoftObjectPath& Original);

    /* Destroy the given actors when a snapshot is applied. */
    void AddNewActorToDespawn(AActor* WorldActor);
    void RemoveNewActorToDespawn(AActor* WorldActor);

    /**
     * Binds properties to an object which are supposed to be rolled back.
     */
    bool AddObjectProperties(UObject* WorldObject, const FPropertySelection& SelectedProperties);
    void RemoveObjectPropertiesFromMap(UObject* WorldObject);

    
    /*************** Basic subobject support ***************/
    
    /**
     * Tracks a subobject so we only restore references pointing to it but not serialize its properties.
     *
     * When applying a snapshot to world and resolving subobjects, it is unclear whether a resolved references is either
     *		1. a reference to a preexisting, valid subobject: for it we use only restored selected properties.
     *		2. a reference to an old, deleted subobject which still exists in memory: for it we restore all properties.
     * 
     * See SnapshotUtil::Object::ResolveObjectDependencyForEditorWorld for more info.
     */
    void MarkSubobjectForRestoringReferencesButSkipProperties(UObject* WorldSubbject);
    bool IsSubobjectMarkedForReferenceRestorationOnly(const FSoftObjectPath& WorldSubobjectPath) const;


    void AddComponentSelection(AActor* EditorWorldActor, const UE::LevelSnapshots::FAddedAndRemovedComponentInfo& ComponentSelection);
    void RemoveComponentSelection(AActor* EditorWorldActor);

    /*************** Custom subobject support ***************/

    /** Recreate a Custom subobject that exists in snapshot world but is missing from editor world. Used for subobjects discovered via ICustomObjectSnapshotSerializer */
    void AddCustomEditorSubobjectToRecreate(UObject* EditorWorldOwner, UObject* SnapshotSubobject);
    void RemoveCustomEditorSubobjectToRecreate(UObject* EditorWorldOwner, UObject* SnapshotSubobject);

    /*************** Queries ***************/

    UE::LevelSnapshots::FRestorableObjectSelection GetObjectSelection(const FSoftObjectPath& EditorWorldObject) const;
    TArray<FSoftObjectPath> GetKeys() const;
	void ForEachModifiedObject(TFunctionRef<void(const FSoftObjectPath&)> Callback) const;
	bool HasAnyModifiedActors() const;

    bool HasChanges(AActor* EditorWorldActor) const;
    bool HasChanges(UActorComponent* EditorWorldComponent) const;
    bool HasChanges(UObject* EditorWorldObject) const;
    
    int32 GetKeyCount() const { return EditorWorldObjectToSelectedProperties.Num(); }
    const TSet<FSoftObjectPath>& GetDeletedActorsToRespawn() const { return DeletedActorsToRespawn; }
    const TSet<TWeakObjectPtr<AActor>>& GetNewActorsToDespawn() const { return NewActorsToDespawn; }

    void Empty(bool bCanShrink = true);

    /* Gets the direct subobjects of root that have selected properties. You can recursively call this function with the element of the result array. */
    TArray<UObject*> GetDirectSubobjectsWithProperties(UObject* Root) const;
    
private:
    
    /**
     * Maps an editor world actor to the properties that should be restored to values in a snapshot.
     * The properties are located in the actor itself. Subobjects have a separate key.
     */
    TMap<FSoftObjectPath, FPropertySelection> EditorWorldObjectToSelectedProperties;

    /**
     * Maps an editor world actor to the components that should be added or removed.
     */
    TMap<FSoftObjectPath, UE::LevelSnapshots::FAddedAndRemovedComponentInfo> EditorActorToComponentSelection;

    /**
     * Maps an editor world object to the custom subobjects discovered by ICustomObjectSnapshotSerializer which need restoring.
     */
    TMap<FSoftObjectPath, UE::LevelSnapshots::FCustomSubobjectRestorationInfo> EditorWorldObjectToCustomSubobjectSelection;
    
    /**
     * Contains objects which we do not serialize into: only replace references to them.
     */
    TSet<FSoftObjectPath> SubobjectsMarkedForReferencesRestorationOnly;

    /**
     * These actors were removed since the snapshot was taken. Re-create them.
     * This contains the original objects paths stored in the snapshot.
     */
    TSet<FSoftObjectPath> DeletedActorsToRespawn;

    /** These actors were added since the snapshot was taken. Remove them. */
    TSet<TWeakObjectPtr<AActor>> NewActorsToDespawn;
};