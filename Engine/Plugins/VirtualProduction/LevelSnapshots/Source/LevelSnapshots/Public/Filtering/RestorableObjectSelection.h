// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"
#include "AddedAndRemovedComponentInfo.h"
#include "PropertySelection.h"

class AActor;

struct FPropertySelectionMap;

namespace UE::LevelSnapshots
{
	struct FCustomSubobjectRestorationInfo;
	
	class LEVELSNAPSHOTS_API FRestorableObjectSelection
	{
		friend FPropertySelectionMap; 

		const FSoftObjectPath ObjectPath;
		const FPropertySelectionMap& Owner;

		FRestorableObjectSelection(const FSoftObjectPath& ObjectPath, const FPropertySelectionMap& Owner)
			:
			ObjectPath(ObjectPath),
			Owner(Owner)
		{}

		public:
		
		const FPropertySelection* GetPropertySelection() const;
		const FAddedAndRemovedComponentInfo* GetComponentSelection() const;
		const FCustomSubobjectRestorationInfo* GetCustomSubobjectSelection() const;
	};
}