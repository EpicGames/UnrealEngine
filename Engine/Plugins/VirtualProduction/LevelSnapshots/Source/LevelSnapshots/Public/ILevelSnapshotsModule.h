// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IRestorationListener.h"
#include "Interfaces/ISnapshotLoader.h"
#include "Interfaces/IPropertyComparer.h"
#include "Interfaces/ISnapshotRestorabilityOverrider.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "Interfaces/IActorSnapshotFilter.h"
#include "Interfaces/ISnapshotFilterExtender.h"
#include "Interfaces/ITakeSnapshotListener.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class ULevelSnapshot;

namespace UE::LevelSnapshots
{
	struct LEVELSNAPSHOTS_API FPreTakeSnapshotEventData
	{
		ULevelSnapshot* Snapshot;
		UWorld* World;
	};

	struct LEVELSNAPSHOTS_API FPostTakeSnapshotEventData
	{
		ULevelSnapshot* Snapshot;
		UWorld* World;
	};

	class LEVELSNAPSHOTS_API ILevelSnapshotsModule : public IModuleInterface
	{
	public:

		static ILevelSnapshotsModule& Get()
		{
			return FModuleManager::Get().GetModuleChecked<ILevelSnapshotsModule>("LevelSnapshots");
		}

		static bool IsAvailable()
		{
			return FModuleManager::Get().IsModuleLoaded("LevelSnapshots");
		}
		

		DECLARE_EVENT_OneParam(ILevelSnapshotsModule, FPreTakeSnapshotEvent, const FPreTakeSnapshotEventData&);
		/** Called before a snapshot is taken. */
		FPreTakeSnapshotEvent& OnPreTakeSnapshot() { return PreTakeSnapshot; }
		
		DECLARE_EVENT_OneParam(ILevelSnapshotsModule, FPostTakeSnapshotEvent, const FPostTakeSnapshotEventData&);
		/** Called after a snapshot is taken */
		FPostTakeSnapshotEvent& OnPostTakeSnapshot() { return PostTakeSnapshot; }

		
		DECLARE_DELEGATE_RetVal_OneParam(bool, FCanTakeSnapshot, const FPreTakeSnapshotEventData&);
		/** Add a named delegate that determines if we can take a snapshot. */
		virtual void AddCanTakeSnapshotDelegate(FName DelegateName, FCanTakeSnapshot Delegate) = 0;

		/** Remove previously added named delegate that determines if we can take a snapshot. */
		virtual void RemoveCanTakeSnapshotDelegate(FName DelegateName) = 0;

		/** Queries the attached snapshot delegate and determines if we can take a snapshot.*/
		virtual bool CanTakeSnapshot(const FPreTakeSnapshotEventData& Event) const = 0;
		

		/** Snapshots will no longer capture nor restore subobjects of this class. Subclasses are implicitly skipped as well. */
		virtual void AddSkippedSubobjectClasses(const TSet<UClass*>& Classes) = 0;
		virtual void RemoveSkippedSubobjectClasses(const TSet<UClass*>& Classes) = 0;
		
		
		/* Registers callbacks that override which actors, components, and properties are restored by default. */
		virtual void RegisterRestorabilityOverrider(TSharedRef<ISnapshotRestorabilityOverrider> Overrider) = 0;
		/* Unregisters an overrider previously registered. */
		virtual void UnregisterRestorabilityOverrider(const TSharedRef<ISnapshotRestorabilityOverrider>& Overrider) = 0;
		
		
		/* Registers a callback for deciding whether a property should be considered changed. Applies to all sub-classes. */
		virtual void RegisterPropertyComparer(UClass* Class, TSharedRef<IPropertyComparer> Comparer) = 0;
		virtual void UnregisterPropertyComparer(UClass* Class, const TSharedRef<IPropertyComparer>& Comparer) = 0;
		
		
		/**
		 * Registers callbacks for snapshotting / restoring certain classes. There can only be one per class. The typical use case using  Level Snapshots for restoring subobjects
		 * you want recreate / find manually.
		 *
		 * @param Class The class to register. This must be a native class (because Blueprint classes may be reinstanced when recompiled - not supported atm).
		 * @param CustomSerializer Your callbacks
		 * @param bIncludeBlueprintChildClasses Whether to use 'CustomSerializer' for Blueprint child classes of 'Class'
		 */
		virtual void RegisterCustomObjectSerializer(UClass* Class, TSharedRef<ICustomObjectSnapshotSerializer> CustomSerializer, bool bIncludeBlueprintChildClasses = true) = 0;
		virtual void UnregisterCustomObjectSerializer(UClass* Class) = 0;
		
		
		/** Registers callbacks for external systems to decide whether a given actor can be modified, recreated, or removed from the world. */
		virtual void RegisterGlobalActorFilter(TSharedRef<IActorSnapshotFilter> Filter) = 0;
		virtual void UnregisterGlobalActorFilter(const TSharedRef<IActorSnapshotFilter>& Filter) = 0;


		/** Register an object that will receive callbacks when a snapshot is loaded */
		virtual void RegisterSnapshotLoader(TSharedRef<ISnapshotLoader> Loader) = 0;
		virtual void UnregisterSnapshotLoader(const TSharedRef<ISnapshotLoader>& Loader) = 0;
		
		
		/** Registers an object that will receive callbacks when a snapshot is applied. */
		virtual void RegisterRestorationListener(TSharedRef<IRestorationListener> Listener) = 0;
		virtual void UnregisterRestorationListener(const TSharedRef<IRestorationListener>& Listener) = 0;

		
		/** Registers an object that listens for when objects are snapshot. */
		virtual void RegisterTakeSnapshotListener(TSharedRef<ITakeSnapshotListener> Extender) = 0;
		virtual void UnregisterTakeSnapshotListener(const TSharedRef<ITakeSnapshotListener>& Listener) = 0;
		

		/** Registers an object that can decide to display additional properties to the user that default snapshot behaviour does not display. */
		virtual void RegisterSnapshotFilterExtender(TSharedRef<ISnapshotFilterExtender> Extender) = 0;
		virtual void UnregisterSnapshotFilterExtender(const TSharedRef<ISnapshotFilterExtender>& Listener) = 0;

		
		/**
		 * Adds properties that snapshots will capture and restore from now on. This allows support for properties that are skipped by default.
		 * Important: Only add add native properties; Blueprint properties may be invalidated (and left dangeling) when recompiled.
		 */
		virtual void AddExplicitilySupportedProperties(const TSet<const FProperty*>& Properties) = 0;
		virtual void RemoveAdditionallySupportedProperties(const TSet<const FProperty*>& Properties) = 0;

		/**
		 * Stops snapshots from capturing / restoring these properties.
		 * Important: Only add add native properties; Blueprint properties may be invalidated (and left dangeling) when recompiled.
		 */
		virtual void AddExplicitlyUnsupportedProperties(const TSet<const FProperty*>& Properties) = 0;
		virtual void RemoveExplicitlyUnsupportedProperties(const TSet<const FProperty*>& Properties) = 0;


		/**
		 * Disable CDO serialization for a class and all of its subclasses.
		 *
		 * Snapshots saves the CDO of every saved object class.
		 * Actors use it as Template when spawned. For all other objects (i.e. subobjects), Serialize(FArchive&) is called with the saved CDO data and then
		 * Serialize(FArchive&) is called again with the actual object's data. This is so changes to CDOs can be detected.
		 *
		 * This function disables the above process. You would use it e.g. when your class implements a custom Serialize(FArchive&) function which
		 * conditionally serializes data depending on whether the serialized object has the RF_ClassDefaultObject flag. Doing so would trigger an ensure in
		 * the snapshot archive code because the data would not be read in the same order as it was written.
		 *
		 * Note: Level Snapshots will no longer detect changes made to the class default values of skipped classes.
		 */
		virtual void AddSkippedClassDefault(const UClass* Class) = 0;
		virtual void RemoveSkippedClassDefault(const UClass* Class) = 0;
		virtual bool ShouldSkipClassDefaultSerialization(const UClass* Class) const = 0;

	protected:

		FPreTakeSnapshotEvent PreTakeSnapshot;
		FPostTakeSnapshotEvent PostTakeSnapshot;
	};
}