// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Templates/SharedPointer.h"
#include "LevelSnapshotsEngineSubsystem.generated.h"

class ULevelSnapshot;
namespace UE::LevelSnapshots
{
	class IRestorationListener;
	struct FPreTakeSnapshotEventData;
	struct FPostTakeSnapshotEventData;
}

// Event data is extracted to structs so event delegate signature does not need to be modified in the future.
// Please follow this pattern when you add new events.

USTRUCT(BlueprintType, DisplayName = "LevelSnapshotEvent")
struct LEVELSNAPSHOTS_API FLevelSnapshotEvent_Blueprint
{
	GENERATED_BODY()

	/** The affected snapshot */
	UPROPERTY(BlueprintReadOnly, Category = "Level Snapshots")
	TObjectPtr<ULevelSnapshot> Snapshot;
};

USTRUCT(DisplayName = "PreTakeSnapshotEventData")
struct LEVELSNAPSHOTS_API FPreTakeSnapshotEventData_Blueprint : public FLevelSnapshotEvent_Blueprint { GENERATED_BODY() };
USTRUCT(DisplayName = "PostTakeSnapshotEventData")
struct LEVELSNAPSHOTS_API FPostTakeSnapshotEventData_Blueprint : public FLevelSnapshotEvent_Blueprint { GENERATED_BODY() };

USTRUCT(DisplayName = "PreApplySnapshotEventData")
struct LEVELSNAPSHOTS_API FPreApplySnapshotEventData_Blueprint : public FLevelSnapshotEvent_Blueprint { GENERATED_BODY() };
USTRUCT(DisplayName = "PostApplySnapshotEventData")
struct LEVELSNAPSHOTS_API FPostApplySnapshotEventData_Blueprint : public FLevelSnapshotEvent_Blueprint { GENERATED_BODY() };

/** Exposes some native Level Snapshots events to Blueprint scripting.  */
UCLASS()
class LEVELSNAPSHOTS_API ULevelSnapshotsEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPreTakeSnapshotEvent, const FPreTakeSnapshotEventData_Blueprint&, EventData);
	/** Called before a level snapshot captures the world's data. */
	UPROPERTY(BlueprintAssignable, Category = "Level Snapshots")
	FPreTakeSnapshotEvent OnPreTakeSnapshot;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPostTakeSnapshotEvent, const FPostTakeSnapshotEventData_Blueprint&, EventData);
	/** Called after a level snapshot captures the world's data. */
	UPROPERTY(BlueprintAssignable, Category = "Level Snapshots")
	FPostTakeSnapshotEvent OnPostTakeSnapshot;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPreApplySnapshotEvent, const FPreApplySnapshotEventData_Blueprint&, EventData);
	/** Called before a level snapshot is applied to a world. */
	UPROPERTY(BlueprintAssignable, Category = "Level Snapshots")
	FPreApplySnapshotEvent OnPreApplySnapshot;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPostApplySnapshotEvent, const FPostApplySnapshotEventData_Blueprint&, EventData);
	/** Called after a level snapshot was applied to a world. */
	UPROPERTY(BlueprintAssignable, Category = "Level Snapshots")
	FPostApplySnapshotEvent OnPostApplySnapshot;

	//~ Begin USubsystem Interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem Interface

private:

	/** Exposes some events to Blueprints. Created by Initialize and Reset() by Deinitialize. */
	TSharedPtr<UE::LevelSnapshots::IRestorationListener> RestorationListener;

	void HandleOnPreTakeSnapshot(const UE::LevelSnapshots::FPreTakeSnapshotEventData& EventData);
	void HandleOnPostTakeSnapshot(const UE::LevelSnapshots::FPostTakeSnapshotEventData& EventData);
};
