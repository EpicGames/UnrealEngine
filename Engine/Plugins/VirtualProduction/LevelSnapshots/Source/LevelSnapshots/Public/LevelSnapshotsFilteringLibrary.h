// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LevelSnapshotsFilteringLibrary.generated.h"

class ULevelSnapshotFilter;
class ULevelSnapshot;

struct FPropertySelectionMap;

/**
 * 
 */
UCLASS()
class LEVELSNAPSHOTS_API ULevelSnapshotsFilteringLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Diffs the snapshot with the given world and returns the changes that pass the filter. */
	static FPropertySelectionMap DiffAndFilterSnapshot(UWorld* InWorld, ULevelSnapshot* InSnapshot, const ULevelSnapshotFilter* Filter = nullptr);
	
	/**
	 * Goes through the properties of the actors and their components calling IsPropertyValid on them.
	 * This function does not recursively check object references, e.g. 'Instanced' uproperties. These properties are currently unsupported by the snapshot framework.
	 */
	static void ApplyFilterToFindSelectedProperties(
		ULevelSnapshot* Snapshot,
		FPropertySelectionMap& MapToAddTo,
		AActor* WorldActor,
		AActor* DeserializedSnapshotActor,
		const ULevelSnapshotFilter* Filter
		);
};
