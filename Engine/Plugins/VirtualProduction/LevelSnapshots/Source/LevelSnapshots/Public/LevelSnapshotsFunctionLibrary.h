// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filtering/PropertySelectionMap.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LevelSnapshotsFunctionLibrary.generated.h"

class ULevelSnapshot;
class ULevelSnapshotFilter;

UCLASS()
class LEVELSNAPSHOTS_API ULevelSnapshotsFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "Level Snapshots", meta = (WorldContext = "WorldContextObject"))
	static ULevelSnapshot* TakeLevelSnapshot(const UObject* WorldContextObject, const FName NewSnapshotName = "NewLevelSnapshot", const FString Description = "");
	
	static ULevelSnapshot* TakeLevelSnapshot_Internal(const UObject* WorldContextObject, const FName NewSnapshotName = "NewLevelSnapshot", UPackage* InPackage = nullptr, const FString Description = "");

	/* Applies the snapshot to the world. If no filter is specified, the entire snapshot is applied. */
	UFUNCTION(BlueprintCallable, Category = "Level Snapshots", meta = (WorldContext = "WorldContextObject"))
	static void ApplySnapshotToWorld(const UObject* WorldContextObject, ULevelSnapshot* Snapshot, ULevelSnapshotFilter* OptionalFilter);
	
	/**
	 * Goes through the properties of the actors and their components calling IsPropertyValid on them.
	 * This function does not recursively check object references, e.g. 'Instanced' uproperties. These properties are currently unsupported by the snapshot framework.
	 */
	UE_DEPRECATED(5.1, "Use ULevelSnapshotsFilteringLibrary::ApplyFilterToFindSelectedProperties instead.")
	static void ApplyFilterToFindSelectedProperties(
		ULevelSnapshot* Snapshot,
		FPropertySelectionMap& MapToAddTo,
		AActor* WorldActor,
		AActor* DeserializedSnapshotActor,
		const ULevelSnapshotFilter* Filter,
		bool bAllowUnchangedProperties = false,
		bool bAllowNonEditableProperties = false
		);

	/**
	 * Recursively gets all subobjects which should be serialized using an ICustomObjectSnapshotSerializer implementation from an external module.
	 *
	 * @param Snapshot The snapshot in which to look
	 * @param SnapshotRootObject Snapshot version of the object to look in. Either an actor, component, or a custom serialized subobject.
	 * @param WorldRootObject Editor version of the object to look in. Either an actor, component, or a custom serialized subobject.
	 * @param HandleCustomSubobjectPair Called when two custom subobjects are matched
	 * @param HandleUnmatchedSnapshotSubobject Called when a custom snapshot subobject exists but no corresponding editor subobject exists
	 */
	static void ForEachMatchingCustomSubobjectPair(
		ULevelSnapshot* Snapshot, 
		UObject* SnapshotRootObject,
		UObject* WorldRootObject,
		TFunctionRef<void(UObject* SnapshotSubobject, UObject* EditorWorldSubobject)> HandleCustomSubobjectPair,
		TFunctionRef<void(UObject* UnmatchedSnapshotSubobject)> HandleUnmatchedSnapshotSubobject
		);
};