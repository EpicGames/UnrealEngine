// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
UE_TRACE_CHANNEL_EXTERN(LevelSnapshotsChannel, LEVELSNAPSHOTS_API);
#define SCOPED_SNAPSHOT_TRACE(TraceName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TraceName, LevelSnapshotsChannel)
#else
#define SCOPED_SNAPSHOT_TRACE(TraceName)
#endif

// Use inside 'implementation' modules, like this one or filters
#define SCOPED_SNAPSHOT_CORE_TRACE(TraceName) SCOPED_SNAPSHOT_TRACE(SnapshotCore_##TraceName)
// Use inside editor modules
#define SCOPED_SNAPSHOT_EDITOR_TRACE(TraceName) SCOPED_SNAPSHOT_TRACE(SnapshotEditor_##TraceName)

DECLARE_LOG_CATEGORY_CLASS(LogLevelSnapshots, Log, All);
