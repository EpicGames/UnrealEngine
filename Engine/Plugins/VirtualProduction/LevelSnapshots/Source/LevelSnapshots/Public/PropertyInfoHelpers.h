// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class FProperty;
class FNumericProperty;
class UObject;

namespace UE::LevelSnapshots
{
	extern float GFloatComparisonPrecision;
	extern double GDoubleComparisonPrecision;

	LEVELSNAPSHOTS_API FProperty* GetParentProperty(const FProperty* Property);

	/* Returns true if the property is a struct, map, array, set, etc */
	LEVELSNAPSHOTS_API bool IsPropertyContainer(const FProperty* Property);
	/* Returns true if the property is a set, array, map, etc but NOT a struct */
	LEVELSNAPSHOTS_API bool IsPropertyCollection(const FProperty* Property);

	/* Is the property an element of a struct, set, array, map, etc */
	LEVELSNAPSHOTS_API bool IsPropertyInContainer(const FProperty* Property);
	/* Is the property an element of a set, array, map, etc */
	LEVELSNAPSHOTS_API bool IsPropertyInCollection(const FProperty* Property);

	LEVELSNAPSHOTS_API bool IsPropertyInStruct(const FProperty* Property);

	LEVELSNAPSHOTS_API bool IsPropertyInMap(const FProperty* Property);

	/* A quick property flag check. Assumes the property flags are properly set/deserialized */
	LEVELSNAPSHOTS_API bool IsPropertyComponentOrSubobject(const FProperty* Property);

	LEVELSNAPSHOTS_API void UpdateDecimalComparisionPrecision(float FloatPrecision, double DoublePrecision);
	LEVELSNAPSHOTS_API bool AreNumericPropertiesNearlyEqual(const FNumericProperty* Property, const void* ValuePtrA, const void* ValuePtrB);
}


