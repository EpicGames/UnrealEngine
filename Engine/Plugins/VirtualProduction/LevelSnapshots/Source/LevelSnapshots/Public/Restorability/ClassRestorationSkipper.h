// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/ISnapshotRestorabilityOverrider.h"
#include "Settings/SkippedClassList.h"

namespace UE::LevelSnapshots
{
	/* Disallows provided classes. Uses callback to obtain class list so the logic is reusable outside the module. */
	class LEVELSNAPSHOTS_API FClassRestorationSkipper : public ISnapshotRestorabilityOverrider
	{
	public:

		DECLARE_DELEGATE_RetVal(const FSkippedClassList&, FGetSkippedClassList)
		
		FClassRestorationSkipper(FGetSkippedClassList SkippedClassListCallback)
			:
			GetSkippedClassListCallback(SkippedClassListCallback)
		{
			check(SkippedClassListCallback.IsBound());
		}
		
		//~ Begin ISnapshotRestorabilityOverrider Interface
		virtual ERestorabilityOverride IsActorDesirableForCapture(const AActor* Actor) override;
		virtual ERestorabilityOverride IsComponentDesirableForCapture(const UActorComponent* Component) override;
		//~ End ISnapshotRestorabilityOverrider Interface
	
	private:

		FGetSkippedClassList GetSkippedClassListCallback;

	};
}


