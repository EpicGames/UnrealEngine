// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/NonNullPointer.h"
#include "UObject/SoftObjectPath.h"

class AActor;
class UObject;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots
{
	using FDeserializedActorGetter = TFunctionRef<TOptional<TNonNullPtr<AActor>>()>;

	/** Params for deciding whether an actor should be recreated */
	struct FCanRecreateActorParams
	{
		/** The world the snapshot is being applied to */
		UWorld* EditorWorld;

		/** The actor's class */
		UClass* Class;
		/** The object path of the actor */
		FSoftObjectPath ActorPath;
		/** The flags of the actor */
		EObjectFlags ObjectFlags;
		/** All world data of the snapshot */
		const FWorldSnapshotData& WorldData;
		
		/** If you need access to the actor's data, you can call this function. */
		FDeserializedActorGetter DeserializeFromSnapshotFunc;
	};

	/** Params for deciding whether an existing actor should have its properties modified. */
	struct FCanModifyMatchedActorParams
	{
		/** The actor in the editor world which is being considered for restoration. */
		AActor& MatchedEditorWorldActor; 
		/** All world data of the snapshot */
		const FWorldSnapshotData& WorldData;
	};
	
	/**
	 * Can decide whether a given actor can be added to or removed from the world.
	 */
	class LEVELSNAPSHOTS_API IActorSnapshotFilter
	{
	public:

		enum class EFilterResult
		{
			/* The object in question is included but only if nobody else returned Disallow.  */
			Allow,
			/* Let other filters decide. If every override return DoNotCare, default snapshot behaviour is applied. */
			DoNotCare,
			/* The object in question is never suitable and is not included. Other filters cannot override this. */
			Disallow
		};

		struct FFilterResultData
		{
			/** Determines what to do with the actor data */
			EFilterResult InclusionResult;
			/** The reason for InclusionResult == Disallow, if any. Displayed to the user. */
			FText ExclusionReason;

			FFilterResultData() = default;
			FFilterResultData(EFilterResult Result, FText ExclusionReason = FText::GetEmpty()) : InclusionResult(Result), ExclusionReason(MoveTemp(ExclusionReason)) {}
		};
		
		/** Called to check whether a matched actor is allowed to have any of its data restored. */
		virtual FFilterResultData CanModifyMatchedActor(const FCanModifyMatchedActorParams& Params) { return EFilterResult::DoNotCare; }

		/** Whether the actor is allowed to be recreated. Called when diffing to world. */
		virtual FFilterResultData CanRecreateMissingActor(const FCanRecreateActorParams& Params)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return CanRecreateActor(Params);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		/** Whether the actor is allowed to be removed. Called when diffing to world. */
		virtual FFilterResultData CanDeleteNewActor(const AActor* EditorActor)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return CanDeleteActor(EditorActor);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
		
		/** Whether the actor is allowed to be recreated. Called when diffing to world. */
		UE_DEPRECATED(5.2, "Use CanRecreateMissingActor instead.")
		virtual EFilterResult CanRecreateActor(const FCanRecreateActorParams& Params) { return EFilterResult::DoNotCare; }

		/** Whether the actor is allowed to be removed. Called when diffing to world. */
		UE_DEPRECATED(5.2, "Use CanDeleteNewActor instead.")
		virtual EFilterResult CanDeleteActor(const AActor* EditorActor) { return EFilterResult::DoNotCare; }
		
		
		virtual ~IActorSnapshotFilter() = default;
	};
}