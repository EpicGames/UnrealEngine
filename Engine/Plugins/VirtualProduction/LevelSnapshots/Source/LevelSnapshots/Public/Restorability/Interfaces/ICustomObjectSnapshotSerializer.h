// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"

class UObject;
struct FPropertySelectionMap;

namespace UE::LevelSnapshots
{
	class ISnapshotSubobjectMetaData;
	class ICustomSnapshotSerializationData;

	/**
	 * External modules can implement this interface to customise how specific classes are snapshot and restored.
	 * Implementations of this interface can be registered with the Level Snapshots module.
	 * 
	 * One instance handles on type of class.
	 * 
	 * ISnapshotObjectSerializer handles the serialisation of the object you're registered to. You can use it to add custom
	 * annotation data you need to restoring object info. You can also save & restore subobjects you wish to manually restore,
	 * provided they're not automatically restored by Level Snapshots. The following subobjects are handled by default:
	 *  - Actor components
	 *  - UProperty exposed UObject* references or collections with CPF_Edit flag, e.g. e.g. UPROPERTY(EditAnywhere) TArray<UObject*>.
	 */
	class LEVELSNAPSHOTS_API ICustomObjectSnapshotSerializer
	{
	public:

		/**
		 * Called when taking a snapshot of an object with the class this implementation is registered to.
		 *
		 * You can use DataStorage to add any data additional meta data needed and add subobjects you want to restore manually.
		 * Note that all uproperties will still be restored normally as with all other objects.
		 *
		 * @param EditorObject The object being snapshotted. Same as DataStorage->GetSerializedObject().
		 * @param DataStorage Data to save for EditorObject: serialize extra data about EditorObject and add subobjects.  
		 */
		virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) {}


		
	    /**
	     * Called when creating objects for the temporary snapshot world. This is called for every subobject added using ISnapshotObjectSerializer::AddSubobjectDependency.
	     * 
	     * This function must either find the subobject in SnapshotObject or recreate it. If the object is recreated, you must fix up any property references yourself.
	     * After this function is called, properties will be serialized into this function's return value. After this, OnPostSerializeSnapshotSubobject is called.
	     *
	     * @param SnapshotObject The outer of the subobject
	     * @param ObjectData The data saved for the subobject. It's the data associated with the index returned by AddSubobjectSnapshot.
	     * @param DataStorage The data saved for the outer
	     * 
	     * @result The found or recreated subobject. If null, the subobject is skipped.
	     */
		virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) 
		{ checkf(false, TEXT("If you register subobjects in OnTakeSnapshot, you must implement this function.")); return nullptr; }
		
		/**
		 * Called when applying into the editor world. This is called for every subobject added using ISnapshotObjectSerializer::AddSubobjectDependency.
		 * 
		 * This function must either find the subobject in EditorObject or recreate it. If the object is recreated, you must fix up any property references yourself.
		 * After this function is called, properties will be serialized into this function's return value. After this, OnPostSerializeEditorSubobject is called.
		 *
		 * @param SnapshotObject The outer of the subobject
		 * @param ObjectData The data saved for the subobject. It's the data associated with the index returned by AddSubobjectSnapshot.
		 * @param DataStorage The data saved for the outer
		 * 
	     * @result The found or recreated subobject. If null, the subobject is skipped.
		 */
		virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage)
		{ checkf(false, TEXT("If you register subobjects in OnTakeSnapshot, you must implement this function.")); return nullptr; }

		/**
		 * Similar to FindOrRecreateSubobjectInEditorWorld, only that the subobject is not recreated if not present. Called when diffing against the world.
		 *
		 * @param SnapshotObject The outer of the subobject
		 * @param ObjectData The data saved for the subobject. It's the data associated with the index returned by AddSubobjectSnapshot.
		 * @param DataStorage The data saved for the outer
		 * 
		 * @result The subobject in the editor world. If you return null, the subobject will be recreated.
		 */
		virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) 
		{ checkf(false, TEXT("If you register subobjects in OnTakeSnapshot, you must implement this function.")); return nullptr; }


		
		/** Optional. Called after GetOrRecreateSubobjectInSnapshotWorld when all properties have been serialized into the subobject. You can do any post processing here. */
		virtual void OnPostSerializeSnapshotSubobject(UObject* Subobject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) {}

		/** Optional. Called after GetOrRecreateSubobjectInEditorWorld when all properties have been serialized into the subobject. You can do any post processing here. */
		virtual void OnPostSerializeEditorSubobject(UObject* Subobject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) {}


		
		/** Optional. Called before properties are applied to anobject for which this interface was registered. Called on the object in the transient snapshot world. */
		virtual void PreApplyToSnapshotObject(UObject* Object, const ICustomSnapshotSerializationData& DataStorage) {}
		
		/** Optional. Called after properties are applied to an object for which this interface was registered. Called on the object in the transient snapshot world. */
		virtual void PostApplyToSnapshotObject(UObject* Object, const ICustomSnapshotSerializationData& DataStorage) {}


		
		/** Optional. Called before properties are applied to anobject for which this interface was registered. Called on the object in the editor world. */
		virtual void PreApplyToEditorObject(UObject* Object, const ICustomSnapshotSerializationData& DataStorage, const FPropertySelectionMap& SelectionMap)
		{
			// Calling it for convenience... usually the implementation is equal
			PreApplyToSnapshotObject(Object, DataStorage);
		}
		
		/** Optional. Called after properties are applied to an object for which this interface was registered. Called on the object in the editor world. */
		virtual void PostApplyToEditorObject(UObject* Object, const ICustomSnapshotSerializationData& DataStorage, const FPropertySelectionMap& SelectionMap)
		{
			// Calling it for convenience... usually the implementation is equal
			PostApplyToSnapshotObject(Object, DataStorage);
		}

		
		virtual ~ICustomObjectSnapshotSerializer() = default;
	};
}

