// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class AActor;
class FProperty;
class UObject;

namespace UE::LevelSnapshots
{
	struct FPropertyComparisonParams;
	
	/* Defines a callback for deciding whether a property has changed.
	* 
	* This is useful for cases where certain properties are disabled by others.
	* For an example see FStaticMeshCollisionPropertyComparer.
	*/
	class LEVELSNAPSHOTS_API IPropertyComparer
	{
		public:

		enum class EPropertyComparison
		{
			/* The properties are equal. */
			TreatEqual,
			/* The properties are not equal. */
			TreatUnequal,
			/* Check the property normally. */
			CheckNormally
		};
		
		virtual ~IPropertyComparer() = default;
		
		/**
		* Decides whether the property should be considered equal.
		* Note that this is only called on root properties. For performance reasons, this is not called on nested struct properties.
		* This is not a big restriction because you can just check the structs content yourself.
		*/
		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const { return EPropertyComparison::CheckNormally; }
	};
}

