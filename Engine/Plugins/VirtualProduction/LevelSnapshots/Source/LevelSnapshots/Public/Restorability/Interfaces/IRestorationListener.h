// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComponentInstanceDataCache.h"
#include "Engine/World.h"
#include "Templates/SubclassOf.h"

class ULevelSnapshot;
class AActor;
class UActorComponent;

struct FPropertySelection;
struct FPropertySelectionMap;

namespace UE::LevelSnapshots
{
	struct FRestorationListenerBaseParams
	{
		/** All of the user's selected properties */
		const FPropertySelectionMap& SelectedProperties;
		
		FRestorationListenerBaseParams(const FPropertySelectionMap& SelectedProperties)
			: SelectedProperties(SelectedProperties)
		{}
	};

	struct FApplySnapshotSharedParams : FRestorationListenerBaseParams
	{
		ULevelSnapshot& LevelSnapshot;
		FApplySnapshotSharedParams(ULevelSnapshot& LevelSnapshot, const FPropertySelectionMap& SelectedProperties)
			: FRestorationListenerBaseParams(SelectedProperties)
			, LevelSnapshot(LevelSnapshot)
		{}
	};
	struct FPreApplySnapshotParams : FApplySnapshotSharedParams {};
	struct FPostApplySnapshotParams : FApplySnapshotSharedParams {};

	struct FPreRemoveActorParams : FRestorationListenerBaseParams
	{
		AActor* ActorToRemove;

		FPreRemoveActorParams(const FPropertySelectionMap& SelectedProperties, AActor* ActorToRemove)
			: FRestorationListenerBaseParams(SelectedProperties),
			  ActorToRemove(ActorToRemove)
		{}
	};
	struct FPostRemoveActorsParams : FRestorationListenerBaseParams {};
	
	struct FApplySnapshotPropertiesParams
	{
		/** The object that receives serialized data. */
		UObject* Object;
		
		/** All of the user's selected properties */
		const FPropertySelectionMap& SelectedProperties;
		
		/** Only valid when bWasRecreated = false */
		TOptional<const FPropertySelection*> PropertySelection;
		
		/** Whether Object did not yet exist in the world and was recreated */
		bool bWasRecreated;

		FApplySnapshotPropertiesParams(UObject* Object, const FPropertySelectionMap& SelectedProperties, TOptional<const FPropertySelection*> PropertySelection, bool bWasRecreated)
			: Object(Object),
			  SelectedProperties(SelectedProperties),
			  PropertySelection(PropertySelection),
			  bWasRecreated(bWasRecreated)
		{}
	};

	struct FApplySnapshotToActorParams
	{
		/** The actor that is modified */
		AActor* Actor;
		
		/** All of the user's selected properties */
		const FPropertySelectionMap& SelectedProperties;
		
		/** Whether Object did not yet exist in the world and was recreated */
		bool bWasRecreated;

		FApplySnapshotToActorParams(AActor* Actor, const FPropertySelectionMap& SelectedProperties, bool bWasRecreated)
			: Actor(Actor),
			SelectedProperties(SelectedProperties),
			bWasRecreated(bWasRecreated)
		{}
	};

	struct FPreRecreateComponentParams
	{
		/** The actor that will own the component */
		AActor* Owner;
		/** The name the component will have. */
		FName ComponentName;
		/** The class the component will have. */
		UClass* ComponentClass;
		/** The creation method that will be set for the component. */
		EComponentCreationMethod CreationMethod;

		FPreRecreateComponentParams(AActor* Owner, FName ComponentName, UClass* ComponentClass, EComponentCreationMethod CreationMethod)
			: Owner(Owner),
			  ComponentName(ComponentName),
			  ComponentClass(ComponentClass),
			  CreationMethod(CreationMethod)
		{}
	};

	struct FPostRemoveComponentParams
	{
		/** The actor the component was removed from */
		AActor* Owner;
		/** The old component's name */
		FName ComponentName;

		/**
		 * Pointer to the still allocated component.
		 * Only use in the callback function. Do not store the object: it is pending kill and will be garbage collected.
		 */
		TWeakObjectPtr<UActorComponent> DestroyedComponent;

		FPostRemoveComponentParams(AActor* Owner, FName ComponentName, TWeakObjectPtr<UActorComponent> DestroyedComponent)
			:
			Owner(Owner),
			ComponentName(ComponentName),
			DestroyedComponent(DestroyedComponent)
		{}
	};

	/**
	 * Exposes callbacks for when:
	 *	- An object is serialized (PreApplySnapshot and PostApplySnapshot)
	 *	- A component is added back to an actor (PreRecreateComponent and PostRecreateComponent)
	 *	- A component is removed from an actor (PreRemoveComponent and PostRemoveComponent)
	 *
	 * In all callbacks, you will receive a non-const UObject. You are free to modify them. Keep the user's expectations in mind:
	 * generally only properties that were selected should change. You should generally ensure that relevant systems are updated.
	 */
	class LEVELSNAPSHOTS_API IRestorationListener
	{
	public:

		/** Called before the snapshot is applied. Except for the changes made by this event, the world has not been changed, yet. */
		virtual void PreApplySnapshot(const FPreApplySnapshotParams& Params) {}
		
		/** Called after all snapshot data was applied. The world is nearly completely changed (the only thing remaining is executing this event). */
		virtual void PostApplySnapshot(const FPostApplySnapshotParams& Params) {}


		
		/**
		 * Called before applying snapshot data to an object.
		 *
		 * For actors, this function is always called.
		 * For subobjects, such as components, this is only called when there are changed properties.
		 *
		 * If this is called on a recreated component, then Params.bWasRecreated will be true; this function will be called
		 * after both PreRecreateComponent and PostRecreateComponent have been called.
		 */
		virtual void PreApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) {}

		/**
		 * Called after applying snapshot data to an object.
		 *
		 * For actors, this function is always called.
		 * For subobjects, such as components, this is only called when there were changed properties.
		 */
		virtual void PostApplySnapshotProperties(const FApplySnapshotPropertiesParams& Params) {} 

		

		/**
		 * Called before an actor receives any property or component change.
		 * Called before PreApplySnapshotProperties, PreRecreateComponent, and PreRemoveComponent.
		 */
		virtual void PreApplySnapshotToActor(const FApplySnapshotToActorParams& Params) {}

		/**
		 * Called after an actor has received all property and component changes.
		 * Called after PostApplySnapshotProperties, PostRecreateComponent, and PostRemoveComponent.
		 */
		virtual void PostApplySnapshotToActor(const FApplySnapshotToActorParams& Params) {}

		

		/**
		 * Called before an actor is recreated to the world and gives the opportunity to override settings.
		 * You cannot override the Name.
		 */
		virtual void PreRecreateActor(UWorld* World, TSubclassOf<AActor> ActorClass, FActorSpawnParameters& InOutSpawnParameters) {}
		
		/** Called after an actor is recreated to the world (but before any data is applied to it) */
		virtual void PostRecreateActor(AActor* RecreatedActor) {}
		

		UE_DEPRECATED(5.1, "Use PreRemoveActor(const FPreRemoveActorParams& Params) instead")
		virtual void PreRemoveActor(AActor* ActorToRemove) {}
		
		/** Called before an actor is removed from the world. */
		virtual void PreRemoveActor(const FPreRemoveActorParams& Params) {}
		
		/** Called after all actors have been removed from the world */
		virtual void PostRemoveActors(const FPostRemoveActorsParams& Params) {}
		

		
		/**
		 * Called before a component is recreated on an actor.
		 * 
		 * PreApplySnapshot and PostApplySnapshot will be called after PostRecreateComponent has executed.
		 */
		virtual void PreRecreateComponent(const FPreRecreateComponentParams& Params) {}

		/**
		 * Called after a component has been recreated on an actor.
		 * 
		 * PreApplySnapshot and PostApplySnapshot will be called after PostRecreateComponent has executed.
		 */
		virtual void PostRecreateComponent(UActorComponent* RecreatedComponent) {}


		
		
		/**
		 * Called before a component is removed from an actor.
		 */
		virtual void PreRemoveComponent(UActorComponent* ComponentToRemove) {}

		/**
		 * Called after a component is removed from an actor.
		 */
		virtual void PostRemoveComponent(const FPostRemoveComponentParams& Params) {}


		
		virtual ~IRestorationListener() = default;
	};
}