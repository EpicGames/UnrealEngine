// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filtering/PropertySelection.h"

class AActor;
class ULevelSnapshot;
class UObject;
struct FPropertySelectionMap;

namespace UE::LevelSnapshots
{
	struct FPreApplyFiltersParams
	{
		/** The editor object being filtered */
		UObject* SnapshotObject;
		/** The snapshot world object being filtered */
		UObject* EditorObject;
		/** The snapshot on which the filtering occurs */
		ULevelSnapshot* Snapshot;
	};
	
	struct FPostApplyFiltersParams
	{
		const FPropertySelection& SelectedProperties;
		/** The editor object being filtered */
		UObject* SnapshotObject;
		/** The snapshot world object being filtered */
		UObject* EditorObject;

		FPostApplyFiltersParams(const FPropertySelection& SelectedProperties, UObject* SnapshotObject, UObject* EditorObject)
			: SelectedProperties(SelectedProperties)
			, SnapshotObject(SnapshotObject)
			, EditorObject(EditorObject)
		{}
	};
	
	struct FPostApplyFiltersResult
	{
		/** Additional properties that should be displayed to the user. They will only be shown if they pass the user's filters. */
		TArray<FLevelSnapshotPropertyChain> AdditionalPropertiesToFilter;
		/** Additional properties that should always be displayed to the user regardless of whether they pass the property filter or not. */
		TArray<FLevelSnapshotPropertyChain> AdditionalPropertiesToForcefullyShow;

		void Combine(FPostApplyFiltersResult Result)
		{
			for (FLevelSnapshotPropertyChain& ToFilter : Result.AdditionalPropertiesToFilter)
			{
				AdditionalPropertiesToFilter.Emplace(MoveTemp(ToFilter));
			}
			for (FLevelSnapshotPropertyChain& ForceShow : Result.AdditionalPropertiesToForcefullyShow)
			{
				AdditionalPropertiesToForcefullyShow.Emplace(MoveTemp(ForceShow));
			}
		}
	};
	
	/**
	 * Allows you to receive callbacks when an object is filtered and to display additional properties in the results view.
	 * This is used whenever a selection set is built.
	 *
	 * By default only modified properties are shown. However, sometimes you may want to include unmodified properties, too.
	 * 
	 * Example 1: Transforms and parenting
	 *	1. Create cube
	 *	2. Rotate it to 40 degrees
	 *  3. Take a snapshot
	 *  4. Create new empty actor
	 *	5. Attach cube to empty actor
	 *	6. Rotate empty actor by 10 degrees
	 *	7. Apply the snapshot
	 *	Result: After applying, the cube has a diff because its rotation is 50 degrees. The reason is because before applying
	 *	the only thing that was different was the cube's attach parent. When the parent actor is removed, its world-space transform is retained;
	 *	actors are removed first and then the properties of modified actors are saved.
	 *	The solution is, using this interface, to add the cube's transform properties even though they're equal to the snapshot version.
	 *	Users will see the property in the UI and can opt to not restore it.
	 *
	 * Example 2: AMediaPlate
	 * Media plates add a transient UAssetUserData object to a component.
	 * In OnPreSaveWorld and OnPostSaveWorld the UAssetUserData would be removed and added back. Similar behaviour should
	 * occur for Level Snapshots when filtering and applying (events about applying are exposed by IRestorationListener).
	 * 
	 * Note that Level Snapshots restores TArrays either fully or not at all.
	 * For this reason, IPropertyComparer may not always be an option for you; this interface gives you more options.
	 * If you have changes that should be ignored by Level Snapshots, this interface allows you to modify the object in PreFilterObject
	 * and undo the changes in PostFilterObject.
	 */
	class LEVELSNAPSHOTS_API ISnapshotFilterExtender
	{
	public:

		/**
		 * Called before the object is filtered.
		 * PreFilterObject is be executed recursively on subobjects before the corresponding call to PostFilterObject is made.
		 */
		virtual void PreApplyFilters(const FPreApplyFiltersParams& Params) {}

		/** Called after all properties have been filtered on the given object pair. */
		virtual FPostApplyFiltersResult PostApplyFilters(const FPostApplyFiltersParams& Params) { return {}; }
		
		virtual ~ISnapshotFilterExtender() = default;
	};
}