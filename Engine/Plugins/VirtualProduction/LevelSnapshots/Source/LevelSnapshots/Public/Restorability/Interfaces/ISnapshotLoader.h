// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UObject;
struct FWorldSnapshotData;

namespace UE::LevelSnapshots
{
	struct LEVELSNAPSHOTS_API FPostLoadSnapshotObjectParams
	{
		/** The object that received its properties */
		UObject* SnapshotObject;
		/** All saved snapshot data. Typical use case is getting the saved version info */
		const FWorldSnapshotData& SnapshotData;

		FPostLoadSnapshotObjectParams(UObject* SnapshotObject, const FWorldSnapshotData& SnapshotData)
			: SnapshotObject(SnapshotObject),
			SnapshotData(SnapshotData)
		{}
	};

	/**
	 * Receives callbacks when an object is loaded into the snapshot world. Gives the opportunity to call custom fix up functions.
	 */
	class LEVELSNAPSHOTS_API ISnapshotLoader
	{
		public:

		/** Called after an object has received all its properties. */
		virtual void PostLoadSnapshotObject(const FPostLoadSnapshotObjectParams& Params) {}
		
		virtual ~ISnapshotLoader() = default;
	};
}