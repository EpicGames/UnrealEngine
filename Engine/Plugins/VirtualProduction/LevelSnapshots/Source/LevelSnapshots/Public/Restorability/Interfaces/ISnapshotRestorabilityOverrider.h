// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class UActorComponent;

namespace UE::LevelSnapshots
{
	/**
	* Exposes callbacks for deciding whether an actor or component is exposed to the snapshot system.
	* 
	* Supported actors, components, or properties are:
	*	- Captured and saved when a snapshot is taken
	*	- Passed to filters
	*	- Show up in the results view
	*	- Restored when a snapshot is applied
	*
	* To see which actors, components, and subobjects are supported by default, see FSnapshotRestorability.
	*/
	class LEVELSNAPSHOTS_API ISnapshotRestorabilityOverrider
	{
		public:
		
		enum class ERestorabilityOverride
		{
			/* The object in question is included but only if nobody else returned Disallow.  */
			Allow,
			/* Let other overrides decide. If every override return DoNotCare, default snapshot behaviour is applied. */
			DoNotCare,
			/* The object in question is never suitable and is not included. Other overriders cannot override this. */
			Disallow
		};

		virtual ~ISnapshotRestorabilityOverrider() = default;
		
		/* Should this actor be visible to the snapshot system? */
		virtual ERestorabilityOverride IsActorDesirableForCapture(const AActor* Actor) { return ERestorabilityOverride::DoNotCare; }
		
		/* Should this component be visible to the snapshot system? */
		virtual ERestorabilityOverride IsComponentDesirableForCapture(const UActorComponent* Component) { return ERestorabilityOverride::DoNotCare; }
	};
}