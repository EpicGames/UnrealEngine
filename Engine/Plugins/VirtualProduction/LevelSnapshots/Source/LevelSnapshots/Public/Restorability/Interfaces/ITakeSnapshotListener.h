// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class ULevelSnapshot;
class UObject;

namespace UE::LevelSnapshots
{
	struct FPostTakeSnapshotEventData;
	struct FPreTakeSnapshotEventData;

	struct FTakeObjectSnapshotEvent
	{
		UObject* Object;
	};

	struct FPreTakeObjectSnapshotEvent : FTakeObjectSnapshotEvent
	{};
	struct FPostTakeObjectSnapshotEvent : FTakeObjectSnapshotEvent
	{};

	/** Exposes callbacks for when a snapshot is taken */
	class LEVELSNAPSHOTS_API ITakeSnapshotListener
	{
	public:

		/** Called before an object's data is recorded by a snapshot. */
		virtual void PreTakeObjectSnapshot(const FPreTakeObjectSnapshotEvent& Params) {}
		/** Called after an object's data is recorded by a snapshot. */
		virtual void PostTakeObjectSnapshot(const FPostTakeObjectSnapshotEvent& Params) {}

		/** Called before a snapshot is taken for a world. */
		virtual void PreTakeSnapshot(const FPreTakeSnapshotEventData& Params) {}
		/** Called after a snapshot is taken for a world. */
		virtual void PostTakeSnapshot(const FPostTakeSnapshotEventData& Params) {}
		
		virtual ~ITakeSnapshotListener() = default;
	};
}
