// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"

class UObject;

namespace UE::LevelSnapshots
{
	DECLARE_DELEGATE_OneParam(FObjectAnnotator, FArchive&);

	/* Manages meta data needed by external modules to restore objects */
	class LEVELSNAPSHOTS_API ISnapshotSubobjectMetaData
	{
	public:
		
		/* Gets the soft path for the object when it was snapshot */
		virtual FSoftObjectPath GetOriginalPath() const = 0;

		/* Add data needed for restoring the object */
		virtual void WriteObjectAnnotation(const FObjectAnnotator& Writer) = 0;
		/* Read data needed for restoring the object */
		virtual void ReadObjectAnnotation(const FObjectAnnotator& Reader) const = 0;

		virtual ~ISnapshotSubobjectMetaData() = default;
	};

	/**
	 * Utility for saving / restoring properties of an object.
	 */
	class LEVELSNAPSHOTS_API ICustomSnapshotSerializationData
	{
	public:

		/** Add data needed for restoring the object. */
		virtual void WriteObjectAnnotation(const FObjectAnnotator& Writer) = 0;
		
		/** Read data needed for restoring the object */
		virtual void ReadObjectAnnotation(const FObjectAnnotator& Reader) const = 0;
		
		/**
		 * Causes the properties of the subobject to be saved.
		 * 
		 * You can add additional meta data needed for restoring by calling GetSubobjectMetaData with this function's result
		 * and using ISnapshotSubobjectMetaData::ISnapshotSubobjectMetaData.
		 * 
		 * You need to make sure of the following, or this function will fail:
		 *	1. 'Subobject' must be of a subobject of GetSerializedObject()
		 *	2. 'Subobject' cannot be visible to standard Level Snapshots serialisation. A subobject is visible if it contained
		 *	  within a uproperty (object reference or collection of references) with the CPF_Edit flag. You can explicitly disallow uproperties
		 *	  using the Level Snapshots module.
		 */
		virtual int32 AddSubobjectSnapshot(UObject* Subobject) = 0;

		/* Gets the meta data stored for a subobject previously added using AddSubobjectDependency. */
		virtual TSharedPtr<ISnapshotSubobjectMetaData> GetSubobjectMetaData(int32 Index) = 0;
		
		virtual const TSharedPtr<ISnapshotSubobjectMetaData> GetSubobjectMetaData(int32 Index) const = 0;

		/** Gets all saved subobject handles. */
		virtual int32 GetNumSubobjects() const = 0;
		
		virtual ~ICustomSnapshotSerializationData() = default;
	};
}