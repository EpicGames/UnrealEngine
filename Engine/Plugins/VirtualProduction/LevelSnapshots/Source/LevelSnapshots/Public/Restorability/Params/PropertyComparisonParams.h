// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class FProperty;
class ULevelSnapshot;
class UObject;
class UStruct;

namespace UE::LevelSnapshots
{
	struct FPropertyComparisonParams
	{
		ULevelSnapshot* Snapshot;
	
		/* The class we're looking at. This is not necessarily the class LeafProperty resides in. */
		UClass* InspectedClass;
		/* The property being checked */
		const FProperty* LeafProperty;

		/* Parameter for FProperty::ContainerPtrToValuePtr. */
		void* SnapshotContainer;
		/* Parameter for FProperty::ContainerPtrToValuePtr. */
		void* WorldContainer;

		/* Either an AActor, UActorComponent, or a subobject of the two. */
		UObject* SnapshotObject;
		/* Either an AActor, UActorComponent, or a subobject of the two. */
		UObject* WorldObject;

		/* Snapshot version of the actor */
		AActor* SnapshotActor;
		/* Actor currently in the world */
		AActor* WorldActor;

		FPropertyComparisonParams(ULevelSnapshot* Snapshot, UClass* InspectedClass, const FProperty* LeafProperty, void* SnapshotContainer, void* WorldContainer, UObject* SnapshotObject, UObject* WorldObject, AActor* SnapshotActor, AActor* WorldActor)
			: Snapshot(Snapshot)
			, InspectedClass(InspectedClass)
			, LeafProperty(LeafProperty)
			, SnapshotContainer(SnapshotContainer)
			, WorldContainer(WorldContainer)
			, SnapshotObject(SnapshotObject)
			, WorldObject(WorldObject)
			, SnapshotActor(SnapshotActor)
			, WorldActor(WorldActor)
		{}
	};
}