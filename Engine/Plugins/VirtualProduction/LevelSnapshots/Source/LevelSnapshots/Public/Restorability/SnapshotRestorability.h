// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ULevelSnapshot;
class AActor;
class FProperty;
class UActorComponent;

namespace UE::LevelSnapshots
{
	struct FCanModifyMatchedActorParams;
	struct FCanRecreateActorParams;
}

namespace UE::LevelSnapshots::Restorability
{
	/* Is this actor captured by the snapshot system? */
	LEVELSNAPSHOTS_API bool IsActorDesirableForCapture(const AActor* Actor);
	/** Can this actor be restored? Stronger requirement than IsActorDesirableForCapture: we may capture the data but not support restoring it at the moment. */
	LEVELSNAPSHOTS_API bool IsActorRestorable(const AActor* Actor);
	
	/* Is this component captured by the snapshot system? */
	LEVELSNAPSHOTS_API bool IsComponentDesirableForCapture(const UActorComponent* Component);

	/* Is this subobject class captured by the snapshot system?*/
	LEVELSNAPSHOTS_API bool IsSubobjectClassDesirableForCapture(const UClass* SubobjectClass);
	/** Is this subobject captured by the snapshot system? */
	LEVELSNAPSHOTS_API bool IsSubobjectDesirableForCapture(const UObject* Subobject);

	/** Can the property be captured? */
	LEVELSNAPSHOTS_API bool IsPropertyDesirableForCapture(const FProperty* Property);
	/* Is this property never captured by the snapshot system? */
	LEVELSNAPSHOTS_API bool IsPropertyExplicitlyUnsupportedForCapture(const FProperty* Property);
	/* Is this property always captured by the snapshot system? */
	LEVELSNAPSHOTS_API bool IsPropertyExplicitlySupportedForCapture(const FProperty* Property);
	
	/** The actor exists in the world and the snapshot. Should we show it in the list of modified actors? */
	LEVELSNAPSHOTS_API bool ShouldConsiderMatchedActorForRestoration(const FCanModifyMatchedActorParams& Params, FText* ExclusionReason = nullptr);
	/** The actor was removed from the snapshot. Should we show it in the list of removed actors? */
	LEVELSNAPSHOTS_API bool ShouldConsiderRemovedActorForRecreation(const FCanRecreateActorParams& Params, FText* ExclusionReason = nullptr);
	/* The actor did not exist in the snapshot. Should we show it in the list of added actors? */
	LEVELSNAPSHOTS_API bool ShouldConsiderNewActorForRemoval(const AActor* Actor, FText* ExclusionReason = nullptr);
	
	/* Is this property captured by the snapshot system? */
	LEVELSNAPSHOTS_API bool IsRestorableProperty(const FProperty* LeafProperty);
}
