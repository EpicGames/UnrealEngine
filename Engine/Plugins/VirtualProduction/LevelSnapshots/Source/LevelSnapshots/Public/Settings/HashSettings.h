// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HashSettings.generated.h"

UENUM()
namespace EHashAlgorithmChooseBehavior
{
	enum Type : int
	{
		/**
		 * Use whichever algorithm is faster
		 */
		UseFastest,
		/**
		 * Always use CRC32 (only works if CRC32 is enabled)
		 */
		UseCrc32,
		/**
		 * Always use MD5 (only works if MD5 is enabled)
		 */
		UseMD5,
	};
}


USTRUCT()
struct LEVELSNAPSHOTS_API FHashSettings
{
	GENERATED_BODY()

	/**
	 * Performance trade-off. Used when filtering a snapshot.
	 * 
	 * For filtering, we need to load every actor into memory. Loading actors takes a long time.
	 * Instead when a snapshot is taken, we compute its hash. When filtering, we can recompute the hash using the actor
	 * in the editor world. If they match, we can skip loading the saved actor data.
	 *
	 * For most actors, it takes about 600 micro seconds to compute a hash. However, there are outliers which can take
	 * more. For such actors, it can be faster to just load the saved actor data into memory.
	 *
	 * Actors for which hashing took more than this configured variable, we skip hashing altogether and immediately load
	 * the actor data. 
	 */
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	double HashCutoffSeconds = 0.3;

	/** Whether to compare world actor's to its saved hash when loading a snapshot. Boosts performance. */
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	bool bUseHashForLoading = true;
	
	/**
	 * Whether Level Snapshots will compute CRC32 hashes (when taking and analysing snapshots)
	 * Speed: CRC32 < MD5 < SHA1.
	 *
	 * Disabling this option results in:
	 *  1. Taking a snapshot will be faster (CRC32 will no longer be computed when snapshot is taken)
	 *  2. When loading a snapshot, CRC32 is no longer available to quickly check whether an actor has changed.
	 */
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	bool bCanUseCRC = true;

	/**
	 * Whether Level Snapshots will compute MD5 hashes (when taking and analysing snapshots).
	 * Speed: CRC32 < MD5 < SHA1.
	 *
	 * Disabling this option results in:
	 *  1. Taking a snapshot will be faster (MD5 will no longer be computed when snapshot is taken)
	 *  2. When loading a snapshot, MD5 is no longer available to quickly check whether an actor has changed.
	 */
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	bool bCanUseMD5 = true;

	/** Which hash algorithm to use when comparing a snapshot to the world. */
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TEnumAsByte<EHashAlgorithmChooseBehavior::Type> SnapshotDiffAlgorithm = EHashAlgorithmChooseBehavior::UseFastest;
};