// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SnapshotCompressionSettings.h"
#include "HashSettings.h"
#include "Settings/SkippedClassList.h"
#include "LevelSnapshotsSettings.generated.h"

UCLASS(config = Engine, defaultconfig)
class LEVELSNAPSHOTS_API ULevelSnapshotsSettings : public UObject
{
	GENERATED_BODY()
public:

	static ULevelSnapshotsSettings* Get(); 

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface
	

	/* Specifies classes and properties that should never be captured nor restored. */
	UPROPERTY(config, EditAnywhere, Category = "Level Snapshots|Behavior")
	FSkippedClassList SkippedClasses;

	

	/** Used when comparing float properties. Floats that have changes beyond this point do not show up as changed. */
	UPROPERTY(Config, EditAnywhere, Category = "Level Snapshots|Behavior", meta = (ClampMin = "0.00000001", ClampMax = "0.1")) // Max value is SMALL_NUMBER = 1e-8
	float FloatComparisonPrecision = 1e-03f;

	/** Used when comparing double properties. Doubles that have changes beyond this point do not show up as changed. */
	UPROPERTY(Config, EditAnywhere, Category = "Level Snapshots|Behavior", meta = (ClampMin = "0.00000001", ClampMax = "0.1")) // Max value is SMALL_NUMBER = 1e-8
	double DoubleComparisonPrecision = 1e-03;

	
	
	/**
	 * Performance trade-off. Used when filtering a snapshot.
	 * 
	 * For filtering, we need to load every actor into memory. Loading actors takes a long time.
	 * Instead when a snapshot is taken, we compute its hash. When filtering, we can recompute the hash using the actor
	 * in the editor world. If they match, we can skip loading the saved actor data.
	 *
	 * For most actors, it takes about 600 micro seconds to compute a hash. However, there are outliers which can take
	 * more. For such actors, it can be faster to just load the saved actor data into memory.
	 *
	 * Actors for which hashing took more than this configured variable, we skip hashing altogether and immediately load
	 * the actor data. 
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Level Snapshots|Performance", meta = (ClampMin = "1"))
	FHashSettings HashSettings;

	/**
	 * Specifies the compression settings to use when a snapshot is saved.
	 *
	 * These settings affect the time:
	 * - taken to save a snapshot
	 * - taken to load a snapshot
	 *
	 * You can safely change these settings even if you're already saved snapshots with different settings.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Level Snapshots|Compression", meta = (ClampMin = "1"))
	FSnapshotCompressionSettings CompressionSettings;
};
