// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"
#include "SkippedClassList.generated.h"

USTRUCT()
struct LEVELSNAPSHOTS_API FSkippedClassList
{
	GENERATED_BODY()

	/* These actor and component classes are skipped. */
	UPROPERTY(EditAnywhere, Category = "Level Snapshots")
	TSet<FSoftClassPath> SkippedClasses;
};