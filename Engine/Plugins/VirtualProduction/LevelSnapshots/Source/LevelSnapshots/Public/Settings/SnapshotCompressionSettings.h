// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Compression/OodleDataCompression.h"
#include "SnapshotCompressionSettings.generated.h"

// ESnapshotCompressionLevel and ESnapshotCompressor copied from OodleDataCompression.h - there is no UENUM version at time of duplication
UENUM()
enum class ESnapshotCompressionLevel : int8
{
	HyperFast4 = -4,
	HyperFast3 = -3,
	HyperFast2 = -2,
	HyperFast1 = -1,
	None = 0,
	SuperFast = 1,
	VeryFast = 2,
	Fast = 3,
	Normal = 4,
	Optimal1 = 5,
	Optimal2 = 6,
	Optimal3 = 7,
	Optimal4 = 8,
};

UENUM()
enum class ESnapshotCompressor : uint8
{
	// Omitted ESnapshotCompressor::NotSet because it corrupts the packages for some reason
	
	/** Lower compression but fastest decodes */
	Selkie = 1,
	/** Mid-level compression with fast decoding */
	Mermaid = 2,
	/** High compression with fast decoding */
	Kraken  = 3,
	/** Highest compression ration */
	Leviathan = 4
};

namespace UE::LevelSnapshots::Compression
{
	FORCEINLINE FOodleDataCompression::ECompressionLevel CastCompressionLevel(ESnapshotCompressionLevel Value) { return static_cast<FOodleDataCompression::ECompressionLevel>(Value); }
	FORCEINLINE FOodleDataCompression::ECompressor CastCompressor(ESnapshotCompressor Value) { return static_cast<FOodleDataCompression::ECompressor>(Value); }
}

USTRUCT()
struct LEVELSNAPSHOTS_API FSnapshotCompressionSettings
{
	GENERATED_BODY()

	/**
	 * Trade-off between compression ratio and time.
	 *
	 * You usually do not want anything higher than Normal.
	 * Normal > Optimal approx. reduces compressed size about 1-2% but increases computation time by about 10x. Each subsequent step has diminishing returns.
	 */
	UPROPERTY(EditAnywhere, Category = "Compression")
	ESnapshotCompressionLevel CompressionLevel = ESnapshotCompressionLevel::Normal;

	/**
	 * Specifies the algorithm to use for compressing & decompressing snapshot data.
	 *
	 * Test in July, 2022 for 5.1:
	 * A 112 mb size snapshot was compressed with Optimal4 setting:
	 *	- Selkie		15.8s	14.3mb
	 *	- Mermaid		23.6s	10.9mb
	 *	- Kraken 		1m19s	9.3mb
	 *	- Leviathan 	1m51s	9.0mb
	 * Decompression took about 2.7s for each algorithm.
	 * 
	 * @see http://www.radgametools.com/oodlecompressors.htm
	 */
	UPROPERTY(EditAnywhere, Category = "Compression")
	ESnapshotCompressor CompressorAlgorithm = ESnapshotCompressor::Kraken;
};

// If this hits, please update the constants to mirror the ones from Core
static_assert(static_cast<int32>(ESnapshotCompressionLevel::HyperFast4) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::HyperFast4));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::HyperFast3) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::HyperFast3));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::HyperFast2) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::HyperFast2));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::HyperFast1) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::HyperFast1));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::None) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::None));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::SuperFast) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::SuperFast));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::VeryFast) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::VeryFast));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::Fast) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::Fast));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::Normal) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::Normal));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::Optimal1) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::Optimal1));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::Optimal2) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::Optimal2));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::Optimal3) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::Optimal3));
static_assert(static_cast<int32>(ESnapshotCompressionLevel::Optimal4) == static_cast<int32>(FOodleDataCompression::ECompressionLevel::Optimal4));

static_assert(static_cast<int32>(ESnapshotCompressor::Selkie) == static_cast<int32>(FOodleDataCompression::ECompressor::Selkie));
static_assert(static_cast<int32>(ESnapshotCompressor::Mermaid) == static_cast<int32>(FOodleDataCompression::ECompressor::Mermaid));
static_assert(static_cast<int32>(ESnapshotCompressor::Kraken) == static_cast<int32>(FOodleDataCompression::ECompressor::Kraken));
static_assert(static_cast<int32>(ESnapshotCompressor::Leviathan) == static_cast<int32>(FOodleDataCompression::ECompressor::Leviathan));