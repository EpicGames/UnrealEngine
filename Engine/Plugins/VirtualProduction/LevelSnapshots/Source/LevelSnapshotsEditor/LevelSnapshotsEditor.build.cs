// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class LevelSnapshotsEditor : ModuleRules
{
	public LevelSnapshotsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"BlueprintGraph",
				"CoreUObject",
				"ContentBrowser",
				"Engine",
				"EngineAssetDefinitions",
				"EditorFramework",
				"EditorStyle",
				"EditorWidgets",
				"GameProjectGeneration",
				"InputCore",
				"Kismet",
				"LevelSnapshots",
				"LevelSnapshotFilters",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
			);
	}
}
