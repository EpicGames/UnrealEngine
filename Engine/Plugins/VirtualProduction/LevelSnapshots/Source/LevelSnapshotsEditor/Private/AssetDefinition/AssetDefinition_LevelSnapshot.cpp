// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_LevelSnapshot.h"

#include "ContentBrowserMenuContexts.h"
#include "LevelSnapshot.h"
#include "LevelSnapshotsEditorFunctionLibrary.h"
#include "LevelSnapshotsEditorModule.h"
#include "LevelSnapshotsEditorStyle.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "Data/LevelSnapshotsEditorData.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_LevelSnapshot"

FText UAssetDefinition_LevelSnapshot::GetAssetDisplayName() const
{
	return LOCTEXT("LevelSnapshot_AssetName", "Level Snapshot");
}

TSoftClassPtr<UObject> UAssetDefinition_LevelSnapshot::GetAssetClass() const
{
	return ULevelSnapshot::StaticClass();
}

FLinearColor UAssetDefinition_LevelSnapshot::GetAssetColor() const
{
	return FColor(238, 181, 235, 255);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_LevelSnapshot::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("LevelSnapshot_AssetCategory", "Virtual Production")), LOCTEXT("LevelSnapshot_CategorySection", "Level Snapshots"), ECategoryMenuType::Section)
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_LevelSnapshot::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	if (Objects.Num() && Objects[0])
	{
		FLevelSnapshotsEditorModule& Module = FLevelSnapshotsEditorModule::Get();		
		Module.OpenLevelSnapshotsDialogWithAssetSelected(FAssetData(Objects[0]));
	}
	
	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_LevelSnapshotAsset
{
	void ExecuteUpdateSnapshotData(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			for (const FAssetData& Asset : Context->SelectedAssets)
			{
				if (Asset.GetClass() == ULevelSnapshot::StaticClass())
				{
					if (ULevelSnapshot* LevelSnapshot = Cast<ULevelSnapshot>(Asset.GetAsset()))
					{
						if (UWorld* World = ULevelSnapshotsEditorData::GetEditorWorld())
						{
							LevelSnapshot->SnapshotWorld(World);
							ULevelSnapshotsEditorFunctionLibrary::GenerateThumbnailForSnapshotAsset(LevelSnapshot);
							LevelSnapshot->MarkPackageDirty();
						}
					}
				}
			}
		}
	}
	
	bool CanExecuteUpdateSnapshotData(const FToolMenuContext& InContext)
	{
		return UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(InContext) == 1;
	}
	
	void ExecuteCaptureThumbnails(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			for (const FAssetData& Asset : Context->SelectedAssets)
			{
				if (Asset.GetClass() == ULevelSnapshot::StaticClass())
				{
					if (ULevelSnapshot* LevelSnapshot = Cast<ULevelSnapshot>(Asset.GetAsset()))
					{
						ULevelSnapshotsEditorFunctionLibrary::GenerateThumbnailForSnapshotAsset(LevelSnapshot);
						LevelSnapshot->MarkPackageDirty();
					}
				}
			}
		}
	}
	
	void ExecuteOpenSnapshot(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (Context && Context->SelectedAssets.Num() == 1)
		{
			const FAssetData& Asset = Context->SelectedAssets[0];
			if (Asset.GetClass() == ULevelSnapshot::StaticClass())
			{
				FLevelSnapshotsEditorModule& Module = FLevelSnapshotsEditorModule::Get();		
				Module.OpenLevelSnapshotsDialogWithAssetSelected(Asset);
			}
		}
	}
	
	bool CanExecuteOpenSnapshot(const FToolMenuContext& InContext)
	{
		return UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(InContext) == 1;
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(ULevelSnapshot::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					InSection.AddMenuEntry("LevelSnapshot_UpdateSnapshotDataAction",
						LOCTEXT("LevelSnapshot_UpdateSnapshotDataLabel", "Update Snapshot Data"),
						LOCTEXT("LevelSnapshot_UpdateSnapshotDataToolTip", "Record a snapshot of the current map to this snapshot asset and update the thumbnail. Equivalent to 'Take Snapshot'. Select only one Level Snapshot asset at a time."),
						FSlateIcon(FLevelSnapshotsEditorStyle::GetStyleSetName(), "LevelSnapshots.ToolbarButton.Small"),
						FToolUIAction(
							FToolMenuExecuteAction::CreateStatic(&ExecuteUpdateSnapshotData),
							FToolMenuCanExecuteAction::CreateStatic(&CanExecuteUpdateSnapshotData),
							FToolMenuGetActionCheckState()
						));
					
					InSection.AddMenuEntry("LevelSnapshot_CaptureThumbnailsAction",
						LOCTEXT("LevelSnapshot_CaptureThumbnailsLabel", "Capture Thumbnails"),
						LOCTEXT("LevelSnapshot_CaptureThumbnailsToolTip", "Capture and update thumbnails only for the selected snapshot assets."),
						FSlateIcon(FLevelSnapshotsEditorStyle::GetStyleSetName(), "LevelSnapshots.ToolbarButton.Small"),
						FToolUIAction(FToolMenuExecuteAction::CreateStatic(&ExecuteCaptureThumbnails)));
					
					InSection.AddMenuEntry("LevelSnapshot_OpenSnapshotAction",
						LOCTEXT("LevelSnapshot_OpenSnapshotLabel", "Open Snapshot in Editor"),
						LOCTEXT("LevelSnapshot_OpenSnapshotToolTip", "Open this snapshot in the Level Snapshots Editor. Select only one Level Snapshot asset at a time."),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.SummonOpenAssetDialog"),
						FToolUIAction(
							FToolMenuExecuteAction::CreateStatic(&ExecuteOpenSnapshot),
							FToolMenuCanExecuteAction::CreateStatic(&CanExecuteOpenSnapshot),
							FToolMenuGetActionCheckState()
						));
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
