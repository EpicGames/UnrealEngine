// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_LevelSnapshotFilter.h"

#include "LevelSnapshotFilters.h"
#include "Factories/LevelSnapshotFilterFactory.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_LevelSnapshotFilter"

FText UAssetDefinition_LevelSnapshotFilter::GetAssetDisplayName() const
{
	return LOCTEXT("LevelSnapshotFilter_AssetName", "Level Snapshot Filter");
}

TSoftClassPtr<UObject> UAssetDefinition_LevelSnapshotFilter::GetAssetClass() const
{
	return ULevelSnapshotBlueprintFilter::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_LevelSnapshotFilter::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("LevelSnapshotFilter_AssetCategory", "Virtual Production")), LOCTEXT("LevelSnapshotFilter_CategorySection", "Level Snapshots"), ECategoryMenuType::Section)
	};

	return Categories;
}

UFactory* UAssetDefinition_LevelSnapshotFilter::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	ULevelSnapshotBlueprintFilterFactory* BlueprintFactory = NewObject<ULevelSnapshotBlueprintFilterFactory>();
	BlueprintFactory->ParentClass = TSubclassOf<ULevelSnapshotBlueprintFilter>(*InBlueprint->GeneratedClass);
	return BlueprintFactory;
}

#undef LOCTEXT_NAMESPACE
