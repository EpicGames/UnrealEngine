// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/NegatableFilterDetailsCustomization.h"

#include "Data/Filters/NegatableFilter.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Algo/AllOf.h"
#include "Algo/Transform.h"

namespace
{
	void ShowNegationBehaviourAndName(IDetailLayoutBuilder& DetailBuilder)
	{
		// Makes the category appear first
		IDetailCategoryBuilder& FilterCategory = DetailBuilder.EditCategory("Filter");
	}

	void ShowChildFilterProperties(IDetailLayoutBuilder& DetailBuilder)
	{
		TArray< TWeakObjectPtr<UObject> > WeakSelectedObjects;
		DetailBuilder.GetObjectsBeingCustomized(WeakSelectedObjects);

		TArray<UObject*> SelectedObjects;
		Algo::Transform(WeakSelectedObjects, SelectedObjects, [&SelectedObjects](TWeakObjectPtr<UObject> Obj) { return ExactCast<UNegatableFilter>(Obj.Get())->GetChildFilter(); });
		
		// Child filter may be null in certain cases, e.g. when user force deleted the filter class
		const bool bNoneAreNull = Algo::AllOf(SelectedObjects, [](UObject* ChildFilter) { return ChildFilter != nullptr; });
		if (!bNoneAreNull || !ensure(SelectedObjects.Num() != 0))
		{
			return;
		}
	
		for (TFieldIterator<FProperty> PropIt(SelectedObjects[0]->GetClass()); PropIt; ++PropIt)
		{
			const FProperty* TestProperty = *PropIt;

			const bool bCanEverBeEdited = TestProperty->HasAnyPropertyFlags(CPF_Edit);
			const bool bIsBlueprintEditable = !TestProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
			if (bCanEverBeEdited && bIsBlueprintEditable)
			{
				const FName CategoryName(*TestProperty->GetMetaData(TEXT("Category"))); 
				IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName);

				if (IDetailPropertyRow* ExternalRow = Category.AddExternalObjectProperty(SelectedObjects, TestProperty->GetFName()))
				{
					ExternalRow->Visibility(EVisibility::Visible);
				}
			}
		}
	}
}

void FNegatableFilterDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{	
	ShowNegationBehaviourAndName(DetailBuilder);
	ShowChildFilterProperties(DetailBuilder);
}

