// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

/* Exposes invertability and name properties on UNegatableFilter. Also shows subobject properties inline. */
class FNegatableFilterDetailsCustomization 
	: public IDetailCustomization
{
public:

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface
	
};
