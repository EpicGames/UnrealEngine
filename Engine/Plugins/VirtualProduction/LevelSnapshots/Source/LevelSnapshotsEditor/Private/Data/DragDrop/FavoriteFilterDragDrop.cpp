// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DragDrop/FavoriteFilterDragDrop.h"

#include "Data/LevelSnapshotsEditorData.h"
#include "Widgets/SLevelSnapshotsEditorFilterRow.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshots"

FFavoriteFilterDragDrop::FFavoriteFilterDragDrop(const TSubclassOf<ULevelSnapshotFilter>& InClassToInstantiate, ULevelSnapshotsEditorData* InSelectedFilterSetter)
	:
	ClassToInstantiate(InClassToInstantiate),
	SelectedFilterSetter(InSelectedFilterSetter)
{
	CursorDecoratorWindow = SWindow::MakeCursorDecorator();
	const bool bShowImmediately = false;
	FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), bShowImmediately);
	
	ShowCannotDropFilter();
}

void FFavoriteFilterDragDrop::OnEnterRow(const TSharedRef<SLevelSnapshotsEditorFilterRow>& EnteredRow)
{
	ShowCanDropFilter();
}

void FFavoriteFilterDragDrop::OnLeaveRow(const TSharedRef<SLevelSnapshotsEditorFilterRow>& LeftRow)
{
	ShowCannotDropFilter();
}

bool FFavoriteFilterDragDrop::OnDropOnRow(const TSharedRef<SLevelSnapshotsEditorFilterRow>& DroppedOnRow)
{
	const TWeakObjectPtr<UConjunctionFilter> AndCondition = DroppedOnRow->GetManagedFilter();
	if (!ensure(AndCondition.IsValid()))
	{
		return false;
	}

	UNegatableFilter* CreatedFilter = AndCondition->CreateChild(ClassToInstantiate);
	SelectedFilterSetter->SetEditedFilter(CreatedFilter);
	return true;
}

void FFavoriteFilterDragDrop::ShowCannotDropFilter()
{
	const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	ShowFeedbackMessage(StatusSymbol, LOCTEXT("CannotDropFilter", "Drop onto a filter group to create filter"));
}

void FFavoriteFilterDragDrop::ShowCanDropFilter()
{
	const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
	ShowFeedbackMessage(StatusSymbol, LOCTEXT("CanDropFilter", "Create new filter"));
}

void FFavoriteFilterDragDrop::ShowFeedbackMessage(const FSlateBrush* Icon, const FText& Message)
{
	if (!Message.IsEmpty())
	{
		CursorDecoratorWindow->ShowWindow();
		CursorDecoratorWindow->SetContent
		(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(SImage)
						.Image(Icon)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				.MaxWidth(500)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.WrapTextAt(480.0f)
					.Text(Message)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				]
			]
		);
	}
	else
	{
		CursorDecoratorWindow->HideWindow();
		CursorDecoratorWindow->SetContent(SNullWidget::NullWidget);
	}
}

#undef LOCTEXT_NAMESPACE