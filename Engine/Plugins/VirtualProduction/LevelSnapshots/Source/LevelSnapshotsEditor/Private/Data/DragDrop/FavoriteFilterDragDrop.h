// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "LevelSnapshotFilters.h"
#include "Templates/SubclassOf.h"

struct FSlateBrush;
class FLevelSnapshotsEditorFilters;
class ULevelSnapshotsEditorData;
class SLevelSnapshotsEditorFilterRow;

/* Governs interaction when a user drags a favorite filter. */
class FFavoriteFilterDragDrop : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FFavoriteFilterDragDrop, FDragDropOperation);

	FFavoriteFilterDragDrop(const TSubclassOf<ULevelSnapshotFilter>& InClassToInstantiate, ULevelSnapshotsEditorData* InSelectedFilterSetter);

	void OnEnterRow(const TSharedRef<SLevelSnapshotsEditorFilterRow>& EnteredRow);
	void OnLeaveRow(const TSharedRef<SLevelSnapshotsEditorFilterRow>& LeftRow);
	bool OnDropOnRow(const TSharedRef<SLevelSnapshotsEditorFilterRow>& DroppedOnRow);

private:

	void ShowCannotDropFilter();
	void ShowCanDropFilter();
	void ShowFeedbackMessage(const FSlateBrush* Icon, const FText& Message);
	
	const TSubclassOf<ULevelSnapshotFilter> ClassToInstantiate;
	/* Used to set the filter being edited to newly created one. */
	const TWeakObjectPtr<ULevelSnapshotsEditorData> SelectedFilterSetter;
	
};
