// Copyright Epic Games, Inc. All Rights Reserved.

#include "FavoriteFilterContainer.h"

#include "AssetRegistry/AssetData.h"
#include "LevelSnapshotFilters.h"
#include "UClassMetaDataDefinitions.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Algo/Transform.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "UObject/UObjectIterator.h"

namespace
{
	const FName NativeClassesCategoryName("C++ Filters");
	const FName BlueprintClassesCategoryName("Blueprint Filters");
	
	bool IsCommonClass(const TSubclassOf<ULevelSnapshotFilter>& ClassToCheck)
	{
		return ClassToCheck->FindMetaData(UClassMetaDataDefinitions::CommonSnapshotFilter) != nullptr;
	}
	bool IsBlueprintClass(const TSubclassOf<ULevelSnapshotFilter>& ClassToCheck)
	{
		return ClassToCheck->IsInBlueprint();
	}
	
	TArray<TSubclassOf<ULevelSnapshotFilter>> FindNativeFilterClasses(const bool bOnlyCommon)
	{
		TArray<TSubclassOf<ULevelSnapshotFilter>> Result;
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* ClassToCheck = *ClassIt;
			if (!ClassToCheck->IsChildOf(ULevelSnapshotFilter::StaticClass()))
			{
				continue;
			}

			const bool bIsClassInstantiatable = ClassToCheck->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract) || FKismetEditorUtilities::IsClassABlueprintSkeleton(ClassToCheck);
			if (bIsClassInstantiatable)
			{
				continue;
			}

			if (IsBlueprintClass(ClassToCheck))
			{
				continue;
			}

			if (bOnlyCommon && !IsCommonClass(ClassToCheck))
			{
				continue;
			}
			const bool bIsInternalOnlyFilter = ClassToCheck->FindMetaData(UClassMetaDataDefinitions::InternalSnapshotFilter) != nullptr;
			if (bIsInternalOnlyFilter)
			{
				continue;
			}

			
			Result.Add(ClassToCheck);
		}
		return Result;
	}

	bool IsBlueprintFilter(const FAssetData& BlueprintClassData)
	{
		UClass* BlueprintFilterClass = ULevelSnapshotBlueprintFilter::StaticClass();
		
		const FString NativeParentClassPath = BlueprintClassData.GetTagValueRef<FString>(FBlueprintTags::NativeParentClassPath);
		const FSoftClassPath ClassPath(NativeParentClassPath);
			
		UClass* NativeParentClass = ClassPath.ResolveClass();
		const bool bInheritsFromBlueprintFilter =
			NativeParentClass // Class may have been removed, or renamed and not correctly redirected
			&& (NativeParentClass == BlueprintFilterClass || NativeParentClass->IsChildOf(BlueprintFilterClass));

		return bInheritsFromBlueprintFilter;
	}
	
	TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>> FindBlueprintFilterClasses()
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		
		TArray<FAssetData> BlueprintList;
		AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), BlueprintList);

		// Returns = 
		TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>> Result;
		for (const FAssetData& BlueprintClassData : BlueprintList)
		{
			if (!IsBlueprintFilter(BlueprintClassData))
			{
				continue;
			}

			// If there are a lot of assets, this may be slow.
			UBlueprint* BlueprintAsset = Cast<UBlueprint>(BlueprintClassData.GetAsset());
			UClass* LoadedClass = BlueprintAsset->GeneratedClass;
			if (ensure(LoadedClass && BlueprintAsset->ParentClass))
			{
				Result.Add(LoadedClass);
			}
		}
		return Result;
	}
}

void UFavoriteFilterContainer::PostInitProperties()
{
	Super::PostInitProperties();

	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (ensure(AssetRegistry))
	{
		OnAssetDeleted = AssetRegistry->OnAssetRemoved().AddLambda([this](const FAssetData& AssetData)
		{
			CleanseFavorites();
		});
	}
}

void UFavoriteFilterContainer::BeginDestroy()
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (AssetRegistry)
	{
		AssetRegistry->OnAssetRemoved().Remove(OnAssetDeleted);
	}
	
	Super::BeginDestroy();
}

void UFavoriteFilterContainer::AddToFavorites(const TSubclassOf<ULevelSnapshotFilter>& NewFavoriteClass)
{
	if (!ensure(NewFavoriteClass))
	{
		return;
	}
	
	const int32 NewIndex = Favorites.AddUnique(NewFavoriteClass);
	const bool bWasNew = NewIndex != INDEX_NONE;
	if (ensure(bWasNew))
	{
		OnFavoritesChanged.Broadcast();
	}
	
	CleanseFavorites();
}

void UFavoriteFilterContainer::RemoveFromFavorites(const TSubclassOf<ULevelSnapshotFilter>& NoLongerFavoriteClass)
{
	if (!ensure(NoLongerFavoriteClass))
	{
		return;
	}

	const bool bIsBlueprint = IsBlueprintClass(NoLongerFavoriteClass);
	if (bIsBlueprint && bIncludeAllBlueprintClasses)
	{
		bIncludeAllBlueprintClasses = false;
	}
	if (!bIsBlueprint && bIncludeAllNativeClasses)
	{
		bIncludeAllNativeClasses = false;
	}
	
	const int32 NumberItemsRemoved = Favorites.RemoveSingle(NoLongerFavoriteClass);
	const bool bWasItemRemoved = NumberItemsRemoved != INDEX_NONE;
	if (ensure(bWasItemRemoved))
	{
		OnFavoritesChanged.Broadcast();
	}

	CleanseFavorites();
}

void UFavoriteFilterContainer::ClearFavorites()
{
	Favorites.Empty();
	OnFavoritesChanged.Broadcast();
}

bool UFavoriteFilterContainer::ShouldIncludeAllClassesInCategory(FName CategoryName) const
{
	if (CategoryName == NativeClassesCategoryName)
	{
		return bIncludeAllNativeClasses;
	}
	if (CategoryName == BlueprintClassesCategoryName)
	{
		return bIncludeAllBlueprintClasses;
	}
	return false;
}

void UFavoriteFilterContainer::SetShouldIncludeAllClassesInCategory(FName CategoryName, bool bShouldIncludeAll)
{
	if (CategoryName == NativeClassesCategoryName)
	{
		SetIncludeAllNativeClasses(bShouldIncludeAll);
	}
	if (CategoryName == BlueprintClassesCategoryName)
	{
		SetIncludeAllBlueprintClasses(bShouldIncludeAll);
	}
}

const TArray<TSubclassOf<ULevelSnapshotFilter>>& UFavoriteFilterContainer::GetFavorites() const
{
	return Favorites;
}

TArray<TSubclassOf<ULevelSnapshotFilter>> UFavoriteFilterContainer::GetCommonFilters() const
{
	// Assume no native C++ classes are added at runtime.
	static TArray<TSubclassOf<ULevelSnapshotFilter>> CachedResult = FindNativeFilterClasses(true);
	return CachedResult;
}

TArray<TSubclassOf<ULevelSnapshotFilter>> UFavoriteFilterContainer::GetAllAvailableFilters() const
{
	TArray<TSubclassOf<ULevelSnapshotFilter>> Result;
	TArray<TSubclassOf<ULevelSnapshotFilter>> NativeFilters = GetAvailableNativeFilters();
	TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>> BlueprintFilters = GetAvailableBlueprintFilters();

	for (const TSubclassOf<ULevelSnapshotFilter>& FilterClass : NativeFilters)
	{
		Result.Add(FilterClass);
	}
	for (const TSubclassOf<ULevelSnapshotBlueprintFilter>& FilterClass : BlueprintFilters)
	{
		Result.Add(FilterClass);
	}

	return Result;
}

TArray<FName> UFavoriteFilterContainer::GetCategories() const
{
	TArray<FName> ReturnValue;

	if (GetAvailableNativeFilters().Num() > 0)
	{
		ReturnValue.Add(NativeClassesCategoryName);
	}

	if (GetAvailableBlueprintFilters().Num() > 0)
	{
		ReturnValue.Add(BlueprintClassesCategoryName);
	}

	return ReturnValue;
}

FText UFavoriteFilterContainer::CategoryNameToText(FName CategoryName) const
{
	return FText::FromName(CategoryName);
}

TArray<TSubclassOf<ULevelSnapshotFilter>> UFavoriteFilterContainer::GetFiltersInCategory(FName CategoryName) const
{
	if (CategoryName == NativeClassesCategoryName)
	{
		return GetAvailableNativeFilters();
	}
	if (CategoryName == BlueprintClassesCategoryName)
	{
		TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>> BlueprintFilters = GetAvailableBlueprintFilters();
		TArray<TSubclassOf<ULevelSnapshotFilter>> Result;
		Algo::Transform(BlueprintFilters, Result, [](const TSubclassOf<ULevelSnapshotBlueprintFilter>& Filter) { return Filter; });
		return Result;
	}
	return {};
}

void UFavoriteFilterContainer::CleanseFavorites()
{
	const int32 NumBeforeRemoval = Favorites.Num();
	for (int32 i = 0; i < Favorites.Num(); ++i)
	{
		if (Favorites[i] == nullptr)
		{
			Favorites.RemoveAt(i);
			--i;
		}
	}

	if (NumBeforeRemoval != Favorites.Num())
	{
		OnFavoritesChanged.Broadcast();
	}
}

void UFavoriteFilterContainer::SetIncludeAllNativeClasses(bool bShouldIncludeNative)
{
	if (bShouldIncludeNative == bIncludeAllNativeClasses)
	{
		return;
	}
	bIncludeAllNativeClasses = bShouldIncludeNative;

	const TArray<TSubclassOf<ULevelSnapshotFilter>>& NativeClasses = GetAvailableNativeFilters();
	for (const TSubclassOf<ULevelSnapshotFilter>& Filter : NativeClasses)
	{
		if (bShouldIncludeNative)
		{
			Favorites.AddUnique(Filter);
		}
		else
		{
			Favorites.RemoveSingle(Filter);
		}
	}
	
	OnFavoritesChanged.Broadcast();
}

void UFavoriteFilterContainer::SetIncludeAllBlueprintClasses(bool bShouldIncludeBlueprint)
{
	if (bShouldIncludeBlueprint == bIncludeAllBlueprintClasses)
	{
		return;
	}
	bIncludeAllBlueprintClasses = bShouldIncludeBlueprint;

	const TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>>& BlueprintClasses = GetAvailableBlueprintFilters();
	for (const TSubclassOf<ULevelSnapshotBlueprintFilter>& Filter : BlueprintClasses)
	{
		if (bShouldIncludeBlueprint)
		{
			Favorites.AddUnique(Filter);
		}
		else
		{
			Favorites.RemoveSingle(Filter);
		}
	}
	
	OnFavoritesChanged.Broadcast();
}

bool UFavoriteFilterContainer::ShouldIncludeAllNativeClasses() const
{
	return bIncludeAllNativeClasses;
}

bool UFavoriteFilterContainer::ShouldIncludeAllBlueprintClasses() const
{
	return bIncludeAllBlueprintClasses;
}

const TArray<TSubclassOf<ULevelSnapshotFilter>>& UFavoriteFilterContainer::GetAvailableNativeFilters() const
{
	// Assume no native C++ classes are added at runtime.
	static TArray<TSubclassOf<ULevelSnapshotFilter>> CachedResult = FindNativeFilterClasses(false);
	return CachedResult;
}

TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>> UFavoriteFilterContainer::GetAvailableBlueprintFilters() const
{
	// Regenerate every time to find blueprints newly added or removed by the user in the editor
	return FindBlueprintFilterClasses();
}
