// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "FavoriteFilterContainer.generated.h"

class ULevelSnapshotFilter;
class ULevelSnapshotBlueprintFilter;

/* Keeps track of selected favorite filters. */
UCLASS()
class UFavoriteFilterContainer : public UObject
{
	GENERATED_BODY()
public:

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	virtual void BeginDestroy() override;
	//~ Begin UObject Interface
	
	void AddToFavorites(const TSubclassOf<ULevelSnapshotFilter>& NewFavoriteClass);
	void RemoveFromFavorites(const TSubclassOf<ULevelSnapshotFilter>& NoLongerFavoriteClass);
	void ClearFavorites();

	bool ShouldIncludeAllClassesInCategory(FName CategoryName) const;
	void SetShouldIncludeAllClassesInCategory(FName CategoryName, bool bShouldIncludeAll);
	
	const TArray<TSubclassOf<ULevelSnapshotFilter>>& GetFavorites() const;
	/* Gets filters with the CommonSnapshotFilter uclass meta tag. */
	TArray<TSubclassOf<ULevelSnapshotFilter>> GetCommonFilters() const;
	/* Gets all valid filter subclasses */
	TArray<TSubclassOf<ULevelSnapshotFilter>> GetAllAvailableFilters() const;
	
	TArray<FName> GetCategories() const;
	FText CategoryNameToText(FName CategoryName) const;
	TArray<TSubclassOf<ULevelSnapshotFilter>> GetFiltersInCategory(FName CategoryName) const;
	
	DECLARE_EVENT(UFavoriteFilterContainer, FOnFavoritesModified);
	FOnFavoritesModified OnFavoritesChanged;
	
private:

	void CleanseFavorites();
	
	void SetIncludeAllNativeClasses(bool bShouldIncludeNative);
	void SetIncludeAllBlueprintClasses(bool bShouldIncludeBlueprint);
	bool ShouldIncludeAllNativeClasses() const;
	bool ShouldIncludeAllBlueprintClasses() const;

	/* Gets C++ filters without CommonSnapshotFilter tag. */
	const TArray<TSubclassOf<ULevelSnapshotFilter>>& GetAvailableNativeFilters() const;
	/* Gets Blueprint filters without CommonSnapshotFilter tag; they cannot have this tag btw. */
	TArray<TSubclassOf<ULevelSnapshotBlueprintFilter>> GetAvailableBlueprintFilters() const;

	
	
	/* The filters the user selected to use. */
	UPROPERTY()
	TArray<TSubclassOf<ULevelSnapshotFilter>> Favorites;
	
	bool bIncludeAllNativeClasses = false;
	bool bIncludeAllBlueprintClasses = false;

	/** So we can remove any filter classes that were favourited */
	FDelegateHandle OnAssetDeleted;
};