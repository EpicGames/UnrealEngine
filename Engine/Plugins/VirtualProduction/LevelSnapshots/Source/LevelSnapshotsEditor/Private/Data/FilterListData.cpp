// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterListData.h"

#include "LevelSnapshot.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotsFilteringLibrary.h"
#include "LevelSnapshotsFunctionLibrary.h"
#include "LevelSnapshotsLog.h"

#include "GameFramework/Actor.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void FFilterListData::UpdateFilteredList(UWorld* World, ULevelSnapshot* FromSnapshot, ULevelSnapshotFilter* FilterToApply)
{
	SCOPED_SNAPSHOT_EDITOR_TRACE(UpdateFilteredList);
	
	// We only track progress of HandleActorExistsInWorldAndSnapshot because the other two functions are relatively fast in comparison: deserialisation takes much longer.
	const int32 ExpectedAmountOfWork = FromSnapshot->GetNumSavedActors();
	FScopedSlowTask DiffDeserializedActors(ExpectedAmountOfWork, LOCTEXT("DiffingActorsKey", "Diffing actors"));
	DiffDeserializedActors.MakeDialogDelayed(1.f);

	RelatedSnapshot = FromSnapshot;
	SelectionMap = ULevelSnapshotsFilteringLibrary::DiffAndFilterSnapshot(World, FromSnapshot, FilterToApply);
}

TWeakObjectPtr<AActor> FFilterListData::GetSnapshotCounterpartFor(const AActor* WorldActor) const
{
	const TOptional<TNonNullPtr<AActor>> DeserializedActor = RelatedSnapshot->GetDeserializedActor(WorldActor);
	return ensureAlwaysMsgf(DeserializedActor, TEXT("Deserialized actor does no exist. Either the snapshots's container world was deleted or the snapshot has no counterpart for this actor"))
		? DeserializedActor->Get() : nullptr;
}

void FFilterListData::ForEachModifiedActor(TFunctionRef<void(AActor*)> Callback) const
{
	TSet<AActor*> AlreadyVisited;
	SelectionMap.ForEachModifiedObject([&AlreadyVisited, Callback](const FSoftObjectPath& ModifiedObject)
	{
		UObject* Resolved = ModifiedObject.ResolveObject();
		AActor* Actor = Cast<AActor>(Resolved);
		Actor = Actor ? Actor : Resolved->GetTypedOuter<AActor>();

		if (Actor && !AlreadyVisited.Contains(Actor))
		{
			AlreadyVisited.Add(Actor);
			Callback(Actor);
		}
	});
}

bool FFilterListData::HasAnyModifiedActors() const
{
	return SelectionMap.HasAnyModifiedActors();
}

#undef LOCTEXT_NAMESPACE
