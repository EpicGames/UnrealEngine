// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Filtering/PropertySelectionMap.h"

class AActor;
class ULevelSnapshot;
class ULevelSnapshotFilter;
struct FScopedSlowTask;

/* Contains all data required to display the filter results panel. */
struct FFilterListData
{
	void UpdateFilteredList(UWorld* World, ULevelSnapshot* FromSnapshot, ULevelSnapshotFilter* FilterToApply);

	/** If WorldActor is modified, returns the deserialized actor. If WorldActor is not modified, returns itself. */
	TWeakObjectPtr<AActor> GetSnapshotCounterpartFor(const AActor* WorldActor) const;

	void ForEachModifiedActor(TFunctionRef<void(AActor*)> Callback) const;
	bool HasAnyModifiedActors() const;
	
	const FPropertySelectionMap& GetModifiedEditorObjectsSelectedProperties_AllowedByFilter() const { return SelectionMap; }
	const TSet<FSoftObjectPath>& GetRemovedOriginalActorPaths_AllowedByFilter() const { return SelectionMap.GetDeletedActorsToRespawn(); }
	const TSet<TWeakObjectPtr<AActor>>& GetAddedWorldActors_AllowedByFilter() const { return SelectionMap.GetNewActorsToDespawn(); }

private:
	
	TWeakObjectPtr<ULevelSnapshot> RelatedSnapshot = nullptr;
	
	/* Selected properties for actors allowed by filters. */
	FPropertySelectionMap SelectionMap;
};