// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/FilterLoader.h"

#include "Data/Filters/LevelSnapshotsFilterPreset.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "Misc/Paths.h"
#include "Misc/TransactionObjectEvent.h"
#include "ScopedTransaction.h"

void UFilterLoader::OverwriteExisting()
{
	const TOptional<FAssetData> AssetData = GetAssetLastSavedOrLoaded();
	if (!ensure(AssetData.IsSet()))
	{
		return;
	}

	// duplicate asset at destination
	UObject* AssetDuplicatedAtCorrectPath = [this, &AssetData]()
	{
		const FString NewPackageName = *AssetData->PackageName.ToString();
		UPackage* DuplicatedPackage = CreatePackage(*NewPackageName);
		UObject* DuplicatedAsset = StaticDuplicateObject(AssetBeingEdited, DuplicatedPackage, *AssetData->AssetName.ToString());

		if (DuplicatedAsset != nullptr)
		{
			// update duplicated asset & notify asset registry
			if (AssetBeingEdited->HasAnyFlags(RF_Transient))
			{
				DuplicatedAsset->ClearFlags(RF_Transient);
				DuplicatedAsset->SetFlags(RF_Public | RF_Standalone);
			}

			if (AssetBeingEdited->GetOutermost()->HasAnyPackageFlags(PKG_DisallowExport))
			{
				DuplicatedPackage->SetPackageFlags(PKG_DisallowExport);
			}

			DuplicatedAsset->MarkPackageDirty();
			FAssetRegistryModule::AssetCreated(DuplicatedAsset);

			// update last save directory
			const FString PackageFilename = FPackageName::LongPackageNameToFilename(NewPackageName);
			const FString PackagePath = FPaths::GetPath(PackageFilename);

			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, PackagePath);
		}
		return DuplicatedAsset;
	}();
	if (!ensure(AssetDuplicatedAtCorrectPath))
	{
		return;
	}
	
	TArray<UObject*> SavedAssets;
	FEditorFileUtils::PromptForCheckoutAndSave({ AssetDuplicatedAtCorrectPath->GetOutermost() }, true, false);

	ULevelSnapshotsFilterPreset* Filter = Cast<ULevelSnapshotsFilterPreset>(AssetDuplicatedAtCorrectPath);
	if (ensure(Filter))
	{
		OnSaveOrLoadAssetOnDisk(Filter);
	}
}

void UFilterLoader::SaveAs()
{
	TArray<UObject*> SavedAssets;
	FEditorFileUtils::SaveAssetsAs({ AssetBeingEdited }, SavedAssets);

	// Remember: user can cancel saving by clicking "cancel"
	const bool bSavedSuccessfully = SavedAssets.Num() == 1;
	if (bSavedSuccessfully)
	{
		UObject* SavedAssetOnDisk = SavedAssets[0];
		OnSaveOrLoadAssetOnDisk(Cast<ULevelSnapshotsFilterPreset>(SavedAssetOnDisk));
	}
}

void UFilterLoader::LoadAsset(const FAssetData& PickedAsset)
{
	UObject* LoadedAsset = PickedAsset.GetAsset();
	if (!ensure(LoadedAsset) || !ensure(Cast<ULevelSnapshotsFilterPreset>(LoadedAsset)))
	{
		return;
	}
	
	ULevelSnapshotsFilterPreset* Filter = Cast<ULevelSnapshotsFilterPreset>(LoadedAsset);
	if (ensure(Filter))
	{
		FScopedTransaction Transaction(FText::FromString("Load filter preset"));
		Modify();
		OnSaveOrLoadAssetOnDisk(Filter);
	}
}

TOptional<FAssetData> UFilterLoader::GetAssetLastSavedOrLoaded() const
{
	UObject* Result = AssetLastSavedOrLoaded.TryLoad();
	return Result ? FAssetData(Result) : TOptional<FAssetData>();
}

void UFilterLoader::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		OnFilterChanged.Broadcast(AssetBeingEdited);
	}
}

void UFilterLoader::SetAssetBeingEdited(ULevelSnapshotsFilterPreset* NewAssetBeingEdited)
{
	AssetBeingEdited = NewAssetBeingEdited;
}

void UFilterLoader::OnSaveOrLoadAssetOnDisk(ULevelSnapshotsFilterPreset* AssetOnDisk)
{
	SetAssetLastSavedOrLoaded(AssetOnDisk);
	
	// Duplicate to avoid referencing asset on disk: if user deletes the asset, this will leave editor with nulled references
	ULevelSnapshotsFilterPreset* DuplicatedFilter = DuplicateObject<ULevelSnapshotsFilterPreset>(AssetOnDisk, GetOutermost());
	// If user does Save as again later, this prevents FEditorFileUtils::SaveAssetsAs from suggesting an invalid file path to a transient package
	DuplicatedFilter->SetFlags(RF_Transient);

	SetAssetBeingEdited(DuplicatedFilter);
	OnFilterChanged.Broadcast(DuplicatedFilter);
}

void UFilterLoader::SetAssetLastSavedOrLoaded(ULevelSnapshotsFilterPreset* NewSavedAsset)
{
	AssetLastSavedOrLoaded = NewSavedAsset;
	OnFilterWasSavedOrLoaded.Broadcast();
}