// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetRegistry/AssetData.h"
#include "Data/Filters/LevelSnapshotsFilterPreset.h"
#include "UObject/Object.h"
#include "FilterLoader.generated.h"

class ULevelSnapshotsFilterPreset;

/* Handles saving and loading of UDisjunctiveNormalFormFilter. */
UCLASS()
class UFilterLoader : public UObject
{
	GENERATED_BODY()
public:

	void OverwriteExisting();
	void SaveAs();
	void LoadAsset(const FAssetData& PickedAsset);

	TOptional<FAssetData> GetAssetLastSavedOrLoaded() const;

	//~ Begin UObject Interface
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	//~ End UObject Interface
	
	DECLARE_EVENT_OneParam(UFilterLoader, FOnLoadedFilters, ULevelSnapshotsFilterPreset*);
	/* Called to notify everybody that the user now wants to edit this filter. */
	FOnLoadedFilters OnFilterChanged;

	DECLARE_EVENT(UFilterLoader, FOnFilterWasSavedOrLoaded);
	/* Used by UI to know when SaveLoaded option may be shown. */
	FOnFilterWasSavedOrLoaded OnFilterWasSavedOrLoaded; 

private:

	friend class ULevelSnapshotsEditorData;
	void SetAssetBeingEdited(ULevelSnapshotsFilterPreset* NewAssetBeingEdited);

	void OnSaveOrLoadAssetOnDisk(ULevelSnapshotsFilterPreset* AssetOnDisk);
	void SetAssetLastSavedOrLoaded(ULevelSnapshotsFilterPreset* NewAsset);
	
	/* Set once user either has used RequestSaveAs or SetPickedAsset. */
	UPROPERTY()
	FSoftObjectPath AssetLastSavedOrLoaded = nullptr;

	UPROPERTY()
	TObjectPtr<ULevelSnapshotsFilterPreset> AssetBeingEdited;
};
