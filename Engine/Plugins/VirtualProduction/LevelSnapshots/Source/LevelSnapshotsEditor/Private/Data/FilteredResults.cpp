// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilteredResults.h"

#include "LevelSnapshot.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotsFunctionLibrary.h"

#include "EngineUtils.h"
#include "Stats/StatsMisc.h"

void UFilteredResults::CleanReferences()
{
	FilteredData = FFilterListData();
	PropertiesToRollback.Empty();
}

void UFilteredResults::SetActiveLevelSnapshot(ULevelSnapshot* InActiveLevelSnapshot)
{
	UserSelectedSnapshot = InActiveLevelSnapshot;
	CleanReferences();
}

void UFilteredResults::SetUserFilters(ULevelSnapshotFilter* InUserFilters)
{
	UserFilters = InUserFilters;
}

void UFilteredResults::UpdateFilteredResults(UWorld* SelectedWorld)
{
	if (!ensure(UserSelectedSnapshot.IsValid()) || !ensure(UserFilters.IsValid()) || !ensure(SelectedWorld))
	{
		return;
	}
 
	// Do not CleanReferences because we want FilteredData to retain some of the memory it has already allocated
	PropertiesToRollback.Empty(false);
	FilteredData.UpdateFilteredList(SelectedWorld, UserSelectedSnapshot.Get(), UserFilters.Get());
}

void UFilteredResults::SetPropertiesToRollback(const FPropertySelectionMap& InSelectionSet)
{
	PropertiesToRollback = InSelectionSet;
}

const FPropertySelectionMap& UFilteredResults::GetPropertiesToRollback() const
{
	return PropertiesToRollback;
}

FFilterListData& UFilteredResults::GetFilteredData()
{
	return FilteredData;
}

TWeakObjectPtr<ULevelSnapshotFilter> UFilteredResults::GetUserFilters() const
{
	return UserFilters;
}
