// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/Object.h"
#include "FilterListData.h"
#include "FilteredResults.generated.h"

class ULevelSnapshot;
class ULevelSnapshotFilter;
class ULevelSnapshotSelectionSet;
class ULevelSnapshotsEditorData;

/* Processes user defined filters into a selection set, which the user inspect in the results tab. */
UCLASS()
class UFilteredResults : public UObject
{
	GENERATED_BODY()
public:

	void CleanReferences();
	
	void SetActiveLevelSnapshot(ULevelSnapshot* InActiveLevelSnapshot);
	void SetUserFilters(ULevelSnapshotFilter* InUserFilters);

	/* Extracts DeserializedActorsAndDesiredPaths and FilterResults is modified. */  
	void UpdateFilteredResults(UWorld* SelectedWorld);

	void SetPropertiesToRollback(const FPropertySelectionMap& InSelectionSet);
	const FPropertySelectionMap& GetPropertiesToRollback() const;
	
	FFilterListData& GetFilteredData();
	TWeakObjectPtr<ULevelSnapshotFilter> GetUserFilters() const;

private:

	TWeakObjectPtr<ULevelSnapshot> UserSelectedSnapshot;

	/* Stores partially filtered data for displaying in filter results view. */
	FFilterListData FilteredData;
	
	TWeakObjectPtr<ULevelSnapshotFilter> UserFilters;
	
	/* Null until UpdatePropertiesToRollback is called. */ 
	FPropertySelectionMap PropertiesToRollback;
};
