// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConjunctionFilter.h"

#include "NegatableFilter.h"
#include "Templates/SubclassOf.h"
#include "ScopedTransaction.h"

namespace
{
	using ConjunctionFilterCallback = TFunction<EFilterResult::Type(UNegatableFilter* Child)>;

	EFilterResult::Type AndChain(const TArray<UNegatableFilter*>& Children, ConjunctionFilterCallback FilterCallback)
	{
		bool bAtLeastOneFilterSaidInclude = false;

		for (UNegatableFilter* ChildFilter : Children)
		{
			const EFilterResult::Type ChildFilterResult = FilterCallback(ChildFilter);

			// Suppose: A and B. If A == false, no need to evaluate B.
			const bool bShortCircuitAndChain = !EFilterResult::CanInclude(ChildFilterResult);
			if (bShortCircuitAndChain)
			{
				return EFilterResult::Exclude;
			}
			
			bAtLeastOneFilterSaidInclude |= EFilterResult::ShouldInclude(ChildFilterResult);
		}

		return bAtLeastOneFilterSaidInclude ? EFilterResult::Include : EFilterResult::DoNotCare;
	}
	
	EFilterResult::Type ExecuteAndChain(bool bIgnoreFilter, const TArray<UNegatableFilter*>& Children, ConjunctionFilterCallback FilterCallback)
	{
		if (bIgnoreFilter || Children.Num() == 0)
		{
			return EFilterResult::DoNotCare;
		}
		
		return  AndChain(Children, FilterCallback);
	}
}

FName UConjunctionFilter::GetChildrenMemberName()
{
	return GET_MEMBER_NAME_CHECKED(UConjunctionFilter, Children);
}

void UConjunctionFilter::MarkTransactional()
{
	SetFlags(RF_Transactional);

	for (UNegatableFilter* Child : Children)
	{
		Child->SetFlags(RF_Transactional);
		Child->GetChildFilter()->SetFlags(RF_Transactional);
	}
}

UNegatableFilter* UConjunctionFilter::CreateChild(const TSubclassOf<ULevelSnapshotFilter>& FilterClass)
{
	if (!ensure(FilterClass.Get()))
	{
		return nullptr;
	}

	FScopedTransaction Transaction(FText::FromString("Add filter to row"));
	Modify();

	ULevelSnapshotFilter* FilterImplementation = NewObject<ULevelSnapshotFilter>(this, FilterClass.Get(), NAME_None, RF_Transactional);
	UNegatableFilter* Child = UNegatableFilter::CreateNegatableFilter(FilterImplementation, this);

	Children.Add(Child);
	OnChildAdded.Broadcast(Child);
	
	return Child;
}

void UConjunctionFilter::RemoveChild(UNegatableFilter* Child)
{
	if (!ensure(Child))
	{
		return;
	}

	FScopedTransaction Transaction(FText::FromString("Remove filter from row"));
	Modify();
	
	const bool bRemovedChild = Children.RemoveSingle(Child) != 0;
	check(bRemovedChild);
	if (bRemovedChild)
	{
		Child->OnRemoved();
		OnChildRemoved.Broadcast(Child);
	}
}

const TArray<UNegatableFilter*>& UConjunctionFilter::GetChildren() const
{
	return Children;
}

void UConjunctionFilter::SetIsIgnored(bool Value)
{
	if (Value != bIgnoreFilter)
	{
		FScopedTransaction Transaction(FText::FromString("Change ignore row"));
		Modify();

		bIgnoreFilter = Value;
	}
}

void UConjunctionFilter::OnRemoved()
{
	for (UNegatableFilter* Child : Children)
	{
		Child->OnRemoved();
	}
}

EFilterResult::Type UConjunctionFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return ExecuteAndChain(bIgnoreFilter, Children, [&Params](UNegatableFilter* Child)
		{
			return Child->IsActorValid(Params);
		});
}

EFilterResult::Type UConjunctionFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return ExecuteAndChain(bIgnoreFilter, Children, [&Params](UNegatableFilter* Child)
		{
			return Child->IsPropertyValid(Params);
		});
}

EFilterResult::Type UConjunctionFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return ExecuteAndChain(bIgnoreFilter, Children, [&Params](UNegatableFilter* Child)
		{
			return Child->IsDeletedActorValid(Params);
		});
}

EFilterResult::Type UConjunctionFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return ExecuteAndChain(bIgnoreFilter, Children, [&Params](UNegatableFilter* Child)
		{
			return Child->IsAddedActorValid(Params);
		});
}
