// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "ConjunctionFilter.generated.h"

class UNegatableFilter;

/*
 * Returns the result of and-ing all child filters.
 * It is valid to have no children: in this case, this filter return false.
 */
UCLASS(meta = (InternalSnapshotFilter))
class UConjunctionFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	static FName GetChildrenMemberName();
	
	void MarkTransactional();
	
	/** Creates a new instance of FilterClass and places it in a new negatable filter.
	 * The resulting negatable filter is added as child.
	 */
	UNegatableFilter* CreateChild(const TSubclassOf<ULevelSnapshotFilter>& FilterClass);
	/** Removes filter created by CreateChild. */
	void RemoveChild(UNegatableFilter* Child);
	const TArray<UNegatableFilter*>& GetChildren() const;

	void SetIsIgnored(bool Value);
	bool IsIgnored() const { return bIgnoreFilter; }
	
	void OnRemoved();

	
	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

	DECLARE_EVENT_OneParam(UConjunctionFilter, FOnChildModified, UNegatableFilter*);
	FOnChildModified OnChildAdded;
	FOnChildModified OnChildRemoved;
	
private:

	UPROPERTY(Instanced)
	TArray<TObjectPtr<UNegatableFilter>> Children;

	UPROPERTY()
	bool bIgnoreFilter = false;
};
