// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsFilterPreset.h"

#include "Data/Filters/ConjunctionFilter.h"

#include "Misc/TransactionObjectEvent.h"
#include "ScopedTransaction.h"
#include "UObject/UObjectGlobals.h"

namespace
{
	bool AreAllChildrenEmpty(const TArray<UConjunctionFilter*>& AndFilters)
	{
		if (AndFilters.Num() == 0)
		{
			return true;
		}
		
		for (UConjunctionFilter* Child : AndFilters)
		{
			const bool bIsEmpty = Child->GetChildren().Num() == 0;
			if (!bIsEmpty)
			{
				return false;
			}
		}
		return true;
	}
	
	using FilterCallback = TFunction<EFilterResult::Type(UConjunctionFilter* Child)>;
	EFilterResult::Type ExecuteOrChain(const TArray<UConjunctionFilter*>& Children, FilterCallback&& FilterCallback)
	{
		if (AreAllChildrenEmpty(Children))
		{
			// "Illogical" edge case: No filter specified
			// For better UX, we show all actors and properties to user
			// Logic says we should return DoNotCare
			return EFilterResult::Include;
		}
		
		bool bNoFilterSaidExclude = true;
		
		for (UConjunctionFilter* ChildFilter : Children)
		{
			const TEnumAsByte<EFilterResult::Type> ChildResult = FilterCallback(ChildFilter);

			// Suppose: A or B. If A == true, no need to evaluate B.
			const bool bShortCircuitOrChain = EFilterResult::ShouldInclude(ChildResult);
			if (bShortCircuitOrChain)
			{
				return EFilterResult::Include;
			}
			
			bNoFilterSaidExclude &= EFilterResult::CanInclude(ChildResult);
		}
		
		return bNoFilterSaidExclude ? EFilterResult::DoNotCare : EFilterResult::Exclude;
	}
}

void ULevelSnapshotsFilterPreset::MarkTransactional()
{
	SetFlags(RF_Transactional);
	for (UConjunctionFilter* Child : Children)
	{
		Child->MarkTransactional();
	}

	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddLambda([this](UObject* ModifiedObject, const FTransactionObjectEvent& TransactionInfo)
	{
		if (!ModifiedObject || bHasJustCreatedNewChild)
		{
			return;
		}

		const bool bChangedChildArray = ModifiedObject == this && TransactionInfo.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(ULevelSnapshotsFilterPreset, Children));
		if (bChangedChildArray)
		{
			OnFilterModified.Broadcast(EFilterChangeType::RowChildFilterAddedOrRemoved);
			return;
		}
		
		const bool bChangedChildInstance = ModifiedObject != this && ModifiedObject->IsIn(this);
		const bool bConjunctionWasAddedOrRemoved = TransactionInfo.HasPendingKillChange();
		if (bChangedChildInstance && !bConjunctionWasAddedOrRemoved)
		{
			const bool bConjunctionChildrenWereChanged = TransactionInfo.GetChangedProperties().Contains(UConjunctionFilter::GetChildrenMemberName());
			const bool bModifiedFilterHierarchy = Cast<UConjunctionFilter>(ModifiedObject) && (bConjunctionWasAddedOrRemoved || bConjunctionChildrenWereChanged);
			OnFilterModified.Broadcast(bModifiedFilterHierarchy ? EFilterChangeType::RowChildFilterAddedOrRemoved : EFilterChangeType::FilterPropertyModified);
		}
	});
}

UConjunctionFilter* ULevelSnapshotsFilterPreset::CreateChild()
{
	UConjunctionFilter* Child;
	{
		FScopedTransaction Transaction(FText::FromString("Add filter row"));
		Modify();
		
		Child = NewObject<UConjunctionFilter>(this, UConjunctionFilter::StaticClass(), NAME_None, RF_Transactional);
		Children.Add(Child);

		OnFilterModified.Broadcast(EFilterChangeType::BlankRowAdded);
		// We broadcast OnFilterModified already. Avoid OnObjectTransacted triggering it again.
		bHasJustCreatedNewChild = true;
	}
	bHasJustCreatedNewChild = false;
	
	
	return Child;
}

void ULevelSnapshotsFilterPreset::RemoveConjunction(UConjunctionFilter* Child)
{
	FScopedTransaction Transaction(FText::FromString("Remove filter row"));
	Modify();
	
	const bool bRemovedChild = Children.RemoveSingle(Child) != 0;
	if (ensure(bRemovedChild))
	{
		Child->OnRemoved();
		OnFilterModified.Broadcast(EFilterChangeType::RowRemoved);
	}
}

const TArray<UConjunctionFilter*>& ULevelSnapshotsFilterPreset::GetChildren() const
{
	return Children;
}

void ULevelSnapshotsFilterPreset::BeginDestroy()
{
	if (OnObjectTransactedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
		OnObjectTransactedHandle.Reset();
	}

	Super::BeginDestroy();
}

EFilterResult::Type ULevelSnapshotsFilterPreset::IsActorValid(const FIsActorValidParams& Params) const
{
	return ExecuteOrChain(Children, [&Params](UConjunctionFilter* Child)
	{
		return Child->IsActorValid(Params);
	});
}

EFilterResult::Type ULevelSnapshotsFilterPreset::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return ExecuteOrChain(Children, [&Params](UConjunctionFilter* Child)
	{
		return Child->IsPropertyValid(Params);
	});
}

EFilterResult::Type ULevelSnapshotsFilterPreset::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return ExecuteOrChain(Children, [&Params](UConjunctionFilter* Child)
	{
		return Child->IsDeletedActorValid(Params);
	});
}

EFilterResult::Type ULevelSnapshotsFilterPreset::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return ExecuteOrChain(Children, [&Params](UConjunctionFilter* Child)
	{
		return Child->IsAddedActorValid(Params);
	});
}
