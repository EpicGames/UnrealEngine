// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotsFilterPreset.generated.h"

class UConjunctionFilter;
class UNegatableFilter;

enum class EFilterChangeType : uint8
{
	/* A blank row was added */
	BlankRowAdded,
	/* A row was removed */
	RowRemoved,
	/* A filter was added or removed to a row */
	RowChildFilterAddedOrRemoved,
	/* A filter property was modified, e.g. set to ignored. */
	FilterPropertyModified
};

/*
 * Manages logic for combining filters in the editor.
 * This filter may have no children: in this case, the filter returns true.
 *
 * Disjunctive normal form = ORs of ANDs. Example: (a && !b) || (c && d) || e
 */
UCLASS(meta = (InternalSnapshotFilter))
class ULevelSnapshotsFilterPreset : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	void MarkTransactional();
	
	/** Creates a new AND-filter and adds it to the list of children. */
	UConjunctionFilter* CreateChild();
	/** Removes a filter previously created by CreateChild. */
	void RemoveConjunction(UConjunctionFilter* Child);
	const TArray<UConjunctionFilter*>& GetChildren() const;

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface
	
	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

	
	/**
	* Called when:
	* - An AND row is added or removed
	* - A negatable filter is added or removed
	* - The properties of one of the filters is modified
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FModifyFilter, EFilterChangeType);
	FModifyFilter OnFilterModified;

private:
	
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UConjunctionFilter>> Children;

	bool bHasJustCreatedNewChild = false;
	
	FDelegateHandle OnObjectTransactedHandle;
};
