// Copyright Epic Games, Inc. All Rights Reserved.

#include "NegatableFilter.h"
#include "ScopedTransaction.h"

namespace
{
	EFilterResult::Type ApplyFilter(bool bIgnoreFilter, EFilterBehavior NegationBehaviour, TFunction<EFilterResult::Type()> ApplyFilter)
	{
		if (bIgnoreFilter)
		{
			return EFilterResult::DoNotCare; 
		}

		const EFilterResult::Type Result = ApplyFilter();
		const bool bShouldNegate = NegationBehaviour == EFilterBehavior::Negate;
		return bShouldNegate ? EFilterResult::Negate(Result) : Result;
	}
}

UNegatableFilter* UNegatableFilter::CreateNegatableFilter(ULevelSnapshotFilter* ChildFilter, const TOptional<UObject*>& Outer)
{
	if (!ensure(ChildFilter))
	{
		return nullptr;
	}
	
	UObject* NewFilterOuter = Outer.Get(ChildFilter->GetOuter());
	UNegatableFilter* Result = NewObject<UNegatableFilter>(NewFilterOuter, StaticClass(), NAME_None, RF_Transactional);
	Result->ChildFilter = ChildFilter;
	return Result;
}

void UNegatableFilter::SetFilterBehaviour(EFilterBehavior NewFilterBehavior)
{
	if (NewFilterBehavior != FilterBehavior)
	{
		FScopedTransaction Transaction(FText::FromString("Change negation behavior"));
		Modify();
		
		FilterBehavior = NewFilterBehavior;
	}
}

void UNegatableFilter::SetIsIgnored(bool Value)
{
	if (Value != bIgnoreFilter)
	{
		FScopedTransaction Transaction(FText::FromString("Change ignore filter"));
		Modify();

		bIgnoreFilter = Value;
	}
}

void UNegatableFilter::OnRemoved()
{
	OnFilterDestroyed.Broadcast(this);
}

FText UNegatableFilter::GetDisplayName() const
{
	// Child filter is nullptr when user deletes filter class and force deletes
	if (!ChildFilter)
	{
		return FText::FromString(TEXT("Missing Filter Class"));
	}

	const bool bHasName = Name.Len() != 0;
	return bHasName ? FText::FromString(Name) : ChildFilter->GetClass()->GetDisplayNameText();
}

EFilterResult::Type UNegatableFilter::IsActorValid(const FIsActorValidParams& Params) const
{
	return ApplyFilter(bIgnoreFilter, FilterBehavior, [this, &Params]()
	{
		// Child filter is nullptr when user deletes filter class and force deletes
		return ChildFilter ? ChildFilter->IsActorValid(Params) : EFilterResult::DoNotCare;
	});
}

EFilterResult::Type UNegatableFilter::IsPropertyValid(const FIsPropertyValidParams& Params) const
{
	return ApplyFilter(bIgnoreFilter, FilterBehavior, [this, &Params]()
	{
		// Child filter is nullptr when user deletes filter class and force deletes
		return ChildFilter ? ChildFilter->IsPropertyValid(Params) : EFilterResult::DoNotCare;
	});
}

EFilterResult::Type UNegatableFilter::IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const
{
	return ApplyFilter(bIgnoreFilter, FilterBehavior, [this, &Params]()
	{
		// Child filter is nullptr when user deletes filter class and force deletes
		return ChildFilter ? ChildFilter->IsDeletedActorValid(Params) : EFilterResult::DoNotCare;
	});
}

EFilterResult::Type UNegatableFilter::IsAddedActorValid(const FIsAddedActorValidParams& Params) const
{
	return ApplyFilter(bIgnoreFilter, FilterBehavior, [this, &Params]()
	{
		// Child filter is nullptr when user deletes filter class and force deletes
		return ChildFilter ? ChildFilter->IsAddedActorValid(Params) : EFilterResult::DoNotCare;
	});
}
