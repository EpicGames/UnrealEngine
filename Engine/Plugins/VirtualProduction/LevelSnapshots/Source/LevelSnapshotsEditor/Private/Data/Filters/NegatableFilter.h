// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotFilters.h"
#include "NegatableFilter.generated.h"

UENUM()
enum class EFilterBehavior : uint8
{
	/* Pass on same result */
	DoNotNegate,
	/* Invert the result */ 
	Negate
};

/*
 * Calls a child filter and possibly negates its results.
 */
UCLASS(meta = (InternalSnapshotFilter))
class UNegatableFilter : public ULevelSnapshotFilter
{
	GENERATED_BODY()
public:

	/* Wraps the given filter with a negation. Defaults to ChildFilter's outer. */
	static UNegatableFilter* CreateNegatableFilter(ULevelSnapshotFilter* ChildFilter, const TOptional<UObject*>& Outer = TOptional<UObject*>());

	void SetFilterBehaviour(EFilterBehavior NewFilterBehavior);
	EFilterBehavior GetFilterBehavior() const { return FilterBehavior; }
	
	void SetIsIgnored(bool Value);
	bool IsIgnored() const { return bIgnoreFilter; }

	void OnRemoved();
	
	FText GetDisplayName() const;
	ULevelSnapshotFilter* GetChildFilter() const { return ChildFilter; }

	//~ Begin ULevelSnapshotFilter Interface
	virtual EFilterResult::Type IsActorValid(const FIsActorValidParams& Params) const override;
	virtual EFilterResult::Type IsPropertyValid(const FIsPropertyValidParams& Params) const override;
	virtual EFilterResult::Type IsDeletedActorValid(const FIsDeletedActorValidParams& Params) const override;
	virtual EFilterResult::Type IsAddedActorValid(const FIsAddedActorValidParams& Params) const override;
	//~ End ULevelSnapshotFilter Interface

	DECLARE_EVENT_OneParam(UNegatableFilter, FFilterDestroyed, UNegatableFilter*);
	FFilterDestroyed OnFilterDestroyed;

public: // Only public to use GET_MEMBER_NAME_CHECKED with compiler checks - do not use directly.
	
	/* Display name in editor. Defaults to class name if left empty. */
	UPROPERTY(EditAnywhere, Category = "Filter")
	FString Name;

private:

	/* Whether to pass on the result of the filter, negate it, or ignore it. */
	UPROPERTY(EditAnywhere, Category = "Filter")
	EFilterBehavior FilterBehavior = EFilterBehavior::DoNotNegate;

	/* Whether this filter should be ignored */
	UPROPERTY(EditAnywhere, Category = "Filter")
	bool bIgnoreFilter = false;
	
	UPROPERTY(Instanced)
	TObjectPtr<ULevelSnapshotFilter> ChildFilter;
};