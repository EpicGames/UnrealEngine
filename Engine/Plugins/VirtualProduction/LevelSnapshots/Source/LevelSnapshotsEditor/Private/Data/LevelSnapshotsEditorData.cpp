// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorData.h"

#include "FavoriteFilterContainer.h"
#include "FilterLoader.h"
#include "FilteredResults.h"
#include "LevelSnapshotsLog.h"

#include "Editor.h"
#include "Engine/World.h"
#include "UObject/UObjectGlobals.h"

ULevelSnapshotsEditorData::ULevelSnapshotsEditorData(const FObjectInitializer& ObjectInitializer)
{
	FavoriteFilters = ObjectInitializer.CreateDefaultSubobject<UFavoriteFilterContainer>(
		this,
		TEXT("FavoriteFilters")
		);
	UserDefinedFilters = ObjectInitializer.CreateDefaultSubobject<ULevelSnapshotsFilterPreset>(
		this,
		TEXT("UserDefinedFilters")
		);

	TrackedFilterModifiedHandle = UserDefinedFilters->OnFilterModified.AddUObject(this, &ULevelSnapshotsEditorData::HandleFilterChange);
	
	FilterLoader = ObjectInitializer.CreateDefaultSubobject<UFilterLoader>(
		this,
		TEXT("FilterLoader")
		);
	FilterLoader->SetFlags(RF_Transactional);
	FilterLoader->SetAssetBeingEdited(UserDefinedFilters);
	FilterLoader->OnFilterChanged.AddLambda([this](ULevelSnapshotsFilterPreset* NewFilterToEdit)
	{
		Modify();
		ULevelSnapshotsFilterPreset* OldFilter = UserDefinedFilters;
		UserDefinedFilters = NewFilterToEdit;
		UserDefinedFilters->MarkTransactional();
		UserDefinedFilters->OnFilterModified.AddUObject(this, &ULevelSnapshotsEditorData::HandleFilterChange);

		FilterResults->Modify();
		FilterResults->SetUserFilters(UserDefinedFilters);
		OnUserDefinedFiltersChanged.Broadcast(NewFilterToEdit, OldFilter);

		OldFilter->OnFilterModified.Remove(TrackedFilterModifiedHandle);
		
		SetIsFilterDirty(true);
	});

	FilterResults = ObjectInitializer.CreateDefaultSubobject<UFilteredResults>(
        this,
        TEXT("FilterResults")
        );
	FilterResults->SetUserFilters(UserDefinedFilters);
}

void ULevelSnapshotsEditorData::PostInitProperties()
{
	Super::PostInitProperties();

	// Class default should not register to global events
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UserDefinedFilters->MarkTransactional();
		OnObjectsEdited = FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &ULevelSnapshotsEditorData::HandleWorldActorsEdited);
		
		OnWorldCleanup = FWorldDelegates::OnWorldCleanup.AddLambda([this](UWorld* World, bool bSessionEnded, bool bCleanupResources)
		{
			ClearActiveSnapshot();
		});
	}
}

void ULevelSnapshotsEditorData::BeginDestroy()
{
	Super::BeginDestroy();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanup);
		FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectsEdited);
		
		OnWorldCleanup.Reset();
		OnObjectsEdited.Reset();
	}

	check(!OnWorldCleanup.IsValid());
	check(!OnObjectsEdited.IsValid());
}

void ULevelSnapshotsEditorData::CleanupAfterEditorClose()
{
	OnActiveSnapshotChanged.Clear();
	OnEditedFilterChanged.Clear();
	OnUserDefinedFiltersChanged.Clear();

	ActiveSnapshot = nullptr;
	EditedFilter = nullptr;

	FilterResults->CleanReferences();
}

void ULevelSnapshotsEditorData::SetActiveSnapshot(ULevelSnapshot* NewActiveSnapshot)
{
	SCOPED_SNAPSHOT_EDITOR_TRACE(SetActiveSnapshot);

	if (bIsApplyingSnapshot)
	{
		return;
	}

	if (ActiveSnapshot)
	{
		ActiveSnapshot->OnPreApplySnapshot().RemoveAll(this);
		ActiveSnapshot->OnPostApplySnapshot().RemoveAll(this);
	}
	
	ActiveSnapshot = NewActiveSnapshot;
	if (ActiveSnapshot)
	{
		ActiveSnapshot->OnPreApplySnapshot().AddUObject(this, &ULevelSnapshotsEditorData::OnPreApplySnapshot);
		ActiveSnapshot->OnPostApplySnapshot().AddUObject(this, &ULevelSnapshotsEditorData::OnPostApplySnapshot);
	}
	
	FilterResults->SetActiveLevelSnapshot(NewActiveSnapshot);
	OnActiveSnapshotChanged.Broadcast(GetActiveSnapshot());
}

UWorld* ULevelSnapshotsEditorData::GetEditorWorld()
{
	// If this function is called very early during startup, the initial editor GWorld may not have been created yet!
	const bool bIsEngineInitialised = GEditor && GEditor->GetWorldContexts().Num() > 0;
	if (bIsEngineInitialised)
	{
		if (UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			return World;
		}
	}
	return nullptr;
}

void ULevelSnapshotsEditorData::SetEditedFilter(UNegatableFilter* InFilter)
{
	if (EditedFilter != InFilter)
	{
		EditedFilter = InFilter;
		OnEditedFilterChanged.Broadcast(GetEditedFilter());
	}
}

UFavoriteFilterContainer* ULevelSnapshotsEditorData::GetFavoriteFilters() const
{
	return FavoriteFilters;
}

ULevelSnapshotsFilterPreset* ULevelSnapshotsEditorData::GetUserDefinedFilters() const
{
	return UserDefinedFilters;
}

UFilterLoader* ULevelSnapshotsEditorData::GetFilterLoader() const
{
	return FilterLoader;
}

UFilteredResults* ULevelSnapshotsEditorData::GetFilterResults() const
{
	return FilterResults;
}

void ULevelSnapshotsEditorData::HandleFilterChange(EFilterChangeType FilterChangeType)
{
	// Blank rows do not functionally change the filter
	if (FilterChangeType != EFilterChangeType::BlankRowAdded)
	{
		SetIsFilterDirty(true);
	}
}

bool ULevelSnapshotsEditorData::IsFilterDirty() const
{
	return bIsFilterDirty;
}

void ULevelSnapshotsEditorData::SetIsFilterDirty(const bool bNewDirtyState)
{
	bIsFilterDirty = bNewDirtyState;
}

namespace
{
	bool IsInAnyLevelOf(UWorld* OwningWorld, UObject* ObjectToTest)
	{
		if (ObjectToTest->IsIn(OwningWorld))
		{
			return true;
		}

		for (ULevel* Level : OwningWorld->GetLevels())
		{
			if (ObjectToTest->IsIn(Level))
			{
				return true;
			}
		}

		return false;
	}
}

void ULevelSnapshotsEditorData::HandleWorldActorsEdited(UObject* Object)
{
	if (UWorld* World = GetEditorWorld())
	{
		if (Object && IsInAnyLevelOf(World, Object))
		{
			SetIsFilterDirty(true);
		}
	}
}
