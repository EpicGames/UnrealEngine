// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"

#include "LevelSnapshot.h"
#include "Data/Filters/LevelSnapshotsFilterPreset.h"
#include "Data/Filters/NegatableFilter.h"
#include "LevelSnapshotsEditorData.generated.h"

class UFavoriteFilterContainer;
class UFilterLoader;
class UFilteredResults;
class ULevelSnapshot;

/* Stores all data shared across the editor's UI. */
UCLASS()
class LEVELSNAPSHOTSEDITOR_API ULevelSnapshotsEditorData : public UObject
{
	GENERATED_BODY()
public:

	ULevelSnapshotsEditorData(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	virtual void BeginDestroy() override;
	//~ End UObject Interface
	
	/* Called when the editor is about to be closed. Clears all pending subscriptions to avoid any memory leaks. */
	void CleanupAfterEditorClose();

	
	/******************** Active snapshot ********************/

	/** @param NewActiveSnapshot Can be nullptr */
	void SetActiveSnapshot(ULevelSnapshot* NewActiveSnapshot);
	void ClearActiveSnapshot() { SetActiveSnapshot(nullptr); }
	ULevelSnapshot* GetActiveSnapshot() const { return ActiveSnapshot; }

	DECLARE_EVENT_OneParam(ULevelSnapshotsEditorData, FOnActiveSnapshotChanged, ULevelSnapshot* /* NewSnapshot */);
	FOnActiveSnapshotChanged OnActiveSnapshotChanged;


	/******************** Selected world ********************/
	
	static UWorld* GetEditorWorld();


	
	/******************** Edited filter ********************/
	
	void SetEditedFilter(UNegatableFilter* InFilter);
	UNegatableFilter* GetEditedFilter() const { return EditedFilter; }
	bool IsEditingFilter(UNegatableFilter* Filter) const { return Filter == GetEditedFilter(); }

	DECLARE_EVENT_OneParam(ULevelSnapshotsEditorData, FOnEditedFilterChanged, UNegatableFilter* /*NewEditedFilter*/);
	FOnEditedFilterChanged OnEditedFilterChanged;

	DECLARE_EVENT(ULevelSnapshotsEditorData, FOnRefreshResults);
	FOnRefreshResults OnRefreshResults;

	
	/******************** Loading filter preset ********************/
	
	DECLARE_EVENT_TwoParams(ULevelSnapshotsEditorData, FUserDefinedFiltersChanged, ULevelSnapshotsFilterPreset* /*NewFilter*/, ULevelSnapshotsFilterPreset* /* OldFilter */);
	/* Called when user loads a new set of filters. */
	FUserDefinedFiltersChanged OnUserDefinedFiltersChanged;
	
	
	/******************** Getters ********************/
	
	UFavoriteFilterContainer* GetFavoriteFilters() const;
	ULevelSnapshotsFilterPreset* GetUserDefinedFilters() const;
	UFilterLoader* GetFilterLoader() const;
	UFilteredResults* GetFilterResults() const;

	void HandleFilterChange(EFilterChangeType FilterChangeType);
	bool IsFilterDirty() const;
	void SetIsFilterDirty(const bool bNewDirtyState);

	void HandleWorldActorsEdited(UObject* Object);
	
private:

	FDelegateHandle OnWorldCleanup;
	FDelegateHandle OnObjectsEdited;
	FDelegateHandle TrackedFilterModifiedHandle;
	
	UPROPERTY()
	TObjectPtr<UFavoriteFilterContainer> FavoriteFilters;
	/* Stores user-defined filters in chain of ORs of ANDs. */
	UPROPERTY()
	TObjectPtr<ULevelSnapshotsFilterPreset> UserDefinedFilters;
	/* Handles save & load requests for exchanging UserDefinedFilters. */
	UPROPERTY()
	TObjectPtr<UFilterLoader> FilterLoader;
	
	/* Used for determining whether the filter state has changed since it was last refreshed. */
	UPROPERTY()
	bool bIsFilterDirty = false;

	/** Whether ActiveSnapshot is currently being restored. Does not allow changing the active snapshot while true.  */
	UPROPERTY()
	bool bIsApplyingSnapshot = false;

	/* Converts UserDefinedFilters into ULevelSnapshotsSelectionSet display in results view. */
	UPROPERTY()
	TObjectPtr<UFilteredResults> FilterResults;

	/* Snapshot selected by user */
	UPROPERTY()
	TObjectPtr<ULevelSnapshot> ActiveSnapshot;
	/* Filter visible in details panel */
	UPROPERTY()
	TObjectPtr<UNegatableFilter> EditedFilter;

	void OnPreApplySnapshot() { bIsApplyingSnapshot = true; }
	void OnPostApplySnapshot() { bIsApplyingSnapshot = false; }
};
