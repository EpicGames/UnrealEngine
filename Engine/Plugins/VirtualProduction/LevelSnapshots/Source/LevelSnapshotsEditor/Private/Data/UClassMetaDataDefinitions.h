// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/* Special meta data used to internally label filter classes. These tags are uses by UFavoriteFilterContainer.
 * Usage: UCLASS(meta = (CommonSnapshotFilter)) or UCLASS(meta = (InternalSnapshotFilter)).
 */
namespace UClassMetaDataDefinitions
{
	/* This filter is shown directly in the favorite filter menu opposed to the native and blueprint filter submenus.
	 * Incompatible with InternalSnapshotFilter.
	 */
	const FName CommonSnapshotFilter("CommonSnapshotFilter");
	/* This filter does not show up in the favorite filter menu.
	 * Incompatible with CommonSnapshotFilter.
	 */
	const FName InternalSnapshotFilter("InternalSnapshotFilter");
}