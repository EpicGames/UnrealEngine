// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/LevelSnapshotEditorFactory.h"

#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"

#include "LevelSnapshot.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotEditorFactory"

ULevelSnapshotEditorFactory::ULevelSnapshotEditorFactory()
{
	SupportedClass = ULevelSnapshot::StaticClass();

	// This factory manufacture new objects from scratch.
	bCreateNew = true;

	// This factory will open the editor for each new object.
	bEditAfterNew = true;
};

UObject* ULevelSnapshotEditorFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<ULevelSnapshot>(InParent, InClass, InName, Flags);
};


bool ULevelSnapshotEditorFactory::ShouldShowInNewMenu() const
{
	return true;
}
uint32 ULevelSnapshotEditorFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

#undef LOCTEXT_NAMESPACE
