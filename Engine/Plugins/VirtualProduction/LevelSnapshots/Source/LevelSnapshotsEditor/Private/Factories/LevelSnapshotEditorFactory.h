// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "LevelSnapshotEditorFactory.generated.h"

UCLASS(MinimalAPI)
class ULevelSnapshotEditorFactory : public UFactory
{
	GENERATED_BODY()

public:

	ULevelSnapshotEditorFactory();
	
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual uint32 GetMenuCategories() const override;
};