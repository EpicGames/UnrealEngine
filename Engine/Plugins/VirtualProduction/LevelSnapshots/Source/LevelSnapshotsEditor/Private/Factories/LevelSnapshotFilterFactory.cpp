// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotFilterFactory.h"
#include "LevelSnapshotFilters.h"

#include "AssetToolsModule.h"
#include "Kismet2/KismetEditorUtilities.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotFilterFactory"

ULevelSnapshotBlueprintFilterFactory::ULevelSnapshotBlueprintFilterFactory()
{
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = ULevelSnapshotBlueprintFilter::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

FText ULevelSnapshotBlueprintFilterFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Level Snapshot Filter");
}

UObject* ULevelSnapshotBlueprintFilterFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UBlueprint* FilterBlueprint = nullptr;
	if (ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		FilterBlueprint = FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), NAME_None);
	}
	return FilterBlueprint;	
}

uint32 ULevelSnapshotBlueprintFilterFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

TArray<FAssetCategoryPath> ULevelSnapshotBlueprintFilterFactory::GetAssetMenuPathsForCategory(FName InCategory) const
{
	if (SupportedClass == UBlueprint::StaticClass() && ParentClass)
	{
		return GetAssetMenuPaths(ParentClass, InCategory);
	}
	
	return Super::GetAssetMenuPathsForCategory(InCategory);
}

#undef LOCTEXT_NAMESPACE
