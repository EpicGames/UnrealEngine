// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"

#include "LevelSnapshotFilterFactory.generated.h"

UCLASS(hidecategories = Object)
class ULevelSnapshotBlueprintFilterFactory : public UFactory
{
	GENERATED_BODY()
public:
	ULevelSnapshotBlueprintFilterFactory();
	
	// The parent class of the created blueprint
    UPROPERTY(VisibleAnywhere, Category="LevelSnapshotBlueprintFilterFactory", meta=(AllowAbstract = "", BlueprintBaseOnly = ""))
    TSubclassOf<UObject> ParentClass;
	
	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual TArray<FAssetCategoryPath> GetAssetMenuPathsForCategory(FName InCategory) const override;
	//~ Begin UFactory Interface
	
};