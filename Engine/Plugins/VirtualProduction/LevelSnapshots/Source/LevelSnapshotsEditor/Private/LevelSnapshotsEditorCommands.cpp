// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorCommands.h"

#include "LevelSnapshotsEditorStyle.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void FLevelSnapshotsEditorCommands::RegisterCommands()
{
	UI_COMMAND(Apply, "Apply", "Apply snapshot to the world", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UpdateResults, "UpdateResults", "Uses filters on selected snapshot and updates the results tab", EUserInterfaceActionType::Button, FInputChord());

	FUICommandInfo::MakeCommandInfo(
		this->AsShared(),
		UseCreationFormToggle,
		FName("UseCreationFormToggle"),
		NSLOCTEXT("LevelSnapshots", "UseCreationForm", "Use Creation Form"),
		NSLOCTEXT("LevelSnapshots", "UseCreationFormTooltip", "Use Creation Form"),
		FSlateIcon(),
		EUserInterfaceActionType::ToggleButton,
		FInputChord()
	);

	FUICommandInfo::MakeCommandInfo(
		this->AsShared(),
		OpenLevelSnapshotsEditorToolbarButton,
		FName("OpenLevelSnapshotsEditorToolbarButton"),
		NSLOCTEXT("LevelSnapshots", "OpenLevelSnapshotsEditorToolbarButton", "Open Level Snapshots Editor"),
		NSLOCTEXT("LevelSnapshots", "OpenLevelSnapshotsEditorTooltip", "Open Level Snapshots Editor"),
		FSlateIcon(),
		EUserInterfaceActionType::Button,
		FInputChord()
	);

	FUICommandInfo::MakeCommandInfo(
		this->AsShared(),
		LevelSnapshotsSettings,
		FName("LevelSnapshotsSettings"),
		NSLOCTEXT("LevelSnapshots", "LevelSnapshotsSettings", "Level Snapshots Settings"),
		NSLOCTEXT("LevelSnapshots", "LevelSnapshotsSettingsTooltip", "Open Level Snapshots Settings"),
		FSlateIcon(),
		EUserInterfaceActionType::Button,
		FInputChord()
	);
}

#undef LOCTEXT_NAMESPACE
