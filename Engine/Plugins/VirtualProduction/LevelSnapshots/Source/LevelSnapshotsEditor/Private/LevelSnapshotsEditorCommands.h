// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"


class FLevelSnapshotsEditorCommands
	: public TCommands<FLevelSnapshotsEditorCommands>
{
public:
	FLevelSnapshotsEditorCommands()
		: TCommands<FLevelSnapshotsEditorCommands>(TEXT("LevelSnapshotsEditor"),
			NSLOCTEXT("Contexts", "LevelSnapshotsEditor", "Level Snapshots Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{ }

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> Apply;
	TSharedPtr<FUICommandInfo> UpdateResults;
	
	TSharedPtr<FUICommandInfo> UseCreationFormToggle;
	TSharedPtr<FUICommandInfo> OpenLevelSnapshotsEditorToolbarButton;
	TSharedPtr<FUICommandInfo> LevelSnapshotsSettings;
};
