// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorFunctionLibrary.h"

#include "LevelSnapshotsLog.h"

#include "Editor.h"
#include "Engine/World.h"
#include "LevelSnapshot.h"
#include "Logging/MessageLog.h"
#include "UObject/MetaData.h"
#include "Util/TakeSnapshotUtil.h"

ULevelSnapshot* ULevelSnapshotsEditorFunctionLibrary::TakeLevelSnapshotAndSaveToDisk(const UObject* WorldContextObject, const FString& FileName, const FString& FolderPath, const FString& Description, const bool bShouldCreateUniqueFileName)
{	
	UWorld* TargetWorld = nullptr;
	if (WorldContextObject)
	{
		TargetWorld = WorldContextObject->GetWorld();
	}

	if (!TargetWorld)
	{
		UE_LOGF(LogLevelSnapshots, Warning, "Snapshot taken with no valid World set");
		return nullptr;
	}

	return SnapshotEditor::TakeLevelSnapshotAndSaveToDisk(TargetWorld, FileName, FolderPath, Description, bShouldCreateUniqueFileName, false);
}

void ULevelSnapshotsEditorFunctionLibrary::TakeAndSaveLevelSnapshotEditorWorld(const FString& FileName, const FString& FolderPath, const FString& Description)
{
	UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
	if (EditorWorld)
	{
		TakeLevelSnapshotAndSaveToDisk(EditorWorld, FileName, FolderPath, Description);
	}
	else
	{
		UE_LOGF(LogLevelSnapshots, Warning, "Could not find valid Editor World.");
	}
}

void ULevelSnapshotsEditorFunctionLibrary::GenerateThumbnailForSnapshotAsset(ULevelSnapshot* Snapshot)
{
	SnapshotEditor::GenerateThumbnailForSnapshotAsset(Snapshot);
}
