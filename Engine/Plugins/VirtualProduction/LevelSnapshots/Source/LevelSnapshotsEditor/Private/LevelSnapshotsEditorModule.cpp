// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorModule.h"

#include "Data/Filters/NegatableFilter.h"
#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshotsEditorCommands.h"
#include "LevelSnapshotsEditorStyle.h"
#include "LevelSnapshotsEditorSettings.h"
#include "LevelSnapshotsLog.h"
#include "PropertyEditorModule.h"
#include "Views/SLevelSnapshotsEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "Util/TakeSnapshotUtil.h"
#include "ToolMenus.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "FLevelSnapshotsEditorModule"

namespace LevelSnapshotsEditor
{
	const FName LevelSnapshotsTabName("LevelSnapshots");
}

FLevelSnapshotsEditorModule& FLevelSnapshotsEditorModule::Get()
{
	return FModuleManager::GetModuleChecked<FLevelSnapshotsEditorModule>("LevelSnapshotsEditor");
}

void FLevelSnapshotsEditorModule::OpenLevelSnapshotsSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Plugins", "Level Snapshots");
}

void FLevelSnapshotsEditorModule::StartupModule()
{
	FLevelSnapshotsEditorStyle::Initialize();
	FLevelSnapshotsEditorCommands::Register();

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));

	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
	{		
		RegisterTabSpawner();
		RegisterEditorToolbar();

		ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
		{
			DataMangementSettingsSectionPtr = SettingsModule.RegisterSettings("Project", "Plugins", "Level Snapshots Editor",
				NSLOCTEXT("LevelSnapshots", "LevelSnapshotsEditorSettingsCategoryDisplayName", "Level Snapshots Editor"),
				NSLOCTEXT("LevelSnapshots", "LevelSnapshotsEditorSettingsDescription", "Configure the Level Snapshots Editor settings"),
				GetMutableDefault<ULevelSnapshotsEditorSettings>()
				);
		}
	});
}

void FLevelSnapshotsEditorModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FLevelSnapshotsEditorStyle::Shutdown();

	if (UObjectInitialized())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(UNegatableFilter::StaticClass()->GetFName());

		UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "LevelSnapshots");
	}

	UnregisterTabSpawner();
	FLevelSnapshotsEditorCommands::Unregister();
	
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	{
		SettingsModule.UnregisterSettings("Project", "Plugins", "Level Snapshots Editor");
	}
}

void FLevelSnapshotsEditorModule::RegisterTabSpawner()
{
	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(LevelSnapshotsEditor::LevelSnapshotsTabName, FOnSpawnTab::CreateRaw(this, &FLevelSnapshotsEditorModule::SpawnLevelSnapshotsTab))
			.SetDisplayName(NSLOCTEXT("LevelSnapshots", "LevelSnapshotsTabTitle", "Level Snapshots"))
			.SetTooltipText(NSLOCTEXT("LevelSnapshots", "LevelSnapshotsTooltipText", "Open the Level Snapshots tab"))
			.SetIcon(FSlateIcon(FLevelSnapshotsEditorStyle::GetStyleSetName(), "LevelSnapshots.ToolbarButton", "LevelSnapshots.ToolbarButton.Small")
			);
	FGlobalTabmanager::Get()->RegisterDefaultTabWindowSize(LevelSnapshotsEditor::LevelSnapshotsTabName, FVector2D(1240, 600));
	
	TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory());
}

void FLevelSnapshotsEditorModule::UnregisterTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterDefaultTabWindowSize(LevelSnapshotsEditor::LevelSnapshotsTabName);
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(LevelSnapshotsEditor::LevelSnapshotsTabName);
}

TSharedRef<SDockTab> FLevelSnapshotsEditorModule::SpawnLevelSnapshotsTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(NomadTab);

	TSharedRef<SLevelSnapshotsEditor> SnapshotsEditor = SNew(SLevelSnapshotsEditor, AllocateTransientPreset(), DockTab, SpawnTabArgs.GetOwnerWindow());
	WeakSnapshotEditor = SnapshotsEditor;
	DockTab->SetContent(SnapshotsEditor);

	return DockTab;
}

ULevelSnapshotsEditorData* FLevelSnapshotsEditorModule::AllocateTransientPreset()
{
	ULevelSnapshotsEditorData* ExistingPreset = FindObject<ULevelSnapshotsEditorData>(nullptr, TEXT("/Temp/LevelSnapshots/PendingSnapshots.PendingSnapshots"));
	if (ExistingPreset)
	{
		return ExistingPreset;
	}

	// If for whatever reason we're in a transaction context, we do not want the creation of this object to be tracked. For example, multiuser should not transact this object to other clients.
	UE_CLOGF(GUndo != nullptr || (GEditor && GEditor->IsTransactionActive()), LogLevelSnapshots, Error, "We shouldn't be transacting right now. Investigate.");
	TGuardValue<ITransaction*> ClearUndo(GUndo, nullptr);
	
	UPackage* NewPackage = CreatePackage(TEXT("/Temp/LevelSnapshots/PendingSnapshots"));
	NewPackage->SetFlags(RF_Transient);
	NewPackage->AddToRoot();

	return NewObject<ULevelSnapshotsEditorData>(NewPackage, TEXT("PendingSnapshots"), RF_Transient | RF_Transactional | RF_Standalone);
}

void FLevelSnapshotsEditorModule::OpenLevelSnapshotsDialogWithAssetSelected(const FAssetData& InAssetData)
{
	OpenSnapshotsEditor();
	if (WeakSnapshotEditor.IsValid())
	{
		WeakSnapshotEditor.Pin()->OpenLevelSnapshotsDialogWithAssetSelected(InAssetData);
	}
}

void FLevelSnapshotsEditorModule::OpenSnapshotsEditor()
{
	FGlobalTabmanager::Get()->TryInvokeTab(LevelSnapshotsEditor::LevelSnapshotsTabName);
}

void FLevelSnapshotsEditorModule::RegisterEditorToolbar()
{
	if (IsRunningGame())
	{
		return;
	}
	
	MapEditorToolbarActions();

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
	FToolMenuSection& Section = Menu->FindOrAddSection("LevelSnapshots");

	FToolMenuEntry LevelSnapshotsButtonEntry = FToolMenuEntry::InitToolBarButton(
		"TakeSnapshotAction",
		FUIAction(FExecuteAction::CreateStatic(&SnapshotEditor::TakeSnapshotWithOptionalForm),
			FCanExecuteAction::CreateLambda([this](){ return true; })),
		NSLOCTEXT("LevelSnapshots", "LevelSnapshots", "Level Snapshots"), // Set Text under image
		NSLOCTEXT("LevelSnapshots", "LevelSnapshotsToolbarButtonTooltip", "Take snapshot with optional form"), //  Set tooltip
		FSlateIcon(FLevelSnapshotsEditorStyle::GetStyleSetName(), "LevelSnapshots.ToolbarButton", "LevelSnapshots.ToolbarButton.Small") // Set image
	);
	LevelSnapshotsButtonEntry.SetCommandList(EditorToolbarButtonCommandList);

	 FToolMenuEntry LevelSnapshotsComboEntry = FToolMenuEntry::InitComboButton(
		"LevelSnapshotsMenu",
		FUIAction(),
		FOnGetContent::CreateRaw(this, &FLevelSnapshotsEditorModule::FillEditorToolbarComboButtonMenuOptions, EditorToolbarButtonCommandList),
		NSLOCTEXT("LevelSnapshots", "LevelSnapshotsOptions_Label", "Level Snapshots Options"), // Set text seen when the Level Editor Toolbar is truncated and the flyout is clicked
		NSLOCTEXT("LevelSnapshots", "LevelSnapshotsToolbarComboButtonTooltip", "Open Level Snapshots Options"), //  Set tooltip
		FSlateIcon(),
		true //bInSimpleComboBox
	);

	Section.AddEntry(LevelSnapshotsButtonEntry);
	Section.AddEntry(LevelSnapshotsComboEntry);
}

void FLevelSnapshotsEditorModule::MapEditorToolbarActions()
{
	EditorToolbarButtonCommandList = MakeShared<FUICommandList>();

	EditorToolbarButtonCommandList->MapAction(
		FLevelSnapshotsEditorCommands::Get().UseCreationFormToggle,
		FUIAction(
			FExecuteAction::CreateStatic(&FLevelSnapshotsEditorModule::ToggleUseCreationForm),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FLevelSnapshotsEditorModule::GetUseCreationForm)
		)
	);

	EditorToolbarButtonCommandList->MapAction(
		FLevelSnapshotsEditorCommands::Get().OpenLevelSnapshotsEditorToolbarButton,
		FExecuteAction::CreateRaw(this, &FLevelSnapshotsEditorModule::OpenSnapshotsEditor)
	);

	EditorToolbarButtonCommandList->MapAction(
		FLevelSnapshotsEditorCommands::Get().LevelSnapshotsSettings,
		FExecuteAction::CreateStatic(&FLevelSnapshotsEditorModule::OpenLevelSnapshotsSettings)
	);
}

TSharedRef<SWidget> FLevelSnapshotsEditorModule::FillEditorToolbarComboButtonMenuOptions(TSharedPtr<class FUICommandList> Commands)
{
	// Create FMenuBuilder instance for the commands we created
	FMenuBuilder MenuBuilder(true, Commands);

	// Then use it to add entries to the submenu of the combo button
	MenuBuilder.BeginSection("Creation", NSLOCTEXT("LevelSnapshots", "Creation", "Creation"));
	MenuBuilder.AddMenuEntry(FLevelSnapshotsEditorCommands::Get().UseCreationFormToggle);
	MenuBuilder.EndSection();
	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddMenuEntry(FLevelSnapshotsEditorCommands::Get().OpenLevelSnapshotsEditorToolbarButton);
	MenuBuilder.AddMenuEntry(FLevelSnapshotsEditorCommands::Get().LevelSnapshotsSettings);

	// Create the widget so it can be attached to the combo button
	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLevelSnapshotsEditorModule, LevelSnapshotsEditor)
