// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISettingsSection.h"
#include "LevelSnapshotsEditorSettings.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWidget.h"
#include "Widgets/Docking/SDockTab.h"

class ULevelSnapshotsSettings;
class ULevelSnapshotsEditorSettings;
class FToolBarBuilder;
class SLevelSnapshotsEditor;
class ULevelSnapshotsEditorData;

class FLevelSnapshotsEditorModule : public IModuleInterface
{
public:

	static FLevelSnapshotsEditorModule& Get();
	static void OpenLevelSnapshotsSettings();

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	static bool GetUseCreationForm() { return ULevelSnapshotsEditorSettings::Get()->bUseCreationForm; }
	static void SetUseCreationForm(bool bInUseCreationForm) { ULevelSnapshotsEditorSettings::Get()->bUseCreationForm = bInUseCreationForm; }
	static void ToggleUseCreationForm() { SetUseCreationForm(!GetUseCreationForm()); }

	void OpenLevelSnapshotsDialogWithAssetSelected(const FAssetData& InAssetData);
	void OpenSnapshotsEditor();

private:

	void RegisterTabSpawner();
	void UnregisterTabSpawner();

	TSharedRef<SDockTab> SpawnLevelSnapshotsTab(const FSpawnTabArgs& SpawnTabArgs);
	ULevelSnapshotsEditorData* AllocateTransientPreset();
	
	void RegisterEditorToolbar();
	void MapEditorToolbarActions();
	TSharedRef<SWidget> FillEditorToolbarComboButtonMenuOptions(TSharedPtr<class FUICommandList> Commands);

	
	/** Command list (for combo button sub menu options) */
	TSharedPtr<FUICommandList> EditorToolbarButtonCommandList;

	/** Lives for as long as the UI is open. */
	TWeakPtr<SLevelSnapshotsEditor> WeakSnapshotEditor;

	TSharedPtr<ISettingsSection> DataMangementSettingsSectionPtr;
};
