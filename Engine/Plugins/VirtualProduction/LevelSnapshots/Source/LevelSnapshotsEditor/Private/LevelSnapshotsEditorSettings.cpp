// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorSettings.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#include "Application/SlateApplicationBase.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UObject/Package.h"

ULevelSnapshotsEditorSettings* ULevelSnapshotsEditorSettings::Get()
{
	return GetMutableDefault<ULevelSnapshotsEditorSettings>();
}

ULevelSnapshotsEditorSettings::ULevelSnapshotsEditorSettings()
{
	float DPIScale = 1.0f;
	if (FSlateApplicationBase::IsInitialized())
	{
		const FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
		DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WorkAreaRect.Left, WorkAreaRect.Top);
	}

	const FVector2D DefaultClientSize = FVector2D(400.f, 400.f);
	PreferredCreationFormWindowWidth = DefaultClientSize.X * DPIScale;
	PreferredCreationFormWindowHeight = DefaultClientSize.Y * DPIScale;
}

FVector2D ULevelSnapshotsEditorSettings::GetLastCreationWindowSize() const
{
	return FVector2D(PreferredCreationFormWindowWidth, PreferredCreationFormWindowHeight);
}

void ULevelSnapshotsEditorSettings::SetLastCreationWindowSize(const FVector2D InLastSize)
{
	PreferredCreationFormWindowWidth = InLastSize.X;
	PreferredCreationFormWindowHeight = InLastSize.Y;
}

void ULevelSnapshotsEditorSettings::SanitizePathInline(FString& InPath, const bool bSkipForwardSlash)
{
	FString IllegalChars = FPaths::GetInvalidFileSystemChars().ReplaceEscapedCharWithChar() + " .";

	// In some cases we want to allow forward slashes in a path so that the end user can define a folder structure
	if (bSkipForwardSlash && IllegalChars.Contains("/"))
	{
		IllegalChars.ReplaceInline(TEXT("/"), TEXT(""));
	}

	for (int32 CharIndex = 0; CharIndex < IllegalChars.Len(); CharIndex++)
	{
		FString Char = FString().AppendChar(IllegalChars[CharIndex]);

		InPath.ReplaceInline(*Char, TEXT(""));
	}
}

void ULevelSnapshotsEditorSettings::SanitizeAllProjectSettingsPaths(const bool bSkipForwardSlash)
{
	FString Dummy;
	const FString PackagePath = FPaths::Combine(RootLevelSnapshotSaveDir.Path, FGuid::NewGuid().ToString());
	// If you enter a path that's not in /Game/ nor in any plugin, this returns false.
	if (!FPackageName::TryConvertLongPackageNameToFilename(PackagePath, Dummy, FPackageName::GetAssetPackageExtension()))
	{
		RootLevelSnapshotSaveDir.Path = TEXT("/Game/LevelSnapshots"); 
	}
	
	SanitizePathInline(RootLevelSnapshotSaveDir.Path, bSkipForwardSlash);
	SanitizePathInline(LevelSnapshotSaveDir, bSkipForwardSlash);
	SanitizePathInline(DefaultLevelSnapshotName, bSkipForwardSlash);
}

namespace UE::LevelSnapshots::Editor
{
	static FFormatNamedArguments GetFormatNamedArguments(const FString& InWorldName)
	{
		FNumberFormattingOptions IntOptions;
		IntOptions.MinimumIntegralDigits = 2;

		const FDateTime& LocalNow = FDateTime::Now();

		FFormatNamedArguments FormatArguments;
		FormatArguments.Add("map", FText::FromString(InWorldName));
		FormatArguments.Add("user", FText::FromString(FPlatformProcess::UserName()));
		FormatArguments.Add("year", FText::FromString(FString::FromInt(LocalNow.GetYear())));
		FormatArguments.Add("month", FText::AsNumber(LocalNow.GetMonth(), &IntOptions));
		FormatArguments.Add("day", FText::AsNumber(LocalNow.GetDay(), &IntOptions));
		FormatArguments.Add("date", FText::Format(FText::FromString("{0}-{1}-{2}"), FormatArguments["year"], FormatArguments["month"], FormatArguments["day"]));
		FormatArguments.Add("time",
			FText::Format(
				FText::FromString("{0}-{1}-{2}"),
				FText::AsNumber(LocalNow.GetHour(), &IntOptions), FText::AsNumber(LocalNow.GetMinute(), &IntOptions), FText::AsNumber(LocalNow.GetSecond(), &IntOptions)));

		return FormatArguments;
	}
}

FText ULevelSnapshotsEditorSettings::ParseLevelSnapshotsTokensInText(const FText& InTextToParse, const FString& InWorldName)
{
	const FFormatNamedArguments FormatArguments = UE::LevelSnapshots::Editor::GetFormatNamedArguments(InWorldName);
	return FText::Format(InTextToParse, FormatArguments);
}

#if WITH_EDITOR
void ULevelSnapshotsEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);

	SanitizeAllProjectSettingsPaths(true);
	SaveConfig();
}
#endif
