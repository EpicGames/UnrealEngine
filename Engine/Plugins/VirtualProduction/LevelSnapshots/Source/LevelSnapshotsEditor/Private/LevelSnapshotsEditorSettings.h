// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/Object.h"

#include "LevelSnapshotsEditorSettings.generated.h"

UCLASS(config = Engine, defaultconfig)
class LEVELSNAPSHOTSEDITOR_API ULevelSnapshotsEditorSettings : public UObject
{
	GENERATED_BODY()
public:

	static ULevelSnapshotsEditorSettings* Get();
	ULevelSnapshotsEditorSettings();

	FVector2D GetLastCreationWindowSize() const;
	/* Setting the Window Size through code will not save the size to the config. To make sure it's saved, call SaveConfig(). */
	void SetLastCreationWindowSize(const FVector2D InLastSize);
	
	/** Removes forbidden file path characters (e.g. /?:&\*"<>|%#@^. ) from project settings path strings. Optionally the forward slash can be kept so that the user can define a file structure. */
	void SanitizeAllProjectSettingsPaths(const bool bSkipForwardSlash);
	/** Removes forbidden file path characters (e.g. /?:&\*"<>|%#@^ ) */
	static void SanitizePathInline(FString& InPath, const bool bSkipForwardSlash);

	UFUNCTION(BlueprintCallable, Category = "Level Snapshots")
	static FText ParseLevelSnapshotsTokensInText(const FText& InTextToParse, const FString& InWorldName);

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	/** The base directory in which all snapshots will be saved. LevelSnapshotSaveDir specifies in which subdirectory snapshots are saved. */
	UPROPERTY(config, EditAnywhere, Category = "Data", meta = (ContentDir))
	FDirectoryPath RootLevelSnapshotSaveDir { TEXT("/Game/LevelSnapshots") };

	/** The format to use for the resulting filename. Extension will be added automatically. Any tokens of the form {token} will be replaced with the corresponding value:
	 * {map}		- The name of the captured map or level
	 * {user}		- The current OS user account name
	 * {year}       - The current year
	 * {month}      - The current month
	 * {day}        - The current day
	 * {date}       - The current date from the local computer in the format of {year}-{month}-{day}
	 * {time}       - The current time from the local computer in the format of hours-minutes-seconds
	 */
	UPROPERTY(config, EditAnywhere, Category = "Data")
	FString LevelSnapshotSaveDir = TEXT("{map}/{year}-{month}-{day}");

	/** The format to use for the resulting filename. Extension will be added automatically. Any tokens of the form {token} will be replaced with the corresponding value:
	 * {map}		- The name of the captured map or level
	 * {user}		- The current OS user account name
	 * {year}       - The current year
	 * {month}      - The current month
	 * {day}        - The current day
	 * {date}       - The current date from the local computer in the format of {year}-{month}-{day}
	 * {time}       - The current time from the local computer in the format of hours-minutes-seconds
	 */
	UPROPERTY(config, EditAnywhere, Category = "Data")
	FString DefaultLevelSnapshotName = TEXT("{map}_{user}_{time}");
	
	UPROPERTY(config, EditAnywhere, Category = "Editor", meta = (ConfigRestartRequired = true))
	bool bEnableLevelSnapshotsToolbarButton = true;

	UPROPERTY(config, EditAnywhere, Category = "Editor")
	bool bUseCreationForm = true;

	/* If true, clicking on an actor group under 'Modified Actors' will select the actor in the scene. The previous selection will be deselected. */
	UPROPERTY(config, EditAnywhere, Category = "Editor")
	bool bClickActorGroupToSelectActorInScene;

	UPROPERTY(config, EditAnywhere, Category = "Editor")
	float PreferredCreationFormWindowWidth;

	UPROPERTY(config, EditAnywhere, Category = "Editor")
	float PreferredCreationFormWindowHeight;
};
