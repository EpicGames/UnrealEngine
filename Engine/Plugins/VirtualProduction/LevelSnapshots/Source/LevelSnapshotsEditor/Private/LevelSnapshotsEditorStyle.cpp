// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"

TSharedPtr<FSlateStyleSet> FLevelSnapshotsEditorStyle::StyleInstance = nullptr;

void FLevelSnapshotsEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FLevelSnapshotsEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FLevelSnapshotsEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("LevelSnapshotsEditor"));
	return StyleSetName;
}

const FLinearColor& FLevelSnapshotsEditorStyle::GetColor(FName PropertyName, const ANSICHAR* Specifier)
{
	return StyleInstance->GetColor(PropertyName, Specifier);
}


const FSlateBrush* FLevelSnapshotsEditorStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return StyleInstance->GetBrush(PropertyName, Specifier);
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(Style->RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

const FVector2D Icon64x64(64.f, 64.f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);

TSharedRef< FSlateStyleSet > FLevelSnapshotsEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>("LevelSnapshotsEditor");

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("LevelSnapshots"));
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		Style->SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	}

	// Toolbar
	Style->Set("LevelSnapshots.ToolbarButton", new IMAGE_BRUSH_SVG("Icons/Icon", Icon20x20));
	Style->Set("LevelSnapshots.ToolbarButton.Small", new IMAGE_BRUSH_SVG("Icons/Icon", Icon16x16));
	Style->Set("ClassIcon.LevelSnapshotsEditorData", new IMAGE_BRUSH_SVG("Icons/Icon", Icon16x16));

	// Button Icons
	Style->Set( "LevelSnapshotsEditor.Add", new IMAGE_BRUSH_SVG("Starship/MainToolbar/ToolBadgePlus", Icon20x20));

	// Border Brushes
	Style->Set("LevelSnapshotsEditor.GroupBorder", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f));
	Style->Set("LevelSnapshotsEditor.BrightBorder", new FSlateColorBrush(FColor(112, 112, 112, 100)));
	Style->Set("LevelSnapshotsEditor.FilterSelected", new BOX_BRUSH("Common/Filter_Selected", FMargin(18.0f / 64.0f)));

	// Border colors for Results view
	Style->Set("LevelSnapshotsEditor.HeaderRowBorder", new FSlateColorBrush(FColor::Black));
	Style->Set("LevelSnapshotsEditor.ActorGroupBorder", new BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.0f / 16.0f)));
	Style->Set("LevelSnapshotsEditor.DefaultBorder", new FSlateColorBrush(FColor(0, 0, 0, 0)));
	Style->Set("LevelSnapshotsEditor.IgnoreFilterBorder", new BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.0f / 16.0f)));

	// Buttons
	FButtonStyle RemoveFilterButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton");
	RemoveFilterButtonStyle.Normal = FSlateNoResource();
	RemoveFilterButtonStyle.NormalPadding = FMargin(0, 1.5f);
	RemoveFilterButtonStyle.PressedPadding = FMargin(0, 1.5f);
	Style->Set("LevelSnapshotsEditor.RemoveFilterButton", RemoveFilterButtonStyle);

	FTextBlockStyle ButtonTextStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ContentBrowser.TopBar.Font");
	FLinearColor ButtonTextColor = ButtonTextStyle.ColorAndOpacity.GetSpecifiedColor();
	ButtonTextColor.A /= 2;
	ButtonTextStyle.ColorAndOpacity = ButtonTextColor;
	ButtonTextStyle.ShadowColorAndOpacity.A /= 2;
	Style->Set("LevelSnapshotsEditor.Button.TextStyle", ButtonTextStyle);

	FTextBlockStyle AndTextStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Graph.CompactNode.Title");
	FTextBlockStyle OrTextStyle = AndTextStyle;
	FSlateFontInfo AndFont = FAppStyle::Get().GetFontStyle("BoldCondensed"); AndFont.Size = 16;
	FSlateFontInfo OrFont = FAppStyle::Get().GetFontStyle("BoldCondensed"); OrFont.Size = 18;
	AndTextStyle.SetFont(AndFont);
	OrTextStyle.SetFont(OrFont);
	Style->Set("LevelSnapshotsEditor.FilterRow.And", AndTextStyle);
	Style->Set("LevelSnapshotsEditor.FilterRow.Or", OrTextStyle);
	
	return Style;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
#undef IMAGE_BRUSH_SVG

void FLevelSnapshotsEditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FLevelSnapshotsEditorStyle::Get()
{
	return *StyleInstance;
}
