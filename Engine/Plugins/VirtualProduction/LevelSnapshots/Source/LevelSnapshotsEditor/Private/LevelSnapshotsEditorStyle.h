// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FLevelSnapshotsEditorStyle
{
public:

	static void Initialize();

	static void Shutdown();

	static void ReloadTextures();

	static const ISlateStyle& Get();

	static FName GetStyleSetName();

	static const FLinearColor& GetColor(FName PropertyName, const ANSICHAR* Specifier = nullptr);

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = nullptr);

	template< typename WidgetStyleType >
	static const WidgetStyleType& GetWidgetStyle(FName PropertyName, const ANSICHAR* Specifier = nullptr)
	{
		return StyleInstance->GetWidgetStyle<WidgetStyleType>(PropertyName, Specifier);
	}

private:
	static TSharedRef<FSlateStyleSet> Create();

private:
	static TSharedPtr<FSlateStyleSet> StyleInstance;
};
