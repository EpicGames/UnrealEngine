// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LevelSnapshotsEditorCustomWidgetGenerator.h"

#include "LevelSnapshotsLog.h"
#include "Filtering/PropertySelection.h"
#include "Views/Results/LevelSnapshotsEditorResultsRow.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void LevelSnapshotsEditorCustomWidgetGenerator::CreateRowsForPropertiesNotHandledByPropertyRowGenerator(
	TFieldPath<FProperty> InFieldPath,
	TObjectPtr<UObject> InSnapshotObject,
	TObjectPtr<UObject> InWorldObject, 
	const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView,
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow)
{
	if (!ensure(InFieldPath->IsValidLowLevel() && IsValid(InSnapshotObject) && IsValid(InWorldObject)))
	{
		return;
	}
	
	const FProperty* Property = InFieldPath.Get();

	if (!ensureAlwaysMsgf(Property,
		TEXT("%hs: Property was not able to be retrieved from InFieldPath. Please ensure InFieldPath is valid before calling this function. InFieldPath: %s"),
		__FUNCTION__, *InFieldPath.ToString()))
	{
		return;
	}

	if (CastField<FStructProperty>(Property))
	{
		// Don't generate widgets for FStructProperty
		return;
	}

	TSharedPtr<SWidget> CustomSnapshotWidget;
	TSharedPtr<SWidget> CustomWorldWidget;

	const void* WorldPropertyValue = GetPropertyValueFromProperty(Property, InWorldObject);
	const void* SnapshotPropertyValue = GetPropertyValueFromProperty(Property, InSnapshotObject);

	if (InFieldPath->GetName() == "AttachParent")
	{
		auto WidgetTextEditLambda = [] (const FString& InWidgetText, const UObject* InPropertyObject)
		{
			if (const USceneComponent* AsSceneComponent = Cast<USceneComponent>(InPropertyObject))
			{
				if (const TObjectPtr<AActor> OwningActor = AsSceneComponent->GetOwner())
				{
					return FString::Printf(TEXT("%s.%s"), *OwningActor->GetActorLabel(), *InWidgetText);
				}
			}

			return LOCTEXT("LevelSnapshotsEditorResults_AttachParentInvalidText","No Attach Parent found.").ToString();
		};

		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			CustomSnapshotWidget = GenerateObjectPropertyWidget(ObjectProperty->GetObjectPropertyValue(SnapshotPropertyValue), WidgetTextEditLambda);
			CustomWorldWidget = GenerateObjectPropertyWidget(ObjectProperty->GetObjectPropertyValue(WorldPropertyValue), WidgetTextEditLambda);
		}
	}
	else
	{
		UE_LOGF(LogLevelSnapshots, Warning, "%s: Unsupported Property found named '%ls' with FieldPath: %ls", __FUNCTION__, *InFieldPath->GetAuthoredName(), *InFieldPath.ToString());
		
		CustomSnapshotWidget = DeterminePropertyTypeAndReturnWidget(Property, SnapshotPropertyValue, InSnapshotObject);
		CustomWorldWidget = DeterminePropertyTypeAndReturnWidget(Property, WorldPropertyValue, InWorldObject);
	}

	if (!CustomSnapshotWidget.IsValid() && !CustomWorldWidget.IsValid())
	{
		return;
	}

	// Create property
	const FLevelSnapshotsEditorResultsRowPtr NewProperty = 
		MakeShared<FLevelSnapshotsEditorResultsRow>(InFieldPath->GetDisplayNameText(), FLevelSnapshotsEditorResultsRow::SingleProperty, 
		InDirectParentRow.IsValid() ? InDirectParentRow.Pin()->GenerateChildWidgetCheckedStateBasedOnParent() : ECheckBoxState::Checked, 
		InResultsView, InDirectParentRow);

	NewProperty->InitPropertyRowWithCustomWidget(InDirectParentRow, InFieldPath.Get(), CustomSnapshotWidget, CustomWorldWidget);

	InDirectParentRow.Pin()->AddToChildRows(NewProperty);
}

void* LevelSnapshotsEditorCustomWidgetGenerator::GetPropertyValueFromProperty(const FProperty* InProperty, const TObjectPtr<UObject> InObject)
{
	void* PropertyValue = nullptr;
	
	if (InObject->IsValidLowLevel())
	{
		const TOptional<FLevelSnapshotPropertyChain> OutChain =
			FLevelSnapshotPropertyChain::FindPathToProperty(InProperty, InObject->GetClass(), true);

		// Owning object is an FProperty (usually for members of a collection)
		// I.E., Property is an item in array/set/map/struct, so OwnerProperty is the collection property)
		if (OutChain.IsSet() && OutChain.GetValue().GetNumProperties() > 0)
		{
			PropertyValue = InObject;

			for (int32 ChainItr = 0; ChainItr < OutChain->GetNumProperties(); ChainItr++)
			{
				if (const FProperty* ChainProperty = OutChain->GetPropertyFromRoot(ChainItr))
				{
					PropertyValue = ChainProperty->ContainerPtrToValuePtr<void>(PropertyValue);
					
					continue;
				}
			
				UE_LOGF(LogLevelSnapshots, Error, "%s: Property named %ls: PropertyChain with length of %i returned but FProperty at index %i from root is null.",
					__FUNCTION__, *InProperty->GetName(), OutChain->GetNumProperties(), ChainItr);
				PropertyValue = nullptr;
			}
		}
		else if (InProperty->GetOwner<UClass>()) // Owning object is something else, make sure it has UClass
		{
			PropertyValue = InProperty->ContainerPtrToValuePtr<void>(InObject);
		}
		else
		{
			const TObjectPtr<UObject> Owner = InProperty->GetOwner<UObject>();
			const FString OwnerName = Owner ? Owner->GetName() : "None";
			UE_LOGF(LogLevelSnapshots, Warning,
				"%s: Property named %ls with owner named '%ls' does not have an owning FProperty or an owning UClass.",
				__FUNCTION__, *InProperty->GetName(), *OwnerName);
		}
	}

	return PropertyValue;
}

TSharedPtr<SWidget> LevelSnapshotsEditorCustomWidgetGenerator::GenerateGenericPropertyWidget(
	const FProperty* InProperty, const void* InPropertyValue, const TObjectPtr<UObject> InObject,
	const TFunction<FString(const FString&, const UObject*)> InWidgetTextEditLambda)
{
	// We should have found a value ptr in the methods above, but we ensure in case of a missed scenario
	if (ensure(InPropertyValue && InObject))
	{
		FString ValueText;
		InProperty->ExportTextItem_Direct(ValueText, InPropertyValue, 0, InObject, PPF_None);

		if (InWidgetTextEditLambda)
		{
			ValueText = InWidgetTextEditLambda(ValueText, InObject);
		}

		return SNew(STextBlock)
			.Text(FText::FromString(ValueText))
			.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
			.ToolTipText(FText::FromString(ValueText));
	}

	return nullptr;
}

TSharedPtr<SWidget> LevelSnapshotsEditorCustomWidgetGenerator::GenerateObjectPropertyWidget(
	const TObjectPtr<UObject> InObject, const TFunction<FString(const FString&, const UObject*)> InWidgetTextEditLambda)
{
	if (InObject)
	{
		FString WidgetText = InObject->GetName();

		if (InWidgetTextEditLambda)
		{
			WidgetText = InWidgetTextEditLambda(WidgetText, InObject);
		}

		return MakeComboBoxWithSelection(WidgetText, FText::FromString(WidgetText));
	}

	return nullptr;
}

TSharedPtr<SWidget> LevelSnapshotsEditorCustomWidgetGenerator::DeterminePropertyTypeAndReturnWidget(
	const FProperty* InProperty, const void* InPropertyValue, const TObjectPtr<UObject> InObject)
{
	if (!ensure(InPropertyValue))
	{
		return nullptr;
	}
	
	// Then let's get the class name
	const FText ToolTipText = FText::FromString(InProperty->GetClass()->GetName());

	// Then we'll iterate over relevant property types to see what kind of widget we should create
	if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
	{
		if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(InProperty))
		{
			const float Value = FloatProperty->GetFloatingPointPropertyValue(InPropertyValue);
			return SNew(SNumericEntryBox<float>)
				.ToolTipText(ToolTipText)
				.Value(Value)
				.AllowSpin(false)
				.MinSliderValue(Value)
				.MaxSliderValue(Value);
		}
		else if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(InProperty))
		{
			const double Value = DoubleProperty->GetFloatingPointPropertyValue(InPropertyValue);
			return SNew(SNumericEntryBox<double>)
				.ToolTipText(ToolTipText)
				.Value(Value)
				.AllowSpin(false)
				.MinSliderValue(Value)
				.MaxSliderValue(Value);
		}
		else // Not a float or double? Then some kind of integer (byte, int8, int16, int32, int64, uint8, uint16 ...)
		{
			const int64 Value = NumericProperty->GetUnsignedIntPropertyValue(InPropertyValue);
			return SNew(SNumericEntryBox<int64>)
				.ToolTipText(ToolTipText)
				.Value(Value)
				.AllowSpin(false)
				.MinSliderValue(Value)
				.MaxSliderValue(Value);
		}
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		return SNew(SCheckBox)
				.IsChecked(BoolProperty->GetPropertyValue(InPropertyValue))
				.ToolTipText(ToolTipText);
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty))
	{
		return GenerateObjectPropertyWidget(ObjectProperty->GetObjectPropertyValue(InPropertyValue));
	}
	else
	{
		// If the property is not supported, use this as a fallback
		return GenerateGenericPropertyWidget(InProperty, InPropertyValue, InObject);
	}
}

TSharedRef<SWidget> LevelSnapshotsEditorCustomWidgetGenerator::MakeComboBoxWithSelection(const FString& InString, const FText& InToolTipText)
{
	return SNew(SComboBox<TSharedPtr<FString>>)
		   [
			   SNew(STextBlock)
				.ToolTipText(InToolTipText)
				.Text(FText::FromString(InString))
				.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
		   ];
}

#undef LOCTEXT_NAMESPACE
