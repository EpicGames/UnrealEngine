// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/FieldPath.h"
#include "Widgets/SWidget.h"

struct FLevelSnapshotsEditorResultsRow;

class SLevelSnapshotsEditorResults;

namespace LevelSnapshotsEditorCustomWidgetGenerator
{
	const TArray<FName> IgnoredClassNames =
		{
			"PostProcessVolume"
		};

	/** When properties do not have widgets generated by the PRG we need to create widgets for them on a case-by-case basis. */
	void CreateRowsForPropertiesNotHandledByPropertyRowGenerator(
		TFieldPath<FProperty> InFieldPath,
		TObjectPtr<UObject> InSnapshotObject,
		TObjectPtr<UObject> InWorldObject, 
		const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView,
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow);

	void* GetPropertyValueFromProperty(const FProperty* InProperty, const TObjectPtr<UObject> InObject);

	/** Used for properties which are not specifically handled in another method. Returns an STextBlock widget with the value exported to text. */
	TSharedPtr<SWidget> GenerateGenericPropertyWidget(
		const FProperty* InProperty, const void* InPropertyValue, const TObjectPtr<UObject> InObject,
		const TFunction<FString(const FString&, const UObject*)> InWidgetTextEditLambda = nullptr);

	TSharedPtr<SWidget> GenerateObjectPropertyWidget(
		const TObjectPtr<UObject> InObject, const TFunction<FString(const FString&, const UObject*)> InWidgetTextEditLambda = nullptr);

	TSharedPtr<SWidget> DeterminePropertyTypeAndReturnWidget(const FProperty* InProperty, const void* InPropertyValue, const TObjectPtr<UObject> InObject);

	TSharedRef<SWidget> MakeComboBoxWithSelection(const FString& InString, const FText& InToolTipText);
};
