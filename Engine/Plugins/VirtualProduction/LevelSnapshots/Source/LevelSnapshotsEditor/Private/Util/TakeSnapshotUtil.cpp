// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/TakeSnapshotUtil.h"

#include "Components/Viewport.h"
#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshot.h"
#include "LevelSnapshotsEditorModule.h"
#include "LevelSnapshotsEditorSettings.h"
#include "LevelSnapshotsEditorStyle.h"
#include "LevelSnapshotsLog.h"
#include "Widgets/SLevelSnapshotsEditorCreationForm.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Editor.h"
#include "Engine/World.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAssetTools.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "ObjectTools.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Engine/GameEngine.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotEditorLibrary"

namespace UE::LevelSnapshots::Editor
{
	TAutoConsoleVariable<bool> CVarSaveAsync(TEXT("LevelSnapshots.SnapshotFormUsesSaveAsync"), true, TEXT("Set whether the Take Snapshot Form, opened by pressing the toolbar icon, saves snapshots using the SAVE_Async flag."));
	TAutoConsoleVariable<bool> CVarIgnorePIEWorld(TEXT("LevelSnapshots.IgnorePIEWorld"), true, TEXT("Set whether the Take Snapshot will ignore the PIE world and use the editor world context.  If this is false, then the PIE world will be used to grab a snapshot."));
	
	UWorld* GetWorld(bool bIgnorePIE)
	{
		// Get the current world.
		if (GIsEditor)
		{
			FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
			if (!bIgnorePIE && PIEWorldContext)
			{
				return PIEWorldContext->World();
			}
			return GEditor->GetEditorWorldContext().World();
		}
		else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			return GameEngine->GetGameWorld();
		}
		return nullptr;
	}

	static void HandleFormReply(const FText& InDescription, const FString& SnapshotName)
	{
		UWorld* World = GetWorld(CVarIgnorePIEWorld.GetValueOnAnyThread());
		if (!ensure(World))
		{
			return;
		}
		
		ULevelSnapshotsEditorSettings* DataManagementSettings = ULevelSnapshotsEditorSettings::Get();
		DataManagementSettings->SanitizeAllProjectSettingsPaths(true);

		const FText NewSnapshotDir = ULevelSnapshotsEditorSettings::ParseLevelSnapshotsTokensInText(
			FText::FromString(FPaths::Combine(DataManagementSettings->RootLevelSnapshotSaveDir.Path, DataManagementSettings->LevelSnapshotSaveDir)),
			World->GetName()
			);
		const FText NewSnapshotName = ULevelSnapshotsEditorSettings::ParseLevelSnapshotsTokensInText(
				FText::FromString(SnapshotName),
				World->GetName()
				);

		const FString ValidatedName = FPaths::MakeValidFileName(NewSnapshotName.ToString());
		const bool bSaveAsync = CVarSaveAsync.GetValueOnAnyThread();
		SnapshotEditor::TakeLevelSnapshotAndSaveToDisk(World, ValidatedName, NewSnapshotDir.ToString(), InDescription.ToString(), false, bSaveAsync);
	}
	
	static void DestroySnapshot(ULevelSnapshot* SnapshotAsset)
	{
		SnapshotAsset->ClearFlags(RF_Public | RF_Standalone);
		SnapshotAsset->Rename(nullptr, GetTransientPackage());
		SnapshotAsset->MarkAsGarbage();
		SnapshotAsset->RemoveFromRoot();
	}
}

void SnapshotEditor::TakeSnapshotWithOptionalForm()
{
	if (ULevelSnapshotsEditorSettings::Get()->bUseCreationForm)
	{
		SLevelSnapshotsEditorCreationForm::MakeAndShowCreationWindow(FCloseCreationFormDelegate::CreateStatic(UE::LevelSnapshots::Editor::HandleFormReply));
	}
	else
	{
		UE::LevelSnapshots::Editor::HandleFormReply(FText::GetEmpty(), ULevelSnapshotsEditorSettings::Get()->DefaultLevelSnapshotName);
	}
}

ULevelSnapshot* SnapshotEditor::TakeLevelSnapshotAndSaveToDisk(UWorld* World, const FString& FileName, const FString& FolderPath, const FString& Description, bool bShouldCreateUniqueFileName, bool bSaveAsync)
{
	SCOPED_SNAPSHOT_EDITOR_TRACE(CreateSnapshotPackage);
	
	FString AssetName = FileName;
    FString PackageName = FPaths::Combine(FolderPath, AssetName);
    if (bShouldCreateUniqueFileName)
    {
    	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
    	AssetTools.CreateUniqueAssetName(PackageName, TEXT(""), PackageName, AssetName);
    }

	FString PackageFileName;
	FPackageName::TryConvertLongPackageNameToFilename(PackageName, PackageFileName, FPackageName::GetAssetPackageExtension());
	
    bool bProceedWithSave = FPaths::ValidatePath(PackageFileName);
    if (bProceedWithSave && !bShouldCreateUniqueFileName)
    {
    	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
    	FAssetData ExistingAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath((PackageName + "." + FileName)));
    	if (ExistingAsset.IsValid() && ExistingAsset.AssetClassPath == ULevelSnapshot::StaticClass()->GetClassPathName())
    	{
    		EAppReturnType::Type ShouldReplace = FMessageDialog::Open( EAppMsgType::YesNo, FText::Format(LOCTEXT("ReplaceAssetMessage", "{0} already exists. Do you want to replace it?"), FText::FromString(FileName)) );
    		bProceedWithSave = (ShouldReplace == EAppReturnType::Yes);
    	}
    	else if (ExistingAsset.IsValid())
    	{
    		UE_LOGF(LogLevelSnapshots, Error, "Asset already exists and is not a snapshot. Aborting.")
			bProceedWithSave = false;
    	}
    }

	FText ErrorReason;
	if (!FPackageName::IsValidLongPackageName(PackageName, true, &ErrorReason))
	{
		UE_LOGF(LogLevelSnapshots, Error, "Invalid package name. Error: %ls", *ErrorReason.ToString());
		bProceedWithSave = false;
	}

    if (bProceedWithSave)
    {
    	// Show notification
    	FNotificationInfo Notification(NSLOCTEXT("LevelSnapshots", "NotificationFormatText_CreatingSnapshot", "Creating Level Snapshot"));
    	Notification.Image = FLevelSnapshotsEditorStyle::GetBrush(TEXT("LevelSnapshots.ToolbarButton"));
    	Notification.bUseThrobber = true;
    	Notification.bUseSuccessFailIcons = true;
    	Notification.ExpireDuration = 2.f;
    	Notification.bFireAndForget = false;
    	
    	TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Notification);
    	NotificationItem->SetCompletionState(SNotificationItem::CS_Pending);
    	ON_SCOPE_EXIT
    	{
    		NotificationItem->ExpireAndFadeout();
    	};

    	UPackage* SavePackage = CreatePackage(*PackageName);
    	const EObjectFlags AssetFlags = RF_Public | RF_Standalone;
    	ULevelSnapshot* SnapshotAsset = NewObject<ULevelSnapshot>(SavePackage, *AssetName, AssetFlags, nullptr);
    	if (!ensure(SavePackage && SnapshotAsset))
    	{
    		return nullptr;
    	}
    	
    	SnapshotAsset->SetSnapshotName(*FileName);
    	SnapshotAsset->SetSnapshotDescription(Description);
        const bool bSuccessful = SnapshotAsset->SnapshotWorld(World);
        if (!bSuccessful)
        {
            // Package and snapshot need to be destroyed again
            UE::LevelSnapshots::Editor::DestroySnapshot(SnapshotAsset);
            ObjectTools::DeleteObjectsUnchecked({ SavePackage });
            
            NotificationItem->SetText(
                NSLOCTEXT("LevelSnapshots", "NotificationFormatText_NoSnapshotTaken", "No snapshot was taken")
                );
            NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
            return nullptr;
        }
    	
    	SnapshotAsset->MarkPackageDirty();
    	FAssetRegistryModule::AssetCreated(SnapshotAsset);
    	GenerateThumbnailForSnapshotAsset(SnapshotAsset);

    	bool bSavingSuccessful;
    	{
    		SCOPED_SNAPSHOT_EDITOR_TRACE(SaveSnapshotPackage);
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Public | RF_Standalone;
			SaveArgs.Error = GWarn;
			SaveArgs.bWarnOfLongFilename = false;
			SaveArgs.SaveFlags = bSaveAsync ? SAVE_Async | SAVE_NoError : SAVE_None;
			bSavingSuccessful = UPackage::SavePackage(SavePackage, SnapshotAsset, *PackageFileName, SaveArgs);
    	}
    	
    	// Notify the user of the outcome
    	if (bSavingSuccessful)
    	{
    		NotificationItem->SetText(
    			FText::Format(
    				NSLOCTEXT("LevelSnapshots", "NotificationFormatText_CreateSnapshotSuccess", "Successfully created Level Snapshot \"{0}\""), FText::FromString(FileName)));
    		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
    	}
    	else
    	{
    		NotificationItem->SetText(
    			FText::Format(
    				NSLOCTEXT("LevelSnapshots", "NotificationFormatText_CreateSnapshotFailure", "Failed to create Level Snapshot \"{0}\". Check the file name."), FText::FromString(FileName)));
    		NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
    	}

    	return SnapshotAsset;
    }

    return nullptr;
}

void SnapshotEditor::GenerateThumbnailForSnapshotAsset(ULevelSnapshot* Snapshot)
{
	if (!ensure(Snapshot))
	{
		return;
	}
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	TArray<FAssetData> SnapshotAssetData;
	AssetRegistry.GetAssetsByPackageName(FName(*Snapshot->GetPackage()->GetPathName()), SnapshotAssetData);
	
	if (GEditor && ensureMsgf(SnapshotAssetData.Num(), TEXT("Failed to find asset data for asset we just saved. Investigate!")))
	{
		// Copied from FAssetFileContextMenu::ExecuteCaptureThumbnail
		FViewport* Viewport = GEditor->GetActiveViewport();

		if (ensure(GCurrentLevelEditingViewportClient) && ensure(Viewport))
		{
			//have to re-render the requested viewport
			FLevelEditorViewportClient* OldViewportClient = GCurrentLevelEditingViewportClient;
			//remove selection box around client during render
			GCurrentLevelEditingViewportClient = NULL;
			Viewport->Draw();

			AssetViewUtils::CaptureThumbnailFromViewport(Viewport, SnapshotAssetData);

			//redraw viewport to have the yellow highlight again
			GCurrentLevelEditingViewportClient = OldViewportClient;
			Viewport->Draw();
		}
	}
}


namespace UE::LevelSnapshots::Private
{

FAutoConsoleCommandWithWorldAndArgs TakeLevelSnaphot(
	TEXT("LevelSnapshots.TakeLevelSnapshot"),
	TEXT("Take a new level snapshot of the current world.")
	TEXT("Arg1 - Name of the level snapshot asset.\n")
	TEXT("Arg2 - Folder path to store the snapshot, e.g. /Game/LevelSnapshots. \n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			if (Args.Num() == 2)
			{
				FString Description(TEXT("Snapshot taken from console command."));
				
				SnapshotEditor::TakeLevelSnapshotAndSaveToDisk(World, Args[0], Args[1], Description, true, false);
			}
			else
			{
				UE_LOGF(LogLevelSnapshots, Warning, "Incorrect number of arguments for the snapshot command.");				
			}
		}
	)
);

} // namespace UE::LevelSnapshots::Private

#undef LOCTEXT_NAMESPACE
