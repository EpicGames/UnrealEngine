// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ULevelSnapshot;

namespace SnapshotEditor
{
	/**
	 * Get the world to use for taking a level snapshot.
	 * @param bIgnorePIE  if true and we are in a PIE world, we ignore the PIE world and use the editor world context.
	 *
	 * @return The current UWorld.
	 */
	UWorld* GetWorld(bool bIgnorePIE);

	/**
	 * Shows the window for taking snapshots, if enabled by the user settings. Otherwise just takes a snapshot.
	 */
	void TakeSnapshotWithOptionalForm();

	/**
	 * @param World The world to take snapshot in
	 * @param FileName Name of saved snapshot asset
	 * @param FolderPath Where to save the asset
	 * @param Description The value of the description field of the snapshot
	 * @param bShouldCreateUniqueFileName Whether to use FileName as base to generate a unique file name
	 * @param bSaveAsync Whether to save the asset async. Saving sync will block user and this can take several minutes.
	 */
	ULevelSnapshot* TakeLevelSnapshotAndSaveToDisk(UWorld* World, const FString& FileName, const FString& FolderPath, const FString& Description, bool bShouldCreateUniqueFileName = true, bool bSaveAsync = false);

	/**
	 * Captures a screenshot from the editor camera and assigns it to the snapshot.
	 */
	void GenerateThumbnailForSnapshotAsset(ULevelSnapshot* Snapshot);
}
