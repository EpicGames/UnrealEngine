// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Filter/SLevelSnapshotsEditorFilters.h"

#include "Customizations/NegatableFilterDetailsCustomization.h"
#include "Data/FilteredResults.h"
#include "Data/Filters/LevelSnapshotsFilterPreset.h"
#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshotsEditorStyle.h"
#include "Widgets/Filter/SFavoriteFilterList.h"
#include "Widgets/SLevelSnapshotsEditorFilterRow.h"
#include "Widgets/Filter/SSaveAndLoadFilters.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

namespace
{
	constexpr float ErrorMessageDisplayTimeInSeconds = 5.f;
}

class SCustomSplitter : public SSplitter
{
public:

	bool IsResizing() const
	{
		return bIsResizing;
	}
};

class SLevelSnapshotsEditorFilterRowGroup : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorFilterRowGroup)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SLevelSnapshotsEditorFilters>& InOwnerPanel, UConjunctionFilter* InManagedFilter)
	{
		ULevelSnapshotsEditorData* EditorData = InOwnerPanel->GetEditorData();
		const bool bIsFirstAndRow = EditorData->GetUserDefinedFilters()->GetChildren().Find(InManagedFilter) == 0;
		const bool bShouldShowOrTextInFrontOfRow = !bIsFirstAndRow;
		
		const TWeakPtr<SLevelSnapshotsEditorFilters> WeakOwner = InOwnerPanel;
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(3.0f, 2.0f))
			[
				SNew(SLevelSnapshotsEditorFilterRow, EditorData, InManagedFilter, bShouldShowOrTextInFrontOfRow)
					.OnClickRemoveRow_Lambda([WeakOwner, InManagedFilter](auto)
					{
						if (ensure(WeakOwner.IsValid()))
						{
							WeakOwner.Pin()->RemoveFilter(InManagedFilter);
						}
					})
			]
		];
	}
};

SLevelSnapshotsEditorFilters::~SLevelSnapshotsEditorFilters()
{
	if (ULevelSnapshotsEditorData* Data = GetEditorData())
	{
		Data->OnUserDefinedFiltersChanged.Remove(OnUserDefinedFiltersChangedHandle);
		Data->OnEditedFilterChanged.Remove(OnEditedFilterChangedHandle);
		Data->GetUserDefinedFilters()->OnFilterModified.Remove(OnFilterModifiedHandle);
	}
}

void SLevelSnapshotsEditorFilters::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* EditorData)
{	
	
	EditorDataPtr = EditorData;

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = nullptr;
	DetailsViewArgs.bSearchInitialKeyFocus = false;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	
	FilterDetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	FilterDetailsView->RegisterInstancedCustomPropertyLayout(UNegatableFilter::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FNegatableFilterDetailsCustomization>(); })
		);


	TSharedPtr<SScrollBar> ScrollBar;
	const TArray<UConjunctionFilter*>* AndFilters = &GetEditorData()->GetUserDefinedFilters()->GetChildren();
	ChildSlot
	[
		SAssignNew(DetailsSplitter, SCustomSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(1.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		.Orientation(Orient_Vertical)

		// Filter config
		+ SSplitter::Slot()
		[
			SNew(SVerticalBox)

			// Refresh results & Save and load buttons
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 5.f)
			[
				SNew(SHorizontalBox)

				// Refresh button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 2.f, 0.f)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("UpdateResults", "Refresh Results"))
					.IsEnabled_Lambda([this]() 
					{
						return EditorDataPtr->IsFilterDirty() && EditorDataPtr->GetActiveSnapshot();
					})
					.ToolTipText_Lambda([this]() 
					{
						return EditorDataPtr->IsFilterDirty() && EditorDataPtr->GetActiveSnapshot() ? 
							FText(LOCTEXT("RefreshResultsTooltip_DirtyState", "Filter changes have been detected, please refresh filters.")) :
						FText(LOCTEXT("RefreshResultsTooltip_CleanState", "Results are up to date, no refresh required."));
					})
					.OnClicked(this, &SLevelSnapshotsEditorFilters::OnClickUpdateResultsView)
				]

				// Save and load
				+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
				[
					SNew(SSaveAndLoadFilters, GetEditorData())
				]
			]
			
            // Favorite filters
            + SVerticalBox::Slot()
            .AutoHeight()
            .Padding(5.f, 0.f)
            .HAlign(HAlign_Fill)
            [
				SAssignNew(FavoriteList, SFavoriteFilterList, EditorData->GetFavoriteFilters(), GetEditorData())
            ]

            // Rows 
            +SVerticalBox::Slot()
				.Padding(FMargin(5.f, 10.f, 5.f, 0.f))
            [
				SNew(SScrollBox)
                .Orientation(Orient_Vertical)

                + SScrollBox::Slot()
                [
                	SNew(SVerticalBox)
                	
                    // Rows
                    + SVerticalBox::Slot()
                    .AutoHeight()
                    [
                        SAssignNew(FilterRowsList, SVerticalBox)
                    ]

                    // Add button
                    + SVerticalBox::Slot()
                    .Padding(0.f, 10.f)
                    .AutoHeight()
                    [
                        SNew(SButton)
                        .Text(LOCTEXT("AddFilterGroup", "Filter Group"))
                        .OnClicked(this, &SLevelSnapshotsEditorFilters::AddFilterClick)
						.HAlign(HAlign_Center)
                    ]
                ]
            ]
			
			
		]
		
		// Filter details panel
		+ SSplitter::Slot()
		[
			SNew(SScrollBox)
			.Orientation(Orient_Vertical)
			
			+SScrollBox::Slot()
			[
				FilterDetailsView.ToSharedRef()
			]
		]
	];

	// Set Delegates
	OnUserDefinedFiltersChangedHandle = GetEditorData()->OnUserDefinedFiltersChanged.AddLambda([this](ULevelSnapshotsFilterPreset* NewFilter, ULevelSnapshotsFilterPreset* OldFilter)
	{
		if (IsValid(OldFilter) && OnFilterModifiedHandle.IsValid())
		{
			OldFilter->OnFilterModified.Remove(OnFilterModifiedHandle);
		}
		OnFilterModifiedHandle = NewFilter->OnFilterModified.AddRaw(this, &SLevelSnapshotsEditorFilters::OnFilterModified);
		
		GetEditorData()->SetEditedFilter(nullptr);
		RefreshGroups();
	});

	OnEditedFilterChangedHandle = GetEditorData()->OnEditedFilterChanged.AddLambda([this](const TOptional<UNegatableFilter*>& ActiveFilter)
	{
		FilterDetailsView->SetObject(ActiveFilter.IsSet() ? ActiveFilter.GetValue() : nullptr);
	});
	OnFilterModifiedHandle = GetEditorData()->GetUserDefinedFilters()->OnFilterModified.AddRaw(this, &SLevelSnapshotsEditorFilters::OnFilterModified);
	
	RefreshGroups();
}

ULevelSnapshotsEditorData* SLevelSnapshotsEditorFilters::GetEditorData() const
{
	return EditorDataPtr.IsValid() ? EditorDataPtr.Get() : nullptr;
}

const TSharedPtr<IDetailsView>& SLevelSnapshotsEditorFilters::GetFilterDetailsView() const
{
	return FilterDetailsView;
}

bool SLevelSnapshotsEditorFilters::IsResizingDetailsView() const
{
	return DetailsSplitter->IsResizing();
}

void SLevelSnapshotsEditorFilters::RemoveFilter(UConjunctionFilter* FilterToRemove)
{
	ULevelSnapshotsFilterPreset* UserDefinedFilters = GetEditorData()->GetUserDefinedFilters();
	UserDefinedFilters->RemoveConjunction(FilterToRemove);
}

void SLevelSnapshotsEditorFilters::OnFilterModified(EFilterChangeType FilterChangeType)
{
	if (FilterChangeType != EFilterChangeType::FilterPropertyModified)
	{
		RefreshGroups();
	}
}

FReply SLevelSnapshotsEditorFilters::OnClickUpdateResultsView()
{
	ULevelSnapshotsEditorData* EditorData = GetEditorData();
	if (!ensure(EditorData))
	{
		return FReply::Handled();
	}
	
	if (EditorData->GetActiveSnapshot())
	{
		EditorData->OnRefreshResults.Broadcast();
	}
	else
	{
		FNotificationInfo Info(LOCTEXT("SelectSnapshotFirst", "Select a snapshot first."));
		Info.ExpireDuration = ErrorMessageDisplayTimeInSeconds;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	return FReply::Handled();
}

void SLevelSnapshotsEditorFilters::RefreshGroups()
{
	if (!ensure(FilterRowsList.IsValid()))
	{
		return;
	}
	
	FilterRowsList->ClearChildren();
	const TArray<UConjunctionFilter*>& AllAndRows = GetEditorData()->GetUserDefinedFilters()->GetChildren(); 
	for (UConjunctionFilter* Row : AllAndRows)
	{
		FilterRowsList->AddSlot()
			.AutoHeight()
		[
			SNew(SLevelSnapshotsEditorFilterRowGroup, SharedThis<SLevelSnapshotsEditorFilters>(this), Row)
		];
	}
}

FReply SLevelSnapshotsEditorFilters::AddFilterClick()
{
	if (ULevelSnapshotsEditorData* EditorData = GetEditorData())
	{
		// This will trigger the filter's OnFilterModified, which will refresh this UI
		EditorData->GetUserDefinedFilters()->CreateChild();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
