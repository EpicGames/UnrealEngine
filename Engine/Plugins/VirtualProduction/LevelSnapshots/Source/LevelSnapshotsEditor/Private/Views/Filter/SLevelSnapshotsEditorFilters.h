// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailsView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

enum class EFilterChangeType : uint8;
class FLevelSnapshotsEditorFilters;
class SFavoriteFilterList;
class SLevelSnapshotsEditorFilterRowGroup;
class SCustomSplitter;
class ULevelSnapshotsEditorData;
class ULevelSnapshotFilter;
class UConjunctionFilter;

enum class EFilterBehavior : uint8;

/* Contents of filters tab */
class SLevelSnapshotsEditorFilters : public SCompoundWidget
{
public:

	~SLevelSnapshotsEditorFilters();

	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorFilters)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* EditorData);

	ULevelSnapshotsEditorData* GetEditorData() const;
	const TSharedPtr<IDetailsView>& GetFilterDetailsView() const;
	bool IsResizingDetailsView() const;
	
	void RemoveFilter(UConjunctionFilter* FilterToRemove);

private:

	void OnFilterModified(EFilterChangeType FilterChangeType);
	FReply OnClickUpdateResultsView();
	
	/** Generate the groups using the preset's layout data. */
	void RefreshGroups();

	FReply AddFilterClick();

	FDelegateHandle OnUserDefinedFiltersChangedHandle;
	FDelegateHandle OnEditedFilterChangedHandle;
	FDelegateHandle OnFilterModifiedHandle;
	
	TSharedPtr<SFavoriteFilterList> FavoriteList;
	TSharedPtr<SVerticalBox> FilterRowsList;

	/** Filter input details view */
	TSharedPtr<IDetailsView> FilterDetailsView;
	/* Splits filters and details panel */
	TSharedPtr<SCustomSplitter> DetailsSplitter;

	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorDataPtr;
};
