// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Input/SLevelSnapshotsEditorContextPicker.h"

#include "Data/LevelSnapshotsEditorData.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Engine/World.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

SLevelSnapshotsEditorContextPicker::~SLevelSnapshotsEditorContextPicker()
{
	FEditorDelegates::OnMapOpened.Remove(OnMapOpenedDelegateHandle);
	OnSelectWorldContextEvent.Unbind();
}

void SLevelSnapshotsEditorContextPicker::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData)
{
	check(InEditorData);
	SelectedWorldPath = InArgs._SelectedWorldPath.Get();
	OnSelectWorldContextEvent = InArgs._OnSelectWorldContext;
	DataPtr = InEditorData;

	OnMapOpenedDelegateHandle = FEditorDelegates::OnMapOpened.AddLambda([this](const FString& FileName, bool bAsTemplate)
    {
		SetSelectedWorld(
			DataPtr->GetEditorWorld()
			);
		
        if (PickerButtonTextBlock.IsValid())
        {
            PickerButtonTextBlock.Get()->SetText(GetWorldPickerMenuButtonText(
                SelectedWorldPath, FName(SelectedWorldPath.GetAssetName()))
                );
        }
    });
	
	ChildSlot
    .Padding(0.0f)
    [
        SNew(SComboButton)
        .ContentPadding(0)
        .ForegroundColor(FSlateColor::UseForeground())
        .ButtonStyle(FAppStyle::Get(), "ToggleButton")
        .OnGetMenuContent(this, &SLevelSnapshotsEditorContextPicker::BuildWorldPickerMenu)
        .ToolTipText(LOCTEXT("WorldPickerButtonTooltip", "The world context whose Level Snapshots you want to view"))
        .ButtonContent()
        [
            SNew(SHorizontalBox)

            + SHorizontalBox::Slot()
            .AutoWidth()
            [
                SNew(SImage)
                .Image(FAppStyle::GetBrush("SceneOutliner.World"))
            ]

            + SHorizontalBox::Slot()
            .Padding(3.f, 0.f)
            [
                SAssignNew(PickerButtonTextBlock, STextBlock)
                .Text(GetCurrentContextText())
            ]
        ]
    ];

	RegisterWorldPickerWithEditorDataClass();
}

UWorld* SLevelSnapshotsEditorContextPicker::GetSelectedWorld() const
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	return Cast<UWorld>(AssetRegistryModule.Get().GetAssetByObjectPath(SelectedWorldPath.GetWithoutSubPath()).GetAsset());
}

FSoftObjectPath SLevelSnapshotsEditorContextPicker::GetSelectedWorldSoftPath() const
{
	return SelectedWorldPath;
}

TSharedRef<SWidget> SLevelSnapshotsEditorContextPicker::BuildWorldPickerMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	// Get all worlds
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> WorldAssets;
	AssetRegistryModule.Get().GetAssetsByClass(UWorld::StaticClass()->GetClassPathName(), WorldAssets);

	MenuBuilder.BeginSection("Other Worlds", LOCTEXT("OtherWorldsHeader", "Other Worlds")); 
	for (const FAssetData& Asset : WorldAssets)
	{
		{
			if (Asset.IsValid())
			{
				MenuBuilder.AddMenuEntry(
					GetWorldPickerMenuButtonText(Asset.ToSoftObjectPath(), Asset.AssetName),
					LOCTEXT("World", "World"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(
							this, &SLevelSnapshotsEditorContextPicker::OnSetWorldContextSelection, Asset),
						FCanExecuteAction(),
						FIsActionChecked::CreateRaw(
							this, &SLevelSnapshotsEditorContextPicker::ShouldRadioButtonBeChecked, Asset.ToSoftObjectPath())
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SLevelSnapshotsEditorContextPicker::RegisterWorldPickerWithEditorDataClass()
{
	if (DataPtr.IsValid())
	{
		SelectedWorldPath = DataPtr->GetEditorWorld();
	}
}

FText SLevelSnapshotsEditorContextPicker::GetWorldDescription(const UWorld* World)
{
	FText PostFix;
	if (World->WorldType == EWorldType::PIE)
	{
		switch(World->GetNetMode())
		{
		case NM_Client:
			PostFix = FText::Format(LOCTEXT("ClientPostfixFormat", " (Client {0})"), FText::AsNumber(World->GetOutermost()->GetPIEInstanceID() - 1));
			break;
		case NM_DedicatedServer:
		case NM_ListenServer:
			PostFix = LOCTEXT("ServerPostfix", " (Server)");
			break;
		case NM_Standalone:
			PostFix = GEditor->bIsSimulatingInEditor ? LOCTEXT("SimulateInEditorPostfix", " (Simulate)") : LOCTEXT("PlayInEditorPostfix", " (PIE)");
			break;

		default:
			break;
		}
	}
	else if (World->WorldType == EWorldType::Editor)
	{
		PostFix = LOCTEXT("EditorPostfix", " (Editor)");
	}

	return FText::Format(LOCTEXT("WorldFormat", "{0}{1}"), FText::FromString(World->GetFName().GetPlainNameString()), PostFix);
}

FText SLevelSnapshotsEditorContextPicker::GetWorldPickerMenuButtonText(const FSoftObjectPath& AssetPath,
	const FName& AssetName) const
{
	if (const bool bDoesAssetPointToEditorWorld = AssetPath == FSoftObjectPath(DataPtr->GetEditorWorld()))
	{
		const FText EditorLabel = LOCTEXT("EditorLabel", "Editor");
		const FText FormattedEditorLabel = FText::Format(INVTEXT(" ({0})"), EditorLabel);

		return FText::Format(INVTEXT("{0}{1}"),
			FText::FromName(AssetName), (bDoesAssetPointToEditorWorld ? FormattedEditorLabel : FText::GetEmpty()));
	}
	else
	{
		return FText::Format(INVTEXT("{0}"), FText::FromName(AssetName));
	}
}

FText SLevelSnapshotsEditorContextPicker::GetCurrentContextText() const
{
	UObject* WorldObject = SelectedWorldPath.ResolveObject();
	UWorld* CurrentWorld = Cast<UWorld>(WorldObject);
	check(CurrentWorld);
	return GetWorldDescription(CurrentWorld);
}

const FSlateBrush* SLevelSnapshotsEditorContextPicker::GetBorderBrush(FSoftObjectPath WorldPath) const
{
	UObject* WorldObject = WorldPath.ResolveObject();
	UWorld* CurrentWorld = Cast<UWorld>(WorldObject);
	check(CurrentWorld);

	if (CurrentWorld->WorldType == EWorldType::PIE)
	{
		return GEditor->bIsSimulatingInEditor ? FAppStyle::GetBrush("LevelViewport.StartingSimulateBorder") : FAppStyle::GetBrush("LevelViewport.StartingPlayInEditorBorder");
	}
	else
	{
		return FAppStyle::GetBrush("LevelViewport.NoViewportBorder");
	}
}

void SLevelSnapshotsEditorContextPicker::OnSetWorldContextSelection(const FAssetData Asset)
{
	check(Asset.IsValid());

	SetSelectedWorld(Asset.ToSoftObjectPath());
	
	if (PickerButtonTextBlock.IsValid())
	{
		PickerButtonTextBlock.Get()->SetText(GetWorldPickerMenuButtonText(SelectedWorldPath, Asset.AssetName));
	}
}

void SLevelSnapshotsEditorContextPicker::SetSelectedWorld(const FSoftObjectPath& SelectedWorld)
{
	SelectedWorldPath = SelectedWorld;
	OnSelectWorldContextEvent.ExecuteIfBound(SelectedWorld);
	
	if (DataPtr.IsValid())
	{
		if (SelectedWorld != DataPtr->GetEditorWorld())
		{
			DataPtr->SetActiveSnapshot(nullptr);
		
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("UnsupportedOperation", "Your selected world is currently not loaded. In order to filter, you'll first have to load the selected world. Filtering operations will continue running on your last selected world."));
		}
	}
}

bool SLevelSnapshotsEditorContextPicker::ShouldRadioButtonBeChecked(const FSoftObjectPath InWorldSoftPath) const
{
	return SelectedWorldPath.IsAsset() && !SelectedWorldPath.IsNull() && SelectedWorldPath == InWorldSoftPath;
}

#undef LOCTEXT_NAMESPACE
