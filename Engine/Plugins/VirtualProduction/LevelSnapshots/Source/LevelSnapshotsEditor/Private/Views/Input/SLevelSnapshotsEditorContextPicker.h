// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class ULevelSnapshotsEditorData;
class FLevelSnapshotsEditorInput;
class SLevelSnapshotsEditorBrowser;
class SLevelSnapshotsEditorContextPicker;
class UWorld;

struct FAssetData;
struct FLevelSnapshotsEditorViewBuilder;

class SLevelSnapshotsEditorContextPicker : public SCompoundWidget
{
public:
	
	DECLARE_DELEGATE_OneParam(FOnSelectWorldContext, FSoftObjectPath);

	~SLevelSnapshotsEditorContextPicker();
	
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorContextPicker) {}
	/** Attribute for retrieving the current context */
	SLATE_ATTRIBUTE(FSoftObjectPath, SelectedWorldPath)
	/** Called when the user explicitly chooses a new context world. */
	SLATE_EVENT(FOnSelectWorldContext, OnSelectWorldContext)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData);

	UWorld* GetSelectedWorld() const;
	FSoftObjectPath GetSelectedWorldSoftPath() const;

private:
	
	static FText GetWorldDescription(const UWorld* World);

	TSharedRef<SWidget> BuildWorldPickerMenu();
	void RegisterWorldPickerWithEditorDataClass();


	FText GetWorldPickerMenuButtonText(const FSoftObjectPath& AssetPath, const FName& AssetName) const;
	FText GetCurrentContextText() const;
	const FSlateBrush* GetBorderBrush(FSoftObjectPath WorldPath) const;

	void OnSetWorldContextSelection(const FAssetData Asset);
	void SetSelectedWorld(const FSoftObjectPath& SelectedWorld);

	bool ShouldRadioButtonBeChecked(const FSoftObjectPath InWorldSoftPath) const;

	FOnSelectWorldContext OnSelectWorldContextEvent;

	TSharedPtr<STextBlock> PickerButtonTextBlock;
	TWeakObjectPtr<ULevelSnapshotsEditorData> DataPtr;

	// The selected radio button's world ref's soft path (for comparison)
	FSoftObjectPath SelectedWorldPath;

	FDelegateHandle OnMapOpenedDelegateHandle;
};
