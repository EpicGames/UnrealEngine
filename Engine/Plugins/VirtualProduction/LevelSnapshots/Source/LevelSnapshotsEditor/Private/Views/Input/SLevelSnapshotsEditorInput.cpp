// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Input/SLevelSnapshotsEditorInput.h"

#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshotsLog.h"
#include "SLevelSnapshotsEditorContextPicker.h"
#include "Widgets/SLevelSnapshotsEditorBrowser.h"

#include "Editor.h"
#include "Engine/World.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

SLevelSnapshotsEditorInput::~SLevelSnapshotsEditorInput()
{
	FEditorDelegates::OnMapOpened.Remove(OnMapOpenedDelegateHandle);
}

void SLevelSnapshotsEditorInput::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData)
{
	EditorData = InEditorData;

	OnMapOpenedDelegateHandle = FEditorDelegates::OnMapOpened.AddLambda([this] (const FString& InFileName, const bool bAsTemplate)
	{
		check(EditorData.IsValid());
		OverrideWorld(EditorData->GetEditorWorld());
	});

	ChildSlot
	[
		SAssignNew(EditorInputOuterVerticalBox, SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(EditorContextPickerPtr, SLevelSnapshotsEditorContextPicker, EditorData.Get())
			.SelectedWorldPath(EditorData->GetEditorWorld())
			.OnSelectWorldContext_Raw(this, &SLevelSnapshotsEditorInput::OverrideWorld)
		]
		
		+ SVerticalBox::Slot()
		[
			SAssignNew(EditorBrowserWidgetPtr, SLevelSnapshotsEditorBrowser, InEditorData)
				.OwningWorldPath(EditorData->GetEditorWorld())
		]
	];
}

void SLevelSnapshotsEditorInput::OpenLevelSnapshotsDialogWithAssetSelected(const FAssetData& InAssetData) const
{
	EditorBrowserWidgetPtr->SelectAsset(InAssetData);
}

void SLevelSnapshotsEditorInput::OverrideWorld(FSoftObjectPath InNewContextPath)
{
	// Replace the Browser widget with new world context if world and builder pointer valid
	if (!ensure(InNewContextPath.IsValid()))
	{
		UE_LOGF(LogLevelSnapshots, Error,
			"%s: Unable to rebuild Snapshot Browser; InNewContext or BuilderPtr are invalid.", __FUNCTION__);
		return;
	}
	
	if (ensure(EditorInputOuterVerticalBox && EditorData.IsValid()))
	{
		// Remove the Browser widget then add a new one into the same slot
		EditorInputOuterVerticalBox->RemoveSlot(EditorBrowserWidgetPtr.ToSharedRef());
		
		EditorInputOuterVerticalBox->AddSlot()
		[
			SAssignNew(EditorBrowserWidgetPtr, SLevelSnapshotsEditorBrowser, EditorData.Get())
			.OwningWorldPath(InNewContextPath)
		];
	}
}

#undef LOCTEXT_NAMESPACE
