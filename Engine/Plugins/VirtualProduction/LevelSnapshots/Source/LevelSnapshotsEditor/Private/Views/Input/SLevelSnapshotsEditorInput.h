// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FLevelSnapshotsEditorInput;
class ULevelSnapshotsEditorData;
class UWorld;
class SLevelSnapshotsEditorBrowser;
class SLevelSnapshotsEditorContextPicker;
class SVerticalBox;

struct FSnapshotEditorViewData;

class SLevelSnapshotsEditorInput : public SCompoundWidget
{
public:

	~SLevelSnapshotsEditorInput();
	
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorInput)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData);

	void OpenLevelSnapshotsDialogWithAssetSelected(const FAssetData& InAssetData) const;

private:
	
	void OverrideWorld(FSoftObjectPath InNewContextPath);

	FDelegateHandle OnMapOpenedDelegateHandle;
	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData;

	TSharedPtr<SVerticalBox> EditorInputOuterVerticalBox;
	TSharedPtr<SLevelSnapshotsEditorContextPicker> EditorContextPickerPtr;
	TSharedPtr<SLevelSnapshotsEditorBrowser> EditorBrowserWidgetPtr;

};