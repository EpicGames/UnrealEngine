// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSnapshotsEditorResultsHelpers.h"

#include "LevelSnapshotsLog.h"
#include "Misc/LevelSnapshotsEditorCustomWidgetGenerator.h"
#include "SLevelSnapshotsEditorResults.h"

#include "Algo/Find.h"
#include "BlueprintEditorSettings.h"
#include "IDetailPropertyRow.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

TArray<TFieldPath<FProperty>> FLevelSnapshotsEditorResultsHelpers::LoopOverProperties(
	const TWeakPtr<FRowGeneratorInfo>& InSnapshotRowGeneratorInfo, const TWeakPtr<FRowGeneratorInfo>& InWorldRowGeneratorInfo,
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter,
	const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView)
{
	TSharedPtr<FPropertyHandleHierarchy> SnapshotHandleHierarchy = nullptr;
	TSharedPtr<FPropertyHandleHierarchy> WorldHandleHierarchy = nullptr;

	if (InSnapshotRowGeneratorInfo.IsValid())
	{
		SnapshotHandleHierarchy = BuildPropertyHandleHierarchy(InSnapshotRowGeneratorInfo);
	}

	if (InWorldRowGeneratorInfo.IsValid())
	{
		WorldHandleHierarchy = BuildPropertyHandleHierarchy(InWorldRowGeneratorInfo);
	}

	TArray<TFieldPath<FProperty>> PropertyRowsGenerated;

	// We start with World Hierarchy because it's more likely that the user wants to update existing actors than add/delete snapshot ones
	if (WorldHandleHierarchy.IsValid())
	{
		// Don't bother with the first FPropertyHandleHierarchy because that's a dummy node to contain the rest
		for (int32 ChildIndex = 0; ChildIndex < WorldHandleHierarchy->DirectChildren.Num(); ChildIndex++)
		{
			const TSharedRef<FPropertyHandleHierarchy>& ChildHierarchy = WorldHandleHierarchy->DirectChildren[ChildIndex];

			LoopOverHandleHierarchiesAndCreateRowHierarchy(
					ObjectType_World, ChildHierarchy, InDirectParentRow,
					PropertySelectionMap, PropertiesThatPassFilter, PropertyRowsGenerated,
					InResultsView, SnapshotHandleHierarchy);
		}
	}

	return PropertyRowsGenerated;
}

void FLevelSnapshotsEditorResultsHelpers::LoopOverHandleHierarchiesAndCreateRowHierarchy(ELevelSnapshotsObjectType InputType,
	const TWeakPtr<FPropertyHandleHierarchy>& InputHierarchy,
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow,
	FPropertySelectionMap& PropertySelectionMap,
	const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter,
	TArray<TFieldPath<FProperty>>& PropertyRowsGenerated,
	const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView, 
	const TWeakPtr<FPropertyHandleHierarchy>& InHierarchyToSearchForCounterparts)
{
	if (!ensureAlwaysMsgf(InputType != ObjectType_None, 
		TEXT("%hs: InputType must NOT be ObjectType_None!"), __FUNCTION__))
	{
		return;
	}
	
	if (!ensureAlwaysMsgf(InputHierarchy.IsValid(), TEXT("%hs: InputHierarchy must be valid."), __FUNCTION__) || 
		!ensureAlwaysMsgf(InDirectParentRow.IsValid(), TEXT("%hs: InDirectParentRow must be valid."), __FUNCTION__))
	{
		return;
	}

	const bool bInputIsWorld = InputType == ELevelSnapshotsObjectType::ObjectType_World;
	
	const TSharedPtr<FPropertyHandleHierarchy> PinnedInputHierarchy = InputHierarchy.Pin();

	const TSharedPtr<IPropertyHandle> InputHandle = PinnedInputHierarchy->Handle;

	// No asserts for handle or property because PropertyRowGenerator assumes there will be a details layout pointer but this scenario doesn't create one
	// Asserting would cause needless debugging breaks every time a snapshot is chosen
	if (!InputHandle->IsValidHandle())
	{
		return;
	}

	FProperty* Property = InputHandle->GetProperty();
	if (!Property)
	{
		return;
	}

	const TFieldPath<FProperty> PropertyField(Property);

	UE_LOGF(LogLevelSnapshots, VeryVerbose, "PropertyField is %ls", *PropertyField.ToString());

	const FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType ParentRowType = InDirectParentRow.Pin()->GetRowType();
	const bool bIsParentRowContainer =
		(ParentRowType == FLevelSnapshotsEditorResultsRow::CollectionGroup ||
			ParentRowType == FLevelSnapshotsEditorResultsRow::StructGroup || 
			ParentRowType == FLevelSnapshotsEditorResultsRow::StructInSetOrArray ||
			ParentRowType == FLevelSnapshotsEditorResultsRow::StructInMap);

	// If the property is within a container, we need to see first if there are any valid children of the property before checking it against PropertiesThatPassFilter
	const bool bIsPropertyFilteredOut = !bIsParentRowContainer && !PropertiesThatPassFilter.Contains(PropertyField);
	if (bIsPropertyFilteredOut)
	{
		return;
	}
	
	bool bFoundCounterpart = false;
	TSharedPtr<FPropertyHandleHierarchy> PinnedCounterpartHierarchy;
	TSharedPtr<IPropertyHandle> CounterpartHandle;
	
	if (InHierarchyToSearchForCounterparts.IsValid())
	{
		const TWeakPtr<FPropertyHandleHierarchy> OutCounterpartHierarchy = FindCorrespondingHandle(
			PinnedInputHierarchy->PropertyChain.Get(FLevelSnapshotPropertyChain()), PinnedInputHierarchy->DisplayName,
			InHierarchyToSearchForCounterparts, bFoundCounterpart);

		if (bFoundCounterpart)
		{
			bFoundCounterpart = OutCounterpartHierarchy.IsValid();

			if (bFoundCounterpart)
			{
				PinnedCounterpartHierarchy = OutCounterpartHierarchy.Pin();
				
				CounterpartHandle = PinnedCounterpartHierarchy->Handle;
			}
		}
	}
	uint32 UnsignedInputChildHandleCount = 0;
	uint32 UnsignedCounterpartChildHandleCount = 0;
	InputHandle->GetNumChildren(UnsignedInputChildHandleCount);

	const int32 InputHierarchyChildCount = PinnedInputHierarchy->DirectChildren.Num();
	int32 CounterpartHierarchyChildCount = 0;

	if (bFoundCounterpart && PinnedCounterpartHierarchy.IsValid())
	{
		if (CounterpartHandle.IsValid())
		{
			CounterpartHandle->GetNumChildren(UnsignedCounterpartChildHandleCount);
		}
		
		CounterpartHierarchyChildCount = PinnedCounterpartHierarchy->DirectChildren.Num();
	}
	
	int32 InputChildHandleCount = UnsignedInputChildHandleCount;
	int32 CounterpartChildHandleCount = UnsignedCounterpartChildHandleCount;

	const int32 MaxHandleChildCount = FMath::Max(InputChildHandleCount, CounterpartChildHandleCount);
	int32 MaxHierarchyChildCount = FMath::Max(InputHierarchyChildCount, CounterpartHierarchyChildCount);
	
	const int32 MaxChildCount = FMath::Max(MaxHandleChildCount, MaxHierarchyChildCount);

	const FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType InRowType = 
		FLevelSnapshotsEditorResultsRow::DetermineRowTypeFromProperty(Property, InputHandle->IsCustomized(), MaxChildCount > 0);
	
	if (!ensure(InRowType != FLevelSnapshotsEditorResultsRow::None))
	{
		return;
	}

	bool bIsCounterpartValueSame = false;

	auto AreHandleValuesEqual = [](const TSharedPtr<IPropertyHandle> HandleA, const TSharedPtr<IPropertyHandle> HandleB)
	{
		if (!HandleA.IsValid() || !HandleB.IsValid())
		{
			return false;
		}
		
		FString ValueA;
		FString ValueB;

		HandleA->GetValueAsFormattedString(ValueA);
		HandleB->GetValueAsFormattedString(ValueB);

		return ValueA.Equals(ValueB);
	};
	
	if (bFoundCounterpart && CounterpartHandle.IsValid() &&
		(InRowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInMap || 
		InRowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInSetOrArray || 
		InRowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInStruct))
	{
		bIsCounterpartValueSame = AreHandleValuesEqual(InputHandle, CounterpartHandle);
	}

	if (!bIsCounterpartValueSame)
	{
		FLevelSnapshotsEditorResultsRowPtr NewRow;

		const FText DisplayName = InputHandle->GetPropertyDisplayName();
		
		if (InRowType == FLevelSnapshotsEditorResultsRow::SubObjectGroup)
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
					
			UObject* WorldSubobject = nullptr;
			UObject* SnapshotSubobject = nullptr;
			InputHandle->GetValue(bInputIsWorld ? WorldSubobject : SnapshotSubobject);

			if (CounterpartHandle.IsValid())
			{
				CounterpartHandle->GetValue(bInputIsWorld ? SnapshotSubobject : WorldSubobject);
			}

			if (!ensure(WorldSubobject))
			{
				return;
			}

			const FPropertySelection* PropertySelection = PropertySelectionMap.GetObjectSelection(WorldSubobject).GetPropertySelection();

			if (!PropertySelection)
			{
				return;
			}

			TWeakPtr<FLevelSnapshotsEditorResultsRow> NewObjectRow =
				BuildModifiedObjectRow(
					WorldSubobject, SnapshotSubobject, PropertyEditorModule,
					PropertySelectionMap,
					PropertySelection->GetSelectedLeafProperties(),
					InDirectParentRow, InResultsView,
					FText::Format(INVTEXT("{PropertyName} ({ObjectName})"),
						DisplayName, FText::FromString(WorldSubobject->GetName())));

			if (NewObjectRow.IsValid())
			{
				NewRow = NewObjectRow.Pin();

				PropertyRowsGenerated.Add(PropertyField);
			}
		}
		else
		{
			// When creating the child rows, we must consider that some nodes do not list children even if the handle has children.
			// In this case we'll create special handle hierarchies based on the handle's child handles
			if (MaxHierarchyChildCount == 0 && MaxHandleChildCount > 0)
			{					
				for (int32 ChildIndex = 0; ChildIndex < MaxHandleChildCount; ChildIndex++)
				{
					TSharedPtr<IPropertyHandle> NewInputChildHandle;
					TSharedPtr<IPropertyHandle> NewCounterpartChildHandle;

					if (InputChildHandleCount > ChildIndex)
					{
						NewInputChildHandle = InputHandle->GetChildHandle(ChildIndex);
					}
			
					if (CounterpartHandle.IsValid() && CounterpartChildHandleCount > ChildIndex)
					{
						NewCounterpartChildHandle = CounterpartHandle->GetChildHandle(ChildIndex);
					}

					if (NewInputChildHandle.IsValid() || NewCounterpartChildHandle.IsValid())
					{
						const bool bAreChildValuesTheSame = AreHandleValuesEqual(NewInputChildHandle, NewCounterpartChildHandle);

						if (!bAreChildValuesTheSame)
						{
							const TSharedRef<FPropertyHandleHierarchy> NewHierarchy =
								MakeShared<FPropertyHandleHierarchy>(nullptr, NewInputChildHandle, PinnedInputHierarchy->ContainingObject);

							if (NewHierarchy->IsValidHierarchy())
							{
								PinnedInputHierarchy->DirectChildren.Add(NewHierarchy);
							}

							if (PinnedCounterpartHierarchy.IsValid())
							{
								const TSharedRef<FPropertyHandleHierarchy> NewCounterpartHierarchy = 
									MakeShared<FPropertyHandleHierarchy>(nullptr, NewCounterpartChildHandle, PinnedCounterpartHierarchy->ContainingObject);

								if (NewCounterpartHierarchy->IsValidHierarchy())
								{
									PinnedCounterpartHierarchy->DirectChildren.Add(NewCounterpartHierarchy);
								}
							}
						}
					}
				}
			}

			const ECheckBoxState StartingCheckedState = (bIsPropertyFilteredOut || bIsCounterpartValueSame) ? 
				ECheckBoxState::Unchecked : InDirectParentRow.IsValid() ?
					InDirectParentRow.Pin()->GenerateChildWidgetCheckedStateBasedOnParent() : ECheckBoxState::Checked;

			// Create property
			NewRow = MakeShared<FLevelSnapshotsEditorResultsRow>(DisplayName, InRowType, StartingCheckedState, 
				InResultsView, InDirectParentRow);
			
			TSharedPtr<FPropertyHandleHierarchy> SnapshotHierarchy = 
				bInputIsWorld ? (PinnedCounterpartHierarchy.IsValid() ? PinnedCounterpartHierarchy : nullptr) : (PinnedInputHierarchy.IsValid() ? PinnedInputHierarchy : nullptr);

			TSharedPtr<FPropertyHandleHierarchy> WorldHierarchy = 
				bInputIsWorld ? (PinnedInputHierarchy.IsValid() ? PinnedInputHierarchy : nullptr) : (PinnedCounterpartHierarchy.IsValid() ? PinnedCounterpartHierarchy : nullptr);
		
			const TWeakPtr<FLevelSnapshotsEditorResultsRow>& ContainingObjectGroup = InDirectParentRow;

			NewRow->InitPropertyRow(ContainingObjectGroup, SnapshotHierarchy, WorldHierarchy, bIsCounterpartValueSame);

			// If either hierarchy has children, then makes rows for them
			MaxHierarchyChildCount = FMath::Max(PinnedInputHierarchy->DirectChildren.Num(), PinnedCounterpartHierarchy.IsValid() ? PinnedCounterpartHierarchy->DirectChildren.Num() : 0);

			for (int32 ChildIndex = 0; ChildIndex < MaxHierarchyChildCount; ChildIndex++)
			{
				TSharedPtr<FPropertyHandleHierarchy> ChildHierarchy;
	
				// Get the input Child first. If it doesn't exist at this index, check for the counterpart child.
				const bool bIsChildFromInputHierarchy = ChildIndex < PinnedInputHierarchy->DirectChildren.Num();

				if (bIsChildFromInputHierarchy)
				{
					ChildHierarchy = PinnedInputHierarchy->DirectChildren[ChildIndex];
				}
				else if (PinnedCounterpartHierarchy.IsValid() && ChildIndex < PinnedCounterpartHierarchy->DirectChildren.Num())
				{
					ChildHierarchy = PinnedCounterpartHierarchy->DirectChildren[ChildIndex];
				}

				if (ChildHierarchy.IsValid())
				{
					// There may be instances of collection property members that exist on the snapshot object but not the world object
					// For example, removing actor tags after taking a snapshot
					ELevelSnapshotsObjectType ChildObjectInputType = ObjectType_World;

					if (bInputIsWorld)
					{
						if (!bIsChildFromInputHierarchy)
						{
							ChildObjectInputType = ObjectType_Snapshot;
						}
					}
					else
					{
						if (bIsChildFromInputHierarchy)
						{
							ChildObjectInputType = ObjectType_Snapshot;
						}
					}

					const bool bShouldSearchCounterparts = SnapshotHierarchy.IsValid() && WorldHierarchy.IsValid();
		
					LoopOverHandleHierarchiesAndCreateRowHierarchy(
						ChildObjectInputType, ChildHierarchy, NewRow, PropertySelectionMap,
						PropertiesThatPassFilter, PropertyRowsGenerated, InResultsView,
						bShouldSearchCounterparts ? (bIsChildFromInputHierarchy ? InHierarchyToSearchForCounterparts : InputHierarchy) : nullptr);
				}
			}

			if (!ensure(NewRow.IsValid()))
			{
				return;
			}
		
			if (NewRow->DoesRowRepresentGroup() && NewRow->GetChildRows().Num() == 0)
			{
				if (!PropertiesThatPassFilter.Contains(PropertyField))
				{
					// No valid children, destroy group
					NewRow.Reset();
				}
			}
			else
			{
				InDirectParentRow.Pin()->AddToChildRows(NewRow);

				PropertyRowsGenerated.Add(PropertyField);
			}
		}
	}
}

TWeakPtr<FPropertyHandleHierarchy> FLevelSnapshotsEditorResultsHelpers::FindCorrespondingHandle(
	const FLevelSnapshotPropertyChain& InPropertyChain, const FText& InDisplayName, const TWeakPtr<FPropertyHandleHierarchy>& HierarchyToSearch, bool& bFoundMatch)
{
	if (!ensureMsgf(HierarchyToSearch.IsValid(), TEXT("FindCorrespondingHandle: HierarchyToSearch was not valid")))
	{
		return nullptr;
	}

	TWeakPtr<FPropertyHandleHierarchy> OutHierarchy = nullptr;

	for (TSharedRef<FPropertyHandleHierarchy> ChildHierarchy : HierarchyToSearch.Pin()->DirectChildren)
	{
		const bool bIsChainSame = ChildHierarchy->PropertyChain == InPropertyChain;
		const bool bIsNameSame = ChildHierarchy->DisplayName.EqualTo(InDisplayName);

		if (bIsChainSame && bIsNameSame)
		{
			OutHierarchy = ChildHierarchy;
			bFoundMatch = true;
			break;
		}

		if (bFoundMatch)
		{
			break;
		}
		else
		{
			if (ChildHierarchy->DirectChildren.Num() > 0)
			{
				OutHierarchy = FindCorrespondingHandle(InPropertyChain, InDisplayName, ChildHierarchy, bFoundMatch);
			}
		}
	}

	return OutHierarchy;
}

void FLevelSnapshotsEditorResultsHelpers::CreatePropertyHandleHierarchyChildrenRecursively(
	const TSharedRef<IDetailTreeNode>& InNode, const TWeakPtr<FPropertyHandleHierarchy>& InParentHierarchy, const TWeakObjectPtr<UObject> InContainingObject)
{
	if (!ensureMsgf(InParentHierarchy.IsValid(),
		TEXT("%hs: InParentHierarchy was not valid. Check to see that InParentHierarchy is valid before calling this method."), __FUNCTION__))
	{
		return;
	}

	TWeakPtr<FPropertyHandleHierarchy> HierarchyToPass = InParentHierarchy;

	const EDetailNodeType NodeType = InNode->GetNodeType();

	if (NodeType == EDetailNodeType::Item)
	{
		TSharedPtr<IPropertyHandle> Handle;

		// If the handle already exists then we should just go get it
		if (InNode->GetRow().IsValid() && InNode->GetRow()->GetPropertyHandle().IsValid())
		{
			Handle = InNode->GetRow()->GetPropertyHandle();
		}
		else // Otherwise let's try to create it
		{
			Handle = InNode->CreatePropertyHandle();
		}

		if (Handle.IsValid())
		{
			const TSharedRef<FPropertyHandleHierarchy> NewHierarchy = MakeShared<FPropertyHandleHierarchy>(InNode, Handle, InContainingObject);

			if (NewHierarchy->IsValidHierarchy())
			{
				HierarchyToPass = NewHierarchy;

				InParentHierarchy.Pin()->DirectChildren.Add(NewHierarchy);
			}
		}
	}

	TArray<TSharedRef<IDetailTreeNode>> NodeChildren;
	InNode->GetChildren(NodeChildren);

	for (const TSharedRef<IDetailTreeNode>& ChildNode : NodeChildren)
	{
		CreatePropertyHandleHierarchyChildrenRecursively(ChildNode, HierarchyToPass, InContainingObject);
	}
}

TSharedPtr<FPropertyHandleHierarchy> FLevelSnapshotsEditorResultsHelpers::BuildPropertyHandleHierarchy(const TWeakPtr<FRowGeneratorInfo>& InRowGenerator)
{
	check(InRowGenerator.IsValid());

	// Create a base hierarchy with dummy info and no handle
	TSharedRef<FPropertyHandleHierarchy> ReturnHierarchy = MakeShared<FPropertyHandleHierarchy>(nullptr, nullptr, nullptr);

	TWeakPtr<IPropertyRowGenerator> RowPtr = InRowGenerator.Pin()->GetGeneratorObject();

	if (RowPtr.IsValid())
	{
		TArray<TSharedRef<IDetailTreeNode>> Nodes = RowPtr.Pin()->GetRootTreeNodes();
		TArray<TWeakObjectPtr<UObject>> SelectedObjects = RowPtr.Pin()->GetSelectedObjects();

		if (SelectedObjects.Num())
		{
			for (const TSharedRef<IDetailTreeNode>& Node : Nodes)
			{
				CreatePropertyHandleHierarchyChildrenRecursively(Node, ReturnHierarchy, SelectedObjects[0]);
			}
		}
	}

	return ReturnHierarchy;
}

UObject* FLevelSnapshotsEditorResultsHelpers::FindCounterpartComponent(const UActorComponent* SubObjectToMatch, const TSet<UActorComponent*>& InCounterpartSubObjects)
{
	if (!ensure(SubObjectToMatch) || !ensure(InCounterpartSubObjects.Num() > 0))
	{
		return nullptr;
	}

	const TArray<UActorComponent*>::ElementType* FoundComponent = Algo::FindByPredicate(InCounterpartSubObjects, 
		[&SubObjectToMatch](UObject* ObjectInLoop)
		{
			const bool bIsSameClass = ObjectInLoop->IsA(SubObjectToMatch->GetClass());
			const bool bIsSameName = ObjectInLoop->GetName().Equals(SubObjectToMatch->GetName());
			return bIsSameClass && bIsSameName;
		});

	if (FoundComponent)
	{
		return *FoundComponent;
	}
	else
	{
		return nullptr;
	}
}

bool FLevelSnapshotsEditorResultsHelpers::IsInstancedActorComponentDesirableForComponentArray(
	UActorComponent* ActorComponent, const bool bHideConstructionScriptComponentsInDetailsView)
{
	const USceneComponent* AsSceneComponent = Cast<USceneComponent>(ActorComponent);
	const USceneComponent* ParentSceneComponent = AsSceneComponent != nullptr ? AsSceneComponent->GetAttachParent() : nullptr;

	// Exclude nested DSOs attached to BP-constructed instances, which are not mutable.
	return (ActorComponent != nullptr
		&& (!ActorComponent->IsVisualizationComponent())
		&& (ActorComponent->CreationMethod != EComponentCreationMethod::UserConstructionScript || !bHideConstructionScriptComponentsInDetailsView)
		&& (!ParentSceneComponent || !ParentSceneComponent->IsCreatedByConstructionScript() || !ActorComponent->HasAnyFlags(RF_DefaultSubObject)))
		&& (ActorComponent->CreationMethod != EComponentCreationMethod::Native || FComponentEditorUtils::GetPropertyForEditableNativeComponent(ActorComponent));
}

TSet<UActorComponent*> FLevelSnapshotsEditorResultsHelpers::FilterOutActorComponentsBasedOnEditorSettings(
	const TSet<UActorComponent*>& UnfilteredActorComponentList)
{
	const bool bHideConstructionScriptComponentsInDetailsView =
		GetDefault<UBlueprintEditorSettings>()->bHideConstructionScriptComponentsInDetailsView;

	// Filter the components by their visibility
	TSet<UActorComponent*> ComponentsToAdd;
	for (UActorComponent* ActorComp : UnfilteredActorComponentList)
	{
		if (ActorComp && IsInstancedActorComponentDesirableForComponentArray(ActorComp, bHideConstructionScriptComponentsInDetailsView))
		{
			ComponentsToAdd.Add(ActorComp);
		}
	}

	return ComponentsToAdd;
}

TSet<UActorComponent*> FLevelSnapshotsEditorResultsHelpers::FilterOutActorComponentsBasedOnEditorSettings(
	const TSet<TWeakObjectPtr<UActorComponent>>& UnfilteredActorComponentList)
{
	TSet<UActorComponent*> RawSet;
	for(const TWeakObjectPtr<UActorComponent>& Component : UnfilteredActorComponentList)
	{
		if (UActorComponent* RawComponent = Component.Get())
		{
			RawSet.Add(RawComponent);
		}
	}

	return FilterOutActorComponentsBasedOnEditorSettings(RawSet);
}

int32 FLevelSnapshotsEditorResultsHelpers::CreateNewHierarchyStructInLoop(const AActor* InActor, USceneComponent* SceneComponent, TArray<TWeakPtr<FComponentHierarchy>>& AllHierarchies)
{
	check(InActor);
	check(SceneComponent);
	
	const TSharedRef<FComponentHierarchy> NewHierarchy = MakeShared<FComponentHierarchy>(SceneComponent);

	const int32 ReturnValue = AllHierarchies.Add(NewHierarchy);

	USceneComponent* ParentComponent = SceneComponent->GetAttachParent();

	if (ParentComponent)
	{
		int32 IndexOfParentHierarchy = AllHierarchies.IndexOfByPredicate(
			[&ParentComponent](const TWeakPtr<FComponentHierarchy>& Hierarchy)
			{
				return Hierarchy.IsValid() && Hierarchy.Pin()->Component == ParentComponent;
			});

		if (IndexOfParentHierarchy == -1)
		{
			IndexOfParentHierarchy = CreateNewHierarchyStructInLoop(InActor, ParentComponent, AllHierarchies);
		}

		const bool bIsParentHierarchyValid = ensureMsgf(AllHierarchies[IndexOfParentHierarchy].IsValid(),
			TEXT("%hs: AllHierarchies at index %d for Component '%s' was not valid."),
			__FUNCTION__, IndexOfParentHierarchy, *(ParentComponent->GetName()));

		if (bIsParentHierarchyValid)
		{
			AllHierarchies[IndexOfParentHierarchy].Pin()->DirectChildren.Add(NewHierarchy);
		}
	}

	return ReturnValue;
}

TSharedRef<FComponentHierarchy> FLevelSnapshotsEditorResultsHelpers::BuildComponentHierarchy(const AActor* InActor, TSet<UActorComponent*>& OutNonSceneComponents)
{
	check(InActor);

	TSharedRef<FComponentHierarchy> ReturnHierarchy = MakeShared<FComponentHierarchy>(InActor->GetRootComponent());

	// A flat representation of the hierarchy used for searching the hierarchy more easily
	TArray<TWeakPtr<FComponentHierarchy>> AllHierarchies;
	AllHierarchies.Add(ReturnHierarchy);

	TSet<UActorComponent*> AllActorComponents = FilterOutActorComponentsBasedOnEditorSettings(InActor->GetComponents());

	for (UActorComponent* Component : AllActorComponents)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
		if (IsValid(SceneComponent) && !SceneComponent->HasAnyFlags(RF_Transient))
		{
			const bool ComponentContained = AllHierarchies.ContainsByPredicate(
				[&SceneComponent](const TWeakPtr<FComponentHierarchy>& Hierarchy)
				{
					return Hierarchy.IsValid() && Hierarchy.Pin()->Component == SceneComponent;
				});

			if (!ComponentContained)
			{
				CreateNewHierarchyStructInLoop(InActor, SceneComponent, AllHierarchies);
			}
		}
		else
		{
			OutNonSceneComponents.Add(Component);
		}
	}

	return ReturnHierarchy;
}

void FLevelSnapshotsEditorResultsHelpers::BuildNestedSceneComponentRowsRecursively(
	const TWeakPtr<FComponentHierarchy>& InHierarchy, const TSet<UActorComponent*>& InCounterpartComponents,
	FPropertyEditorModule& PropertyEditorModule, FPropertySelectionMap& PropertySelectionMap, 
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView)
{
	struct Local
	{
		static void CheckComponentsForVisiblePropertiesRecursively(const TWeakPtr<FComponentHierarchy>& InHierarchy, const FPropertySelectionMap& PropertySelectionMap, bool& bHasVisibleComponents)
		{
			USceneComponent* CurrentComponent = InHierarchy.Pin()->Component.Get();
			
			const FPropertySelection* PropertySelection = PropertySelectionMap.GetObjectSelection(CurrentComponent).GetPropertySelection();

			bHasVisibleComponents = PropertySelection ? true : false;

			if (!bHasVisibleComponents)
			{
				for (const TSharedRef<FComponentHierarchy>& Child : InHierarchy.Pin()->DirectChildren)
				{
					CheckComponentsForVisiblePropertiesRecursively(Child, PropertySelectionMap, bHasVisibleComponents);

					if (bHasVisibleComponents)
					{
						break;
					}
				}
			}
		}
	};
	
	check(InHierarchy.IsValid());

	USceneComponent* CurrentComponent = InHierarchy.Pin()->Component.Get();

	check(CurrentComponent);

	bool bShouldCreateComponentRow = false;

	// If this specific component doesn't have properties to display, we need to check the child components recursively
	Local::CheckComponentsForVisiblePropertiesRecursively(InHierarchy, PropertySelectionMap, bShouldCreateComponentRow);

	if (bShouldCreateComponentRow)
	{		
		if (ensureAlwaysMsgf(CurrentComponent,
			TEXT("%hs: CurrentComponent was nullptr. Please check the InHierarchy for valid component."), __FUNCTION__))
		{
			const FPropertySelection* PropertySelection = PropertySelectionMap.GetObjectSelection(CurrentComponent).GetPropertySelection();

			const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter =
				PropertySelection ? PropertySelection->GetSelectedLeafProperties() : TArray<TFieldPath<FProperty>>();

			const TWeakPtr<FLevelSnapshotsEditorResultsRow>& ComponentPropertyAsRow =
				FindComponentCounterpartAndBuildRow(
					CurrentComponent, InCounterpartComponents, PropertyEditorModule, PropertySelectionMap, PropertiesThatPassFilter, InDirectParentRow, InResultsView);

			// Build rows for nested components after creating property rows for component
			if (ensureAlwaysMsgf(ComponentPropertyAsRow.IsValid(),
				TEXT("%hs: ComponentPropertyAsRow for component '%s' was not valid but code path should not return null value."),
				__FUNCTION__, *CurrentComponent->GetName()))
			{
				for (const TSharedRef<FComponentHierarchy>& ChildHierarchy : InHierarchy.Pin()->DirectChildren)
				{
					BuildNestedSceneComponentRowsRecursively(
						ChildHierarchy, InCounterpartComponents, 
						PropertyEditorModule, PropertySelectionMap, 
						ComponentPropertyAsRow, InResultsView);
				}
			}
		}
	}
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsHelpers::BuildModifiedObjectRow(UObject* InWorldObject, UObject* InSnapshotObject,
	FPropertyEditorModule& PropertyEditorModule, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter, 
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView, const FText& InDisplayNameOverride,
	TWeakPtr<FRowGeneratorInfo> WorldRowGeneratorInfoOverride, TWeakPtr<FRowGeneratorInfo> SnapshotRowGeneratorInfoOverride)
{
	check(InWorldObject);

	// Create group
	FLevelSnapshotsEditorResultsRowPtr NewObjectGroup = MakeShared<FLevelSnapshotsEditorResultsRow>(
		InDisplayNameOverride.IsEmpty() ? FText::FromString(InWorldObject->GetName()) : InDisplayNameOverride,
		InWorldObject->IsA(UActorComponent::StaticClass()) ? FLevelSnapshotsEditorResultsRow::ModifiedComponentGroup : FLevelSnapshotsEditorResultsRow::SubObjectGroup,
		InDirectParentRow.IsValid() ? InDirectParentRow.Pin()->GenerateChildWidgetCheckedStateBasedOnParent() : ECheckBoxState::Checked, InResultsView, InDirectParentRow);

	
	// Copy or create Row Generators for object and counterpart
	TWeakPtr<FRowGeneratorInfo> WorldRowGeneratorInfo = WorldRowGeneratorInfoOverride;

	if (!WorldRowGeneratorInfo.IsValid())
	{
		WorldRowGeneratorInfo = InResultsView.Pin()->RegisterRowGenerator(NewObjectGroup, ObjectType_World, PropertyEditorModule);

		WorldRowGeneratorInfo.Pin()->GetGeneratorObject().Pin()->SetObjects({ InWorldObject });
	}

	TWeakPtr<FRowGeneratorInfo> SnapshotRowGeneratorInfo;
	
	if (InSnapshotObject)
	{
		SnapshotRowGeneratorInfo = SnapshotRowGeneratorInfoOverride;

		if (!SnapshotRowGeneratorInfo.IsValid())
		{
			SnapshotRowGeneratorInfo = InResultsView.Pin()->RegisterRowGenerator(NewObjectGroup, ObjectType_Snapshot, PropertyEditorModule);
				
			SnapshotRowGeneratorInfo.Pin()->GetGeneratorObject().Pin()->SetObjects({ InSnapshotObject });
		}
	}
	
	NewObjectGroup->InitObjectRow(InSnapshotObject, InWorldObject,SnapshotRowGeneratorInfo, WorldRowGeneratorInfo);

	UE_LOGF(LogLevelSnapshots, Log, "About to loop over properties for object named %ls", *InWorldObject->GetName());
	
	TArray<TFieldPath<FProperty>> PropertyRowsGenerated =
		LoopOverProperties(SnapshotRowGeneratorInfo, WorldRowGeneratorInfo,
			NewObjectGroup, PropertySelectionMap, PropertiesThatPassFilter, InResultsView);

	if (PropertyRowsGenerated.Num() == 0)
	{
		// In this case check the row generators to see if they were overridden. If so, create new ones and try again.
		if (WorldRowGeneratorInfoOverride.IsValid() || SnapshotRowGeneratorInfoOverride.IsValid())
		{
			WorldRowGeneratorInfo = InResultsView.Pin()->RegisterRowGenerator(NewObjectGroup, ObjectType_World, PropertyEditorModule);

			WorldRowGeneratorInfo.Pin()->GetGeneratorObject().Pin()->SetObjects({ InWorldObject });

			if (InSnapshotObject)
			{
				SnapshotRowGeneratorInfo = InResultsView.Pin()->RegisterRowGenerator(NewObjectGroup, ObjectType_Snapshot, PropertyEditorModule);
				
				SnapshotRowGeneratorInfo.Pin()->GetGeneratorObject().Pin()->SetObjects({ InSnapshotObject });
			}

			PropertyRowsGenerated =
				LoopOverProperties(SnapshotRowGeneratorInfo, WorldRowGeneratorInfo,
					NewObjectGroup, PropertySelectionMap, PropertiesThatPassFilter, InResultsView);
		}
	}

	// Generate fallback rows for properties not supported by PropertyRowGenerator
	for (TFieldPath<FProperty> FieldPath : PropertiesThatPassFilter)
	{
		if (!PropertyRowsGenerated.Contains(FieldPath))
		{
			LevelSnapshotsEditorCustomWidgetGenerator::CreateRowsForPropertiesNotHandledByPropertyRowGenerator(
				FieldPath, InSnapshotObject, InWorldObject, InResultsView, NewObjectGroup);
		}
	}

	InDirectParentRow.Pin()->InsertChildRowAtIndex(NewObjectGroup);
		
	return NewObjectGroup;
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsHelpers::BuildObjectRowForAddedObjectsToRemove(
	UObject* InAddedObject,	const TSharedRef<FLevelSnapshotsEditorResultsRow> InDirectParentRow, const TSharedRef<SLevelSnapshotsEditorResults> InResultsView)
{
	check(InAddedObject);

	// Create group
	FLevelSnapshotsEditorResultsRowPtr NewObjectGroup = MakeShared<FLevelSnapshotsEditorResultsRow>(
		FText::FromString(InAddedObject->GetName()), FLevelSnapshotsEditorResultsRow::AddedComponentToRemove,
		InDirectParentRow->GetWidgetCheckedState(), InResultsView, InDirectParentRow);
	
	NewObjectGroup->InitAddedObjectRow(InAddedObject);

	InDirectParentRow->InsertChildRowAtIndex(NewObjectGroup);
		
	return NewObjectGroup;
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsHelpers::BuildObjectRowForRemovedObjectsToAdd(
	UObject* InRemovedObject, const TSharedRef<FLevelSnapshotsEditorResultsRow> InDirectParentRow,
	const TSharedRef<SLevelSnapshotsEditorResults> InResultsView)
{
	check(InRemovedObject);

	// Create group
	FLevelSnapshotsEditorResultsRowPtr NewObjectGroup = MakeShared<FLevelSnapshotsEditorResultsRow>(
		FText::FromString(InRemovedObject->GetName()), FLevelSnapshotsEditorResultsRow::RemovedComponentToAdd,
		InDirectParentRow->GetWidgetCheckedState(), InResultsView, InDirectParentRow);
	
	NewObjectGroup->InitRemovedObjectRow(InRemovedObject);

	InDirectParentRow->InsertChildRowAtIndex(NewObjectGroup);
		
	return NewObjectGroup;
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsHelpers::FindComponentCounterpartAndBuildRow(
	UActorComponent* InWorldObject, const TSet<UActorComponent*>& InCounterpartObjects, 
	FPropertyEditorModule& PropertyEditorModule, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter, 
    const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView)
{
	check(InWorldObject);

	UObject* CounterpartObject = FindCounterpartComponent(InWorldObject, InCounterpartObjects);

	// We only want to build a row with properties if properties are modified. Added/Removed Components are generated separately as they display no properties.
	if (CounterpartObject)
	{
		return BuildModifiedObjectRow(
			InWorldObject, CounterpartObject, PropertyEditorModule, PropertySelectionMap,
			PropertiesThatPassFilter, InDirectParentRow, InResultsView);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

