// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsEditorResultsRow.h"

struct FComponentHierarchy
{
	FComponentHierarchy(USceneComponent* InComponent)
		: Component(InComponent) {};

	const TWeakObjectPtr<USceneComponent> Component;
	TArray<TSharedRef<FComponentHierarchy>> DirectChildren;
};

namespace FLevelSnapshotsEditorResultsHelpers
{
	TArray<TFieldPath<FProperty>> LoopOverProperties(
		const TWeakPtr<FRowGeneratorInfo>& InSnapshotRowGeneratorInfo, const TWeakPtr<FRowGeneratorInfo>& InWorldRowGeneratorInfo,
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter,
		const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView);

	/* Do not pass in the base hierarchy, only children of the base hierarchy. */
	void LoopOverHandleHierarchiesAndCreateRowHierarchy(ELevelSnapshotsObjectType InputType, const TWeakPtr<FPropertyHandleHierarchy>& InputHierarchy,
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter,
		TArray<TFieldPath<FProperty>>& PropertyRowsGenerated, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView, 
		const TWeakPtr<FPropertyHandleHierarchy>& InHierarchyToSearchForCounterparts = nullptr);

	/* Finds a hierarchy entry recursively */
	TWeakPtr<FPropertyHandleHierarchy> FindCorrespondingHandle(
		const FLevelSnapshotPropertyChain& InPropertyChain, const FText& InDisplayName, const TWeakPtr<FPropertyHandleHierarchy>& HierarchyToSearch, bool& bFoundMatch);

	/* A helper function called by BuildPropertyHandleHierarchy */
	void CreatePropertyHandleHierarchyChildrenRecursively(
		const TSharedRef<IDetailTreeNode>& InNode, const TWeakPtr<FPropertyHandleHierarchy>& InParentHierarchy, const TWeakObjectPtr<UObject> InContainingObject);

	/* Creates a node tree of all property handles created by PRG. The first node is a dummy node to contain all children. */
	TSharedPtr<FPropertyHandleHierarchy> BuildPropertyHandleHierarchy(const TWeakPtr<FRowGeneratorInfo>& InRowGenerator);

	UObject* FindCounterpartComponent(const UActorComponent* SubObjectToMatch, const TSet<UActorComponent*>& InCounterpartSubObjects);

	/**
	 * Determines whether the specified ActorComponent is an instanced actor component and whether or not it is desirable
	 * for the Level Snapshots process based on its creation method, this function's parameters and editor settings.
	 * @param ActorComponent The ActorComponent to check.
	 * @param bHideConstructionScriptComponentsInDetailsView Whether to hide construction script components in the details view. Default is true.
	 * @return true if the ActorComponent should be added to a given Level Snapshots component array, false otherwise.
	 */
	bool IsInstancedActorComponentDesirableForComponentArray(
		UActorComponent* ActorComponent, const bool bHideConstructionScriptComponentsInDetailsView = true);

	/**
	 * Filters out invalid and instanced actor components based on editor settings.
	 * @param UnfilteredActorComponentList The unfiltered list of actor components.
	 * @return The filtered set of actor components.
	 */
	TSet<UActorComponent*> FilterOutActorComponentsBasedOnEditorSettings(const TSet<UActorComponent*>& UnfilteredActorComponentList);

	/**
	 * Filters out invalid and instanced actor components based on editor settings.
	 * @param UnfilteredActorComponentList The unfiltered list of actor components.
	 * @return The filtered set of actor components.
	 */
	TSet<UActorComponent*> FilterOutActorComponentsBasedOnEditorSettings(const TSet<TWeakObjectPtr<UActorComponent>>& UnfilteredActorComponentList);

	int32 CreateNewHierarchyStructInLoop(const AActor* InActor, USceneComponent* SceneComponent, TArray<TWeakPtr<FComponentHierarchy>>& AllHierarchies);

	/* Creates a node tree of all scene components in an actor. Only scene components can have children. Non-scene actor components do not */
	TSharedRef<FComponentHierarchy> BuildComponentHierarchy(const AActor* InActor, TSet<UActorComponent*>& OutNonSceneComponents);

	void BuildNestedSceneComponentRowsRecursively(const TWeakPtr<FComponentHierarchy>& InHierarchy, const TSet<UActorComponent*>& InCounterpartComponents,
		FPropertyEditorModule& PropertyEditorModule, FPropertySelectionMap& PropertySelectionMap, 
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView);

	// ForSubobjects and components with a known pairing. If RowGeneratorInfo pointers are not specified, ones will be created using the input objects.
	TWeakPtr<FLevelSnapshotsEditorResultsRow> BuildModifiedObjectRow(UObject* InWorldObject, UObject* InSnapshotObject,
		FPropertyEditorModule& PropertyEditorModule, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter, 
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView,
		const FText& InDisplayNameOverride = FText::GetEmpty(), TWeakPtr<FRowGeneratorInfo> WorldRowGeneratorInfoOverride = nullptr,
		TWeakPtr<FRowGeneratorInfo> SnapshotRowGeneratorInfoOverride = nullptr);

	TWeakPtr<FLevelSnapshotsEditorResultsRow> BuildObjectRowForAddedObjectsToRemove(
		UObject* InAddedObject, const TSharedRef<FLevelSnapshotsEditorResultsRow> InDirectParentRow, const TSharedRef<SLevelSnapshotsEditorResults> InResultsView);

	TWeakPtr<FLevelSnapshotsEditorResultsRow> BuildObjectRowForRemovedObjectsToAdd(
		UObject* InRemovedObject, const TSharedRef<FLevelSnapshotsEditorResultsRow> InDirectParentRow, const TSharedRef<SLevelSnapshotsEditorResults> InResultsView);

	// For components without a known pairing
	TWeakPtr<FLevelSnapshotsEditorResultsRow> FindComponentCounterpartAndBuildRow(UActorComponent* InWorldObject, const TSet<UActorComponent*>& InCounterpartObjects,
		FPropertyEditorModule& PropertyEditorModule, FPropertySelectionMap& PropertySelectionMap, const TArray<TFieldPath<FProperty>>& PropertiesThatPassFilter, 
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow, const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView);
};
