// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Results/LevelSnapshotsEditorResultsRow.h"

#include "LevelSnapshotsEditorResultsHelpers.h"
#include "LevelSnapshotsEditorStyle.h"
#include "LevelSnapshotsFunctionLibrary.h"
#include "Misc/LevelSnapshotsEditorCustomWidgetGenerator.h"
#include "SLevelSnapshotsEditorResults.h"

#include "ClassIconFinder.h"
#include "IDetailPropertyRow.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "LevelSnapshotsLog.h"
#include "Modules/ModuleManager.h"
#include "PropertyInfoHelpers.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

TWeakPtr<IPropertyRowGenerator> FRowGeneratorInfo::GetGeneratorObject() const
{
	return GeneratorObject;
}

void FRowGeneratorInfo::FlushReferences()
{
	GeneratorObject.Reset();
}

FPropertyHandleHierarchy::FPropertyHandleHierarchy(
	const TSharedPtr<IDetailTreeNode>& InNode, const TSharedPtr<IPropertyHandle>& InHandle, const TWeakObjectPtr<UObject> InContainingObject)
	: Node(InNode)
	, Handle(InHandle)
	, ContainingObject(InContainingObject)
{
	if (Handle.IsValid() && ContainingObject.IsValid())
	{			
		DisplayName = Handle->GetPropertyDisplayName();

		// For TMaps
		if (const TSharedPtr<IPropertyHandle> KeyHandle = Handle->GetKeyHandle())
		{
			FString OutValue;
			KeyHandle->GetValueAsFormattedString(OutValue);
			DisplayName = FText::FromString("Key: " + OutValue);
		}
		
		if (const FProperty* Property = Handle->GetProperty())
		{
			UObject* Object = ContainingObject.Get();
			check(Object);

			UStruct* IterableStruct;

			if (UScriptStruct* AsScriptStruct = Cast<UScriptStruct>(Object))
			{
				IterableStruct = AsScriptStruct;
			}
			else
			{
				IterableStruct = Object->GetClass();
			}
			check(IterableStruct);
			
			PropertyChain = FLevelSnapshotPropertyChain::FindPathToProperty(Property, IterableStruct);
		}
	}
}

FLevelSnapshotsEditorResultsRow::~FLevelSnapshotsEditorResultsRow()
{
	FlushReferences();
}

void FLevelSnapshotsEditorResultsRow::FlushReferences()
{
	if (ChildRows.Num())
	{
		ChildRows.Empty();
	}

	if (HeaderColumns.Num())
	{
		HeaderColumns.Empty();
	}

	if (SnapshotObject.IsValid())
	{
		SnapshotObject.Reset();
	}
	if (WorldObject.IsValid())
	{
		WorldObject.Reset();
	}
	if (ResultsViewPtr.IsValid())
	{
		ResultsViewPtr.Reset();
	}

	if (SnapshotPropertyHandleHierarchy.IsValid())
	{
		SnapshotPropertyHandleHierarchy = nullptr;
	}
	if (WorldPropertyHandleHierarchy.IsValid())
	{
		WorldPropertyHandleHierarchy = nullptr;
	}
}

FLevelSnapshotsEditorResultsRow::FLevelSnapshotsEditorResultsRow(
	const FText InDisplayName,                                     
	const ELevelSnapshotsEditorResultsRowType InRowType,
	const ECheckBoxState StartingWidgetCheckboxState, 
	const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView,
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow)
		: RowType(InRowType)
		, DisplayName(InDisplayName)
		, Tooltip(DisplayName)
		, DirectParentRow(InDirectParentRow)
		, ResultsViewPtr(InResultsView)
{
	check(ResultsViewPtr.IsValid());
	SetWidgetCheckedState(StartingWidgetCheckboxState);
}

void FLevelSnapshotsEditorResultsRow::InitHeaderRow(const ELevelSnapshotsEditorResultsTreeViewHeaderType InHeaderType,
	const TArray<FText>& InColumns)
{
	HeaderType = InHeaderType;
	HeaderColumns = InColumns;

	// Set visibility values to false for header rows so they hide correctly when they have no visible children
	bDoesRowMatchSearchTerms = false;

	ApplyRowStateMemoryIfAvailable();
}

void FLevelSnapshotsEditorResultsRow::InitAddedActorRow(AActor* InAddedActor)
{
	WorldObject = InAddedActor;

	ApplyRowStateMemoryIfAvailable();
	InitTooltipWithObject(InAddedActor);
}

void FLevelSnapshotsEditorResultsRow::InitRemovedActorRow(const FSoftObjectPath& InRemovedActorPath)
{
	RemovedActorPath = InRemovedActorPath;

	ApplyRowStateMemoryIfAvailable();
	InitTooltipWithObject(InRemovedActorPath);
}

void FLevelSnapshotsEditorResultsRow::InitAddedObjectRow(UObject* InAddedObjectToRemove)
{
	WorldObject = InAddedObjectToRemove;

	ApplyRowStateMemoryIfAvailable();
	InitTooltipWithObject(InAddedObjectToRemove);
}

void FLevelSnapshotsEditorResultsRow::InitRemovedObjectRow(UObject* InRemovedObjectToAdd)
{
	WorldObject = InRemovedObjectToAdd;

	ApplyRowStateMemoryIfAvailable();
	InitTooltipWithObject(InRemovedObjectToAdd);
}

void FLevelSnapshotsEditorResultsRow::InitActorRow(
	AActor* InSnapshotActor, AActor* InWorldActor)
{
	SnapshotObject = InSnapshotActor;
	WorldObject = InWorldActor;

	ApplyRowStateMemoryIfAvailable();
	InitTooltipWithObject(InWorldActor);
}

void FLevelSnapshotsEditorResultsRow::InitObjectRow(
	UObject* InSnapshotObject, UObject* InWorldObject,
	const TWeakPtr<FRowGeneratorInfo>& InSnapshotRowGenerator,
	const TWeakPtr<FRowGeneratorInfo>& InWorldRowGenerator)
{
	SnapshotObject = InSnapshotObject;
	WorldObject = InWorldObject;
	SnapshotRowGeneratorInfo = InSnapshotRowGenerator;
	WorldRowGeneratorInfo = InWorldRowGenerator;

	ApplyRowStateMemoryIfAvailable();
}

void FLevelSnapshotsEditorResultsRow::InitPropertyRow(
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InContainingObjectGroup,
	const TSharedPtr<FPropertyHandleHierarchy>& InSnapshotHierarchy, const TSharedPtr<FPropertyHandleHierarchy>& InWorldHandleHierarchy,
	const bool bNewIsCounterpartValueSame)
{
	ContainingObjectGroup = InContainingObjectGroup;
	SnapshotPropertyHandleHierarchy = InSnapshotHierarchy;
	WorldPropertyHandleHierarchy = InWorldHandleHierarchy;
	bIsCounterpartValueSame = bNewIsCounterpartValueSame;

	ApplyRowStateMemoryIfAvailable();
}

void FLevelSnapshotsEditorResultsRow::InitPropertyRowWithCustomWidget(
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InContainingObjectGroup, FProperty* InProperty,
	const TSharedPtr<SWidget> InSnapshotCustomWidget, const TSharedPtr<SWidget> InWorldCustomWidget)
{
	ContainingObjectGroup = InContainingObjectGroup;
	RepresentedProperty = InProperty;
	SnapshotPropertyCustomWidget = InSnapshotCustomWidget;
	WorldPropertyCustomWidget = InWorldCustomWidget;

	ApplyRowStateMemoryIfAvailable();
}

void FLevelSnapshotsEditorResultsRow::ApplyRowStateMemoryIfAvailable()
{
	FLevelSnapshotsEditorResultsRowStateMemory RowStateMemory;
	const TSharedPtr<SLevelSnapshotsEditorResults>& ResultsPinned = ResultsViewPtr.Pin();
	
	if (ResultsPinned->FindRowStateMemoryByPath(GetOrGenerateRowPath(), RowStateMemory))
	{
		if (DoesRowRepresentGroup())
		{
			if (RowStateMemory.bIsExpanded)
			{
				ResultsPinned->SetTreeViewItemExpanded(SharedThis(this), true);
			}
		}

		SetWidgetCheckedState(RowStateMemory.WidgetCheckedState);
	}
}

const FString& FLevelSnapshotsEditorResultsRow::GetOrGenerateRowPath()
{
	if (RowPath.IsEmpty())
	{
		RowPath = GetDisplayName().ToString();

		if (DirectParentRow.IsValid())
		{
			RowPath = DirectParentRow.Pin()->GetOrGenerateRowPath() + "." + RowPath;
		}
	}

	return RowPath;
}

void FLevelSnapshotsEditorResultsRow::GenerateModifiedActorGroupChildren(FPropertySelectionMap& PropertySelectionMap)
{
	check(ResultsViewPtr.IsValid());
	check(WorldObject.IsValid());
	check(SnapshotObject.IsValid());

	ChildRows.Empty();
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	AActor* SnapshotActorLocal = Cast<AActor>(SnapshotObject.Get());
	SnapshotRowGeneratorInfo = ResultsViewPtr.Pin()->RegisterRowGenerator(SharedThis(this), ObjectType_Snapshot, PropertyEditorModule);
	SnapshotRowGeneratorInfo.Pin()->GetGeneratorObject().Pin()->SetObjects({ SnapshotObject.Get() });

	AActor* WorldActorLocal = Cast<AActor>(WorldObject.Get());
	WorldRowGeneratorInfo = ResultsViewPtr.Pin()->RegisterRowGenerator(SharedThis(this), ObjectType_World, PropertyEditorModule);
	WorldRowGeneratorInfo.Pin()->GetGeneratorObject().Pin()->SetObjects({ WorldObject.Get() });

	// Get Added and removed components
	const UE::LevelSnapshots::FAddedAndRemovedComponentInfo* AddedAndRemovedComponents = PropertySelectionMap.GetObjectSelection(WorldActorLocal).GetComponentSelection();

	TSet<UActorComponent*> WorldComponentsToRemove;
	TSet<UActorComponent*> SnapshotComponentsToAdd;
	if (AddedAndRemovedComponents)
	{
		WorldComponentsToRemove =
			FLevelSnapshotsEditorResultsHelpers::FilterOutActorComponentsBasedOnEditorSettings(
				AddedAndRemovedComponents->EditorWorldComponentsToRemove);
		
		SnapshotComponentsToAdd =
			FLevelSnapshotsEditorResultsHelpers::FilterOutActorComponentsBasedOnEditorSettings(
				AddedAndRemovedComponents->SnapshotComponentsToAdd);
	}

	// Iterate over components and subobjects
	TSet<UActorComponent*> WorldActorNonSceneComponents;
	const TSharedRef<FComponentHierarchy> WorldComponentHierarchy = 
		FLevelSnapshotsEditorResultsHelpers::BuildComponentHierarchy(WorldActorLocal, WorldActorNonSceneComponents);

	// Remove Components found in AddedAndRemovedComponents->EditorWorldComponentsToRemove from WorldActorNonSceneComponents
	WorldActorNonSceneComponents = WorldActorNonSceneComponents.Difference(WorldComponentsToRemove);
	
	// This set will include non-UActorComponent subobjects later in the list, but first we only need to worry about UActorComponents
	TSet<UActorComponent*> SnapshotComponents;

	if (SnapshotActorLocal)
	{
		SnapshotComponents = SnapshotActorLocal->GetComponents();
	}

	// Non-scene actor components cannot be nested and have no children or parents, so we'll add them separately
	for (UActorComponent* WorldComponent : WorldActorNonSceneComponents)
	{
		if (WorldComponent)
		{
			if (const FPropertySelection* PropertySelection = PropertySelectionMap.GetObjectSelection(WorldComponent).GetPropertySelection())
			{
				// Get remaining properties after filter
				if (PropertySelection->GetSelectedLeafProperties().Num())
				{
					FLevelSnapshotsEditorResultsHelpers::FindComponentCounterpartAndBuildRow(WorldComponent, SnapshotComponents,
						PropertyEditorModule, PropertySelectionMap, PropertySelection->GetSelectedLeafProperties(), SharedThis(this), ResultsViewPtr);
				}
			}
		}
	}

	// Next up are recently added Components which are marked for removal
	for (UActorComponent* AddedComponentToRemove : WorldComponentsToRemove)
	{
		if (AddedComponentToRemove)
		{
			FLevelSnapshotsEditorResultsHelpers::BuildObjectRowForAddedObjectsToRemove(AddedComponentToRemove, SharedThis(this), ResultsViewPtr.Pin().ToSharedRef());
		}
	}

	// Then Components which were removed since the snapshot
	for (UActorComponent* RemovedComponentToAdd : SnapshotComponentsToAdd)
	{
		if (RemovedComponentToAdd)
		{
			FLevelSnapshotsEditorResultsHelpers::BuildObjectRowForRemovedObjectsToAdd(RemovedComponentToAdd, SharedThis(this), ResultsViewPtr.Pin().ToSharedRef());
		}
	}

	// Custom subobjects handled by external ICustomObjectSnapshotSerializer implementation
	const TSharedPtr<SLevelSnapshotsEditorResults> ResultsViewPin = ResultsViewPtr.Pin();
	ULevelSnapshot* ActiveSnapshot = ResultsViewPin && ResultsViewPin->GetEditorDataPtr()->GetActiveSnapshot()
		? ResultsViewPtr.Pin()->GetEditorDataPtr()->GetActiveSnapshot()
		: nullptr;

	const TFunction<void(UObject* UnmatchedSnapshotSubobject)> HandleUnmatchedCustomSnapshotSubobject = [](UObject* UnmatchedCustomSnapshotSubobject)
	{
		// TODO: UnmatchedCustomSnapshotSubobject has no counterpart in the editor world. Display a row for it.
		// If the row is checked when the snapshot is applied, call FPropertySelectionMap::AddCustomEditorSubobjectToRecreate
	};

	TMap<UObject*, UObject*> WorldToSnapshotCustomSubobjects;
	const TFunction<void(UObject* SnapshotSubobject, UObject* EditorWorldSubobject)> HandleMatchedCustomSubobjectPair = [&WorldToSnapshotCustomSubobjects, ActiveSnapshot, &HandleMatchedCustomSubobjectPair, &HandleUnmatchedCustomSnapshotSubobject](UObject* SnapshotSubobject, UObject* EditorWorldSubobject)
	{
		check(EditorWorldSubobject);
		check(SnapshotSubobject);
		
		WorldToSnapshotCustomSubobjects.Add(EditorWorldSubobject, SnapshotSubobject);
		ULevelSnapshotsFunctionLibrary::ForEachMatchingCustomSubobjectPair(ActiveSnapshot, SnapshotSubobject, EditorWorldSubobject, HandleMatchedCustomSubobjectPair, HandleUnmatchedCustomSnapshotSubobject);
	};

	if (ActiveSnapshot)
	{
		ULevelSnapshotsFunctionLibrary::ForEachMatchingCustomSubobjectPair(ActiveSnapshot, SnapshotActorLocal, WorldActorLocal, HandleMatchedCustomSubobjectPair, HandleUnmatchedCustomSnapshotSubobject);
	}

	for (const TPair<UObject*, UObject*> CustomSubObjectPair : WorldToSnapshotCustomSubobjects)
	{
		if (CustomSubObjectPair.Key && CustomSubObjectPair.Value)
		{
			if (const FPropertySelection* PropertySelection = PropertySelectionMap.GetObjectSelection(CustomSubObjectPair.Key).GetPropertySelection())
			{
				// Get remaining properties after filter
				if (PropertySelection->GetSelectedLeafProperties().Num())
				{
					// Generator overrides are specified here because custom subobjects generally have their properties' handles generated in the parent object pass
					FLevelSnapshotsEditorResultsHelpers::BuildModifiedObjectRow(CustomSubObjectPair.Key, CustomSubObjectPair.Value,
						PropertyEditorModule, PropertySelectionMap, PropertySelection->GetSelectedLeafProperties(), SharedThis(this), ResultsViewPtr,
						FText::GetEmpty(), WorldRowGeneratorInfo, SnapshotRowGeneratorInfo);
				}
			}
		}
	}
	
	// Nested Components
	if (WorldComponentHierarchy->Component != nullptr) // Some Actors have no components, like World Settings
	{
		FLevelSnapshotsEditorResultsHelpers::BuildNestedSceneComponentRowsRecursively(WorldComponentHierarchy, SnapshotComponents,
			PropertyEditorModule, PropertySelectionMap, SharedThis(this), ResultsViewPtr);
	}

	// Top-level Actor Properties
	if (const FPropertySelection* PropertySelection = PropertySelectionMap.GetObjectSelection(GetWorldObject()).GetPropertySelection())
	{
		if (PropertySelection->GetSelectedLeafProperties().Num())
		{
			UE_LOGF(LogLevelSnapshots, Log, "About to loop over properties for actor named %ls", *GetWorldObject()->GetName());
			
			const TArray<TFieldPath<FProperty>>& PropertyRowsGenerated = FLevelSnapshotsEditorResultsHelpers::LoopOverProperties(
				SnapshotRowGeneratorInfo, WorldRowGeneratorInfo, SharedThis(this),
				PropertySelectionMap, PropertySelection->GetSelectedLeafProperties(), ResultsViewPtr);

			// Certain actors have some extreme customizations and need to have some of their properties skipped, so generating fallback widgets is not desired
			const FName& ClassName = GetWorldObject()->GetClass()->GetFName();
			if (!LevelSnapshotsEditorCustomWidgetGenerator::IgnoredClassNames.Contains(ClassName))
			{
				// Generate fallback rows for properties not supported by PropertyRowGenerator
				for (TFieldPath<FProperty> FieldPath : PropertySelection->GetSelectedLeafProperties())
				{
					if (!PropertyRowsGenerated.Contains(FieldPath))
					{
						LevelSnapshotsEditorCustomWidgetGenerator::CreateRowsForPropertiesNotHandledByPropertyRowGenerator(
								FieldPath, SnapshotActorLocal, WorldActorLocal, ResultsViewPtr, SharedThis(this));
					}
				}
			}
		}
	}

	SetHasGeneratedChildren(true);

	// Remove cached search terms now that the actor group has child rows to search
	SetCachedSearchTerms("");

	// Apply Search
	check(ResultsViewPtr.IsValid());
	
	if (const TSharedPtr<SLevelSnapshotsEditorResults>& PinnedResults = ResultsViewPtr.Pin())
	{
		PinnedResults->ExecuteResultsViewSearchOnSpecifiedActors(PinnedResults->GetSearchStringFromSearchInputField(), { SharedThis(this) });
	}
}

bool FLevelSnapshotsEditorResultsRow::DoesRowRepresentGroup() const
{
	const TArray<ELevelSnapshotsEditorResultsRowType> GroupTypes =
	{
		TreeViewHeader,
		ModifiedActorGroup,
		ModifiedComponentGroup,
		SubObjectGroup,
		StructGroup,
		StructInMap,
		StructInSetOrArray,
		CollectionGroup
	};
	
	return GroupTypes.Contains(GetRowType());
}

bool FLevelSnapshotsEditorResultsRow::DoesRowRepresentObject() const
{
	const TArray<ELevelSnapshotsEditorResultsRowType> GroupTypes =
	{
		ModifiedActorGroup,
		ModifiedComponentGroup,
		SubObjectGroup,
		AddedActorToRemove,
		RemovedActorToAdd
	};

	return GroupTypes.Contains(GetRowType());
}

FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType FLevelSnapshotsEditorResultsRow::GetRowType() const
{
	return RowType;
}

FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType FLevelSnapshotsEditorResultsRow::DetermineRowTypeFromProperty(
	FProperty* InProperty, const bool bIsCustomized, const bool bHasChildProperties)
{
	if (!InProperty)
	{
		return None;
	}

	ELevelSnapshotsEditorResultsRowType ReturnRowType = SingleProperty;

	if (bIsCustomized && !bHasChildProperties)
	{
		return ReturnRowType;
	}
	
	if (UE::LevelSnapshots::IsPropertyContainer(InProperty))
	{
		if (UE::LevelSnapshots::IsPropertyCollection(InProperty))
		{
			ReturnRowType = CollectionGroup;
		}
		else
		{
			ReturnRowType = StructGroup;

			// If this struct's parent is a collection
			if (UE::LevelSnapshots::IsPropertyInCollection(InProperty))
			{
				ReturnRowType = StructInSetOrArray;

				if (UE::LevelSnapshots::IsPropertyInMap(InProperty))
				{
					ReturnRowType = StructInMap;
				}
			}

			// There are use cases in which a struct property is represented by a single handle but is not considered "Customized"
			// There are no children in the generated hierarchy in these cases,
			// so we'll just treat the struct property as any other single property with no children
			if (!bHasChildProperties)
			{
				switch (ReturnRowType)
				{
				case StructGroup:
					// We don't need to know if struct groups are in another struct, 
					// but we do need to know if a 'single property' is in a struct
					ReturnRowType = SingleProperty;

					if (UE::LevelSnapshots::IsPropertyInStruct(InProperty))
					{
						ReturnRowType = SinglePropertyInStruct;
					}
					break;

				case StructInSetOrArray:
					ReturnRowType = SinglePropertyInSetOrArray;
					break;

				case StructInMap:
					ReturnRowType = SinglePropertyInMap;
					break;

				default:
					checkNoEntry();
				}
			}
		}
	}
	else if (UE::LevelSnapshots::IsPropertyComponentOrSubobject(InProperty))
	{
		// Components are handled separately, so if this is true we can assume it's a subobject
		ReturnRowType = SubObjectGroup;
	}
	else // Single Property Row. If it's in a collection it needs a custom widget.
	{
		if (UE::LevelSnapshots::IsPropertyInContainer(InProperty))
		{
			if (UE::LevelSnapshots::IsPropertyInStruct(InProperty))
			{
				ReturnRowType = SinglePropertyInStruct;
			}
			else if (UE::LevelSnapshots::IsPropertyInMap(InProperty))
			{
				ReturnRowType = SinglePropertyInMap;
			}
			else
			{
				ReturnRowType = SinglePropertyInSetOrArray;
			}
		}
	}

	return ReturnRowType;
}

const TArray<FText>& FLevelSnapshotsEditorResultsRow::GetHeaderColumns() const
{
	return HeaderColumns;
}

FText FLevelSnapshotsEditorResultsRow::GetDisplayName() const
{
	if (GetRowType() == TreeViewHeader && HeaderColumns.Num() > 0)
	{
		return HeaderColumns[0];
	}
	
	return DisplayName;
}

const FSlateBrush* FLevelSnapshotsEditorResultsRow::GetIconBrush() const
{
	if (RowType == ModifiedComponentGroup)
	{		
		ELevelSnapshotsObjectType ObjectType;
		UObject* RowObject = GetFirstValidObject(ObjectType);
		
		if (const UActorComponent* AsComponent = Cast<UActorComponent>(RowObject))
		{
			return FSlateIconFinder::FindIconBrushForClass(AsComponent->GetClass(), TEXT("SCS.Component"));
		}
	}
	else if (RowType == ModifiedActorGroup)
	{		
		ELevelSnapshotsObjectType ObjectType;
		UObject* RowObject = GetFirstValidObject(ObjectType);
		
		if (AActor* AsActor = Cast<AActor>(RowObject))
		{
			FName IconName = AsActor->GetCustomIconName();
			if (IconName == NAME_None)
			{
				IconName = AsActor->GetClass()->GetFName();
			}

			return FClassIconFinder::FindIconForActor(AsActor);
		}
	}

	return nullptr;
}

const TArray<FLevelSnapshotsEditorResultsRowPtr>& FLevelSnapshotsEditorResultsRow::GetChildRows() const
{
	return ChildRows;
}

void FLevelSnapshotsEditorResultsRow::AddToChildRows(const FLevelSnapshotsEditorResultsRowPtr& InRow)
{
	ChildRows.Add(InRow);
}

void FLevelSnapshotsEditorResultsRow::InsertChildRowAtIndex(const FLevelSnapshotsEditorResultsRowPtr& InRow, const int32 AtIndex)
{
	ChildRows.Insert(InRow, AtIndex);
}

bool FLevelSnapshotsEditorResultsRow::GetIsTreeViewItemExpanded() const
{
	return DoesRowRepresentGroup() && bIsTreeViewItemExpanded;
}

void FLevelSnapshotsEditorResultsRow::SetIsTreeViewItemExpanded(const bool bNewExpanded)
{
	bIsTreeViewItemExpanded = bNewExpanded;
}

bool FLevelSnapshotsEditorResultsRow::GetShouldExpandAllChildren() const
{
	return bShouldExpandAllChildren;
}

void FLevelSnapshotsEditorResultsRow::SetShouldExpandAllChildren(const bool bNewShouldExpandAllChildren)
{
	bShouldExpandAllChildren = bNewShouldExpandAllChildren;
}

uint8 FLevelSnapshotsEditorResultsRow::GetChildDepth() const
{
	return ChildDepth;
}

void FLevelSnapshotsEditorResultsRow::SetChildDepth(const uint8 InDepth)
{
	ChildDepth = InDepth;
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsRow::GetDirectParentRow() const
{
	return DirectParentRow;
}

void FLevelSnapshotsEditorResultsRow::SetDirectParentRow(const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow)
{
	DirectParentRow = InDirectParentRow;
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsRow::GetParentRowAtTopOfHierarchy()
{
	TWeakPtr<FLevelSnapshotsEditorResultsRow> TopOfHierarchy(SharedThis(this));

	while (TopOfHierarchy.Pin()->GetDirectParentRow().IsValid())
	{
		TopOfHierarchy = TopOfHierarchy.Pin()->GetDirectParentRow();
	}

	return TopOfHierarchy;
}

TWeakPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsRow::GetContainingObjectGroup() const
{
	return ContainingObjectGroup;
}

bool FLevelSnapshotsEditorResultsRow::GetHasGeneratedChildren() const
{
	return bHasGeneratedChildren;
}

void FLevelSnapshotsEditorResultsRow::SetHasGeneratedChildren(const bool bNewGenerated)
{
	bHasGeneratedChildren = bNewGenerated;
}

bool FLevelSnapshotsEditorResultsRow::MatchSearchTokensToSearchTerms(const TArray<FString> InTokens, const bool bMatchAnyTokens)
{
	bool bMatchFound = false;

	if (InTokens.Num() == 0) // If the search is cleared we'll consider the row to pass search
	{
		bMatchFound = true;
	}
	else
	{
		const FString& SearchTerms = GetOrCacheSearchTerms();

		for (const FString& Token : InTokens)
		{
			if (SearchTerms.Contains(Token))
			{
				bMatchFound = true;

				if (bMatchAnyTokens)
				{
					break;
				}
			}
			else
			{
				if (!bMatchAnyTokens)
				{
					bMatchFound = false;
					break;
				}
			}
		}
	}

	SetDoesRowMatchSearchTerms(bMatchFound);

	return bMatchFound;
}

void FLevelSnapshotsEditorResultsRow::ExecuteSearchOnChildNodes(const FString& SearchString) const
{
	TArray<FString> Tokens;

	SearchString.ParseIntoArray(Tokens, TEXT(" "), true);

	ExecuteSearchOnChildNodes(Tokens);
}

void FLevelSnapshotsEditorResultsRow::ExecuteSearchOnChildNodes(const TArray<FString>& Tokens) const
{
	for (const FLevelSnapshotsEditorResultsRowPtr& ChildRow : GetChildRows())
	{
		if (!ensure(ChildRow.IsValid()))
		{
			continue;
		}

		if (ChildRow->DoesRowRepresentGroup())
		{
			if (ChildRow->MatchSearchTokensToSearchTerms(Tokens))
			{
				// If the group name matches then we pass an empty string to search child nodes since we want them all to be visible
				ChildRow->ExecuteSearchOnChildNodes("");
			}
			else
			{
				// Otherwise we iterate over all child nodes to determine which should and should not be visible
				ChildRow->ExecuteSearchOnChildNodes(Tokens);
			}
		}
		else
		{
			ChildRow->MatchSearchTokensToSearchTerms(Tokens);
		}
	}
}

UObject* FLevelSnapshotsEditorResultsRow::GetSnapshotObject() const
{
	if (SnapshotObject.IsValid())
	{
		return SnapshotObject.Get();
	}

	return nullptr;
}

UObject* FLevelSnapshotsEditorResultsRow::GetWorldObject() const
{
	if (WorldObject.IsValid())
	{
		return WorldObject.Get();
	}
	
	return nullptr;
}

UObject* FLevelSnapshotsEditorResultsRow::GetFirstValidObject(ELevelSnapshotsObjectType& ReturnedType) const
{
	if (UObject* WorldObjectLocal = GetWorldObject())
	{
		ReturnedType = ObjectType_World;
		return WorldObjectLocal;
	}
	else if (UObject* SnapshotActorLocal = GetSnapshotObject())
	{
		ReturnedType = ObjectType_Snapshot;
		return SnapshotActorLocal;
	}

	ReturnedType = ObjectType_None;
	return nullptr;
}

FSoftObjectPath FLevelSnapshotsEditorResultsRow::GetObjectPath() const
{
	if (GetRowType() == RemovedActorToAdd)
	{
		return RemovedActorPath;
	}
	else if (GetWorldObject())
	{
		return FSoftObjectPath(GetWorldObject());
	}

	return nullptr;
}

FProperty* FLevelSnapshotsEditorResultsRow::GetProperty() const
{
	if (!RepresentedProperty)
	{
		TSharedPtr<IPropertyHandle> OutHandle;
		GetFirstValidPropertyHandle(OutHandle);

		if (OutHandle.IsValid())
		{
			return OutHandle->GetProperty();
		}
	}

	return RepresentedProperty;
}

FLevelSnapshotPropertyChain FLevelSnapshotsEditorResultsRow::GetPropertyChain() const
{
	TSharedPtr<FPropertyHandleHierarchy> ExistingHierachy =
		WorldPropertyHandleHierarchy.IsValid() ? WorldPropertyHandleHierarchy :
			SnapshotPropertyHandleHierarchy.IsValid() ? SnapshotPropertyHandleHierarchy : nullptr;

	if (ExistingHierachy.IsValid() && ExistingHierachy->PropertyChain.IsSet())
	{
		return ExistingHierachy->PropertyChain.GetValue();
	}
			
	const FProperty* Property = GetProperty();
	check(Property);

	ELevelSnapshotsObjectType OutType;
	UObject* Object = ContainingObjectGroup.Pin()->GetFirstValidObject(OutType);
	check(Object);

	UStruct* IterableStruct;

	if (UScriptStruct* AsScriptStruct = Cast<UScriptStruct>(Object))
	{
		IterableStruct = AsScriptStruct;
	}
	else
	{
		IterableStruct = Object->GetClass();
	}
	check(IterableStruct);

	TOptional<FLevelSnapshotPropertyChain> OutChain = FLevelSnapshotPropertyChain::FindPathToProperty(Property, IterableStruct);
	
	return OutChain.IsSet() ? OutChain.GetValue() : FLevelSnapshotPropertyChain();
}

TSharedPtr<IDetailTreeNode> FLevelSnapshotsEditorResultsRow::GetSnapshotPropertyNode() const
{
	return SnapshotPropertyHandleHierarchy.IsValid() ? SnapshotPropertyHandleHierarchy->Node : nullptr;
}

TSharedPtr<IDetailTreeNode> FLevelSnapshotsEditorResultsRow::GetWorldPropertyNode() const
{
	return WorldPropertyHandleHierarchy.IsValid() ? WorldPropertyHandleHierarchy->Node : nullptr;
}

ELevelSnapshotsObjectType FLevelSnapshotsEditorResultsRow::GetFirstValidPropertyNode(
	TSharedPtr<IDetailTreeNode>& OutNode) const
{
	if (const TSharedPtr<IDetailTreeNode>& WorldNode = GetWorldPropertyNode())
	{
		OutNode = WorldNode;
		return ObjectType_World;
	}
	else if (const TSharedPtr<IDetailTreeNode>& SnapshotNode = GetSnapshotPropertyNode())
	{
		OutNode = SnapshotNode;
		return ObjectType_Snapshot;
	}

	return ObjectType_None;
}

TSharedPtr<IPropertyHandle> FLevelSnapshotsEditorResultsRow::GetSnapshotPropertyHandle() const
{
	return SnapshotPropertyHandleHierarchy.IsValid() ? SnapshotPropertyHandleHierarchy->Handle : nullptr;
}

TSharedPtr<IPropertyHandle> FLevelSnapshotsEditorResultsRow::GetWorldPropertyHandle() const
{
	return WorldPropertyHandleHierarchy.IsValid() ? WorldPropertyHandleHierarchy->Handle : nullptr;
}

ELevelSnapshotsObjectType FLevelSnapshotsEditorResultsRow::GetFirstValidPropertyHandle(TSharedPtr<IPropertyHandle>& OutHandle) const
{
	if (const TSharedPtr<IPropertyHandle>& WorldHandle = GetWorldPropertyHandle())
	{
		OutHandle = WorldHandle;
		return ObjectType_World;
	}
	else if (const TSharedPtr<IPropertyHandle>& SnapshotHandle = GetSnapshotPropertyHandle())
	{
		OutHandle = SnapshotHandle;
		return ObjectType_Snapshot;
	}

	return ObjectType_None;
}

TSharedPtr<SWidget> FLevelSnapshotsEditorResultsRow::GetSnapshotPropertyCustomWidget() const
{
	return SnapshotPropertyCustomWidget;
}

TSharedPtr<SWidget> FLevelSnapshotsEditorResultsRow::GetWorldPropertyCustomWidget() const
{
	return WorldPropertyCustomWidget;
}

bool FLevelSnapshotsEditorResultsRow::HasCustomWidget(ELevelSnapshotsObjectType InQueryType) const
{
	switch (InQueryType)
	{
		case ELevelSnapshotsObjectType::ObjectType_Snapshot:
			return SnapshotPropertyCustomWidget.IsValid();

		case ELevelSnapshotsObjectType::ObjectType_World:
			return WorldPropertyCustomWidget.IsValid();

		default:
			return SnapshotPropertyCustomWidget.IsValid() && WorldPropertyCustomWidget.IsValid();
	}
}

bool FLevelSnapshotsEditorResultsRow::GetIsCounterpartValueSame() const
{
	return bIsCounterpartValueSame;
}

void FLevelSnapshotsEditorResultsRow::SetIsCounterpartValueSame(const bool bIsValueSame)
{
	bIsCounterpartValueSame = bIsValueSame;
}

ECheckBoxState FLevelSnapshotsEditorResultsRow::GetWidgetCheckedState() const
{
	return WidgetCheckedState;
}

void FLevelSnapshotsEditorResultsRow::SetWidgetCheckedState(const ECheckBoxState NewState, const bool bShouldUpdateHierarchyCheckedStates)
{
	WidgetCheckedState = NewState;

	if (bShouldUpdateHierarchyCheckedStates)
	{
		// Set Children to same checked state
		for (const FLevelSnapshotsEditorResultsRowPtr& ChildRow : GetChildRows())
		{
			if (!ChildRow.IsValid())
			{
				continue;
			}

			ChildRow->SetWidgetCheckedState(NewState, true);
		}
		
		EvaluateAndSetAllParentGroupCheckedStates();
	}

	const ELevelSnapshotsEditorResultsRowType RowTypeLocal = GetRowType();
	
	if ((RowTypeLocal == ModifiedActorGroup || RowTypeLocal == AddedActorToRemove || RowTypeLocal == RemovedActorToAdd) && ResultsViewPtr.IsValid())
	{
		ResultsViewPtr.Pin()->UpdateSnapshotInformationText();
		ResultsViewPtr.Pin()->RefreshScroll();
	}
}

ECheckBoxState FLevelSnapshotsEditorResultsRow::GenerateChildWidgetCheckedStateBasedOnParent() const
{
	ECheckBoxState NewState = GetWidgetCheckedState();

	if (NewState == ECheckBoxState::Undetermined)
	{
		NewState = ECheckBoxState::Checked;
	}

	return NewState;
}

bool FLevelSnapshotsEditorResultsRow::GetIsNodeChecked() const
{
	return GetWidgetCheckedState() == ECheckBoxState::Checked ? true : false;
}

void FLevelSnapshotsEditorResultsRow::SetIsNodeChecked(const bool bNewChecked, const bool bShouldUpdateHierarchyCheckedStates)
{
	SetWidgetCheckedState(bNewChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked, bShouldUpdateHierarchyCheckedStates);
}

bool FLevelSnapshotsEditorResultsRow::HasVisibleChildren() const
{	
	bool bVisibleChildFound = false;

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ChildRow : GetChildRows())
	{
		if (!ChildRow.IsValid())
		{
			continue;
		}

		if (ChildRow->ShouldRowBeVisible())
		{
			bVisibleChildFound = true;

			break;
		}
	}

	return bVisibleChildFound;
}

bool FLevelSnapshotsEditorResultsRow::HasCheckedChildren() const
{
	bool bCheckedChildFound = false;

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ChildRow : GetChildRows())
	{
		if (!ChildRow.IsValid())
		{
			continue;
		}

		bCheckedChildFound = ChildRow->GetIsNodeChecked() || ChildRow->HasCheckedChildren();

		if (bCheckedChildFound)
		{
			break;
		}
	}

	return bCheckedChildFound;
}

bool FLevelSnapshotsEditorResultsRow::HasUncheckedChildren() const
{
	bool bUncheckedChildFound = false;

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ChildRow : GetChildRows())
	{
		bUncheckedChildFound = !ChildRow->GetIsNodeChecked() || ChildRow->HasUncheckedChildren();

		if (bUncheckedChildFound)
		{
			break;
		}
	}

	return bUncheckedChildFound;
}

bool FLevelSnapshotsEditorResultsRow::HasChangedChildren() const
{
	bool bChangedChildFound = false;

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ChildRow : GetChildRows())
	{
		if (!ChildRow.IsValid())
		{
			continue;
		}

		bChangedChildFound = !ChildRow->GetIsCounterpartValueSame() || ChildRow->HasChangedChildren();

		if (bChangedChildFound)
		{
			break;
		}
	}

	return bChangedChildFound;
}

bool FLevelSnapshotsEditorResultsRow::ShouldRowBeVisible() const
{
	const bool bShowUnselectedRows = ResultsViewPtr.IsValid() ? ResultsViewPtr.Pin()->GetShowUnselectedRows() : true;
	const bool bShouldBeVisibleBasedOnCheckedState = bShowUnselectedRows ? true : GetWidgetCheckedState() != ECheckBoxState::Unchecked;
	return bShouldBeVisibleBasedOnCheckedState && (bDoesRowMatchSearchTerms || HasVisibleChildren());
}

void FLevelSnapshotsEditorResultsRow::InitTooltipWithObject(const FSoftObjectPath& RowObject)
{
	Tooltip = FText::Format(
		LOCTEXT("TooltipFmt", "{0} ({1})"),
		DisplayName,
		FText::FromString(RowObject.ToString())
	);
}

bool FLevelSnapshotsEditorResultsRow::GetShouldCheckboxBeHidden() const
{
	return bShouldCheckboxBeHidden;
}

void FLevelSnapshotsEditorResultsRow::SetShouldCheckboxBeHidden(const bool bNewShouldCheckboxBeHidden)
{
	bShouldCheckboxBeHidden = bNewShouldCheckboxBeHidden;
}

EVisibility FLevelSnapshotsEditorResultsRow::GetDesiredVisibility() const
{
	return ShouldRowBeVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

const FString& FLevelSnapshotsEditorResultsRow::GetOrCacheSearchTerms()
{
	if (CachedSearchTerms.IsEmpty())
	{
		SetCachedSearchTerms(GetDisplayName().ToString());
	}
	
	return CachedSearchTerms;
}

void FLevelSnapshotsEditorResultsRow::SetCachedSearchTerms(const FString& InTerms)
{
	CachedSearchTerms = InTerms;
}

void FLevelSnapshotsEditorResultsRow::SetDoesRowMatchSearchTerms(const bool bNewMatch)
{
	bDoesRowMatchSearchTerms = bNewMatch;
}

void FLevelSnapshotsEditorResultsRow::GetAllCheckedChildProperties(TArray<FLevelSnapshotsEditorResultsRowPtr>& CheckedSinglePropertyNodeArray) const
{
	if (HasCheckedChildren())
	{
		for (const FLevelSnapshotsEditorResultsRowPtr& ChildRow : GetChildRows())
		{		
			if (!ChildRow.IsValid())
			{
				continue;
			}

			const ELevelSnapshotsEditorResultsRowType ChildRowType = ChildRow->GetRowType();

			if ((ChildRowType == SingleProperty || ChildRowType == SinglePropertyInStruct || ChildRowType == CollectionGroup) && ChildRow->GetIsNodeChecked())
			{
				CheckedSinglePropertyNodeArray.Add(ChildRow);
			}

			if (ChildRowType == StructGroup || ChildRowType == StructInSetOrArray || ChildRowType == StructInMap)
			{
				CheckedSinglePropertyNodeArray.Add(ChildRow);
				
				ChildRow->GetAllCheckedChildProperties(CheckedSinglePropertyNodeArray);
			}
		}
	}
}

void FLevelSnapshotsEditorResultsRow::GetAllUncheckedChildProperties(
	TArray<FLevelSnapshotsEditorResultsRowPtr>& UncheckedSinglePropertyNodeArray) const
{
	if (HasUncheckedChildren())
	{
		for (const FLevelSnapshotsEditorResultsRowPtr& ChildRow : GetChildRows())
		{
			if (!ChildRow.IsValid())
			{
				continue;
			}
			
			const ELevelSnapshotsEditorResultsRowType ChildRowType = ChildRow->GetRowType();

			if ((ChildRowType == SingleProperty || ChildRowType == SinglePropertyInStruct || ChildRowType == CollectionGroup) && !ChildRow->GetIsNodeChecked())
			{
				UncheckedSinglePropertyNodeArray.Add(ChildRow);
			}

			if (ChildRowType == StructGroup || ChildRowType == StructInSetOrArray || ChildRowType == StructInMap)
			{
				if (!ChildRow->GetIsNodeChecked())
				{
					UncheckedSinglePropertyNodeArray.Add(ChildRow);
				}

				ChildRow->GetAllUncheckedChildProperties(UncheckedSinglePropertyNodeArray);
			}
		}
	}
}

void FLevelSnapshotsEditorResultsRow::EvaluateAndSetAllParentGroupCheckedStates() const
{
	TWeakPtr<FLevelSnapshotsEditorResultsRow> ParentRow = GetDirectParentRow();

	ECheckBoxState NewWidgetCheckedState = ECheckBoxState::Unchecked;

	while (ParentRow.IsValid())
	{
		const FLevelSnapshotsEditorResultsRowPtr& PinnedParent = ParentRow.Pin();
		
		if (PinnedParent->DoesRowRepresentGroup())
		{
			if (NewWidgetCheckedState != ECheckBoxState::Undetermined)
			{
				const bool bHasCheckedChildren = PinnedParent->HasCheckedChildren();
				const bool bHasUncheckedChildren = PinnedParent->HasUncheckedChildren();

				if (!bHasCheckedChildren && bHasUncheckedChildren)
				{
					NewWidgetCheckedState = ECheckBoxState::Unchecked;
				}
				else if (bHasCheckedChildren && !bHasUncheckedChildren)
				{
					NewWidgetCheckedState = ECheckBoxState::Checked;
				}
				else
				{
					NewWidgetCheckedState = ECheckBoxState::Undetermined;
				}
			}

			PinnedParent->SetWidgetCheckedState(NewWidgetCheckedState);
		}

		ParentRow = PinnedParent->GetDirectParentRow();
	}
}

#undef LOCTEXT_NAMESPACE