// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/FilterListData.h"

#include "CoreMinimal.h"
#include "IPropertyRowGenerator.h"
#include "Styling/SlateTypes.h"

class SLevelSnapshotsEditorResults;

struct FLevelSnapshotsEditorResultsRow;

typedef TSharedPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsRowPtr;

enum ELevelSnapshotsObjectType : uint8
{
	ObjectType_None,
	ObjectType_Snapshot,
	ObjectType_World
};

struct FLevelSnapshotsEditorResultsSplitterManager
{
	float NestedColumnWidth = 0.5f; // The right side of the first splitter which contains the nested splitter for the property widgets
	float SnapshotPropertyColumnWidth = 0.5f;
};

struct FLevelSnapshotsEditorResultsRowStateMemory
{
	FLevelSnapshotsEditorResultsRowStateMemory()
	{
		bIsExpanded = false;
		WidgetCheckedState = ECheckBoxState::Checked;
	};
	
	FLevelSnapshotsEditorResultsRowStateMemory(const FString& InPathToRow, const bool bNewIsExpanded, const ECheckBoxState NewWidgetCheckedState)
		: PathToRow(InPathToRow)
		, bIsExpanded(bNewIsExpanded)
		, WidgetCheckedState(NewWidgetCheckedState)
	{};
	
	FString PathToRow;
	bool bIsExpanded;
	ECheckBoxState WidgetCheckedState;
};

struct FRowGeneratorInfo
{
	~FRowGeneratorInfo()
	{
		FlushReferences();
	}
	
	FRowGeneratorInfo(
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InBoundObject, const ELevelSnapshotsObjectType InGeneratorType, const TSharedRef<IPropertyRowGenerator>& InGeneratorObject)
		: BoundObject(InBoundObject)
		, GeneratorType(InGeneratorType)
		, GeneratorObject(InGeneratorObject)
	{};
	
	TWeakPtr<IPropertyRowGenerator> GetGeneratorObject() const;

	void FlushReferences();

private:
	/* The object that represents the object passed into the generator */
	TWeakPtr<FLevelSnapshotsEditorResultsRow> BoundObject;
	/* Whether this generator represents the snapshot or world object */
	ELevelSnapshotsObjectType GeneratorType = ObjectType_None;
	/* The actual generator ptr */
	TSharedPtr<IPropertyRowGenerator> GeneratorObject;
};

struct FPropertyHandleHierarchy
{
	FPropertyHandleHierarchy(const TSharedPtr<IDetailTreeNode>& InNode, const TSharedPtr<IPropertyHandle>& InHandle, const TWeakObjectPtr<UObject> InContainingObject);
	
	TSharedPtr<IDetailTreeNode> Node;
	TSharedPtr<IPropertyHandle> Handle;
	TArray<TSharedRef<FPropertyHandleHierarchy>> DirectChildren;

	TWeakObjectPtr<UObject> ContainingObject;
	TOptional<FLevelSnapshotPropertyChain> PropertyChain;
	
	// Used as a fallback for identifying counterparts for collection members
	FText DisplayName;

	bool IsValidHierarchy() const
	{
		return Handle.IsValid() && ContainingObject.IsValid() && PropertyChain.IsSet();
	}
};

struct FLevelSnapshotsEditorResultsRow final : TSharedFromThis<FLevelSnapshotsEditorResultsRow>
{
	enum ELevelSnapshotsEditorResultsRowType
	{
		None,
		TreeViewHeader, // Includes rows that represent headers grouping together added, removed, or modified actor rows
		AddedActorToRemove, // Includes rows that represent actors which have been created in the world since the snapshot was taken
		RemovedActorToAdd, // Includes rows that represent actors which have been deleted from the world since the snapshot was taken
		ModifiedActorGroup, // Modified Actor group. Use AddedActor or RemovedActor for other Actor row types.
		AddedComponentToRemove, // Includes rows that represent Components which have been created in the world since the snapshot was taken
		RemovedComponentToAdd, // Includes rows that represent Components which have been deleted from the world since the snapshot was taken
		ModifiedComponentGroup, // Includes rows that represent UActorComponent types
		SubObjectGroup, // Includes rows that represent separate non-component objects wholly owned by the world or snapshot actor
		StructGroup, // Rows that represent a single struct or a struct inside of a struct
		StructInMap, // Rows that represent a struct that's a value inside of a map
		StructInSetOrArray, // Rows that represent a struct inside a collection that isn't a map
		CollectionGroup, // Includes rows that represent TMap, TSet, and TArray.
		SingleProperty,
		SinglePropertyInStruct,
		SinglePropertyInMap,
		SinglePropertyInSetOrArray
	};

	enum ELevelSnapshotsEditorResultsTreeViewHeaderType
	{
		HeaderType_None,
		HeaderType_ModifiedActors,
		HeaderType_AddedActors,
		HeaderType_RemovedActors,
	};

	~FLevelSnapshotsEditorResultsRow();

	void FlushReferences();
	
	FLevelSnapshotsEditorResultsRow(
		const FText InDisplayName,
		const ELevelSnapshotsEditorResultsRowType InRowType,
		const ECheckBoxState StartingWidgetCheckboxState, 
		const TWeakPtr<SLevelSnapshotsEditorResults>& InResultsView,
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow = nullptr
		);

	void InitHeaderRow(
		const ELevelSnapshotsEditorResultsTreeViewHeaderType InHeaderType, const TArray<FText>& InColumns);
	
	void InitAddedActorRow(AActor* InAddedActor);
	void InitRemovedActorRow(const FSoftObjectPath& InRemovedActorPath);

	void InitAddedObjectRow(UObject* InAddedObjectToRemove);
	void InitRemovedObjectRow(UObject* InRemovedObjectToAdd);
	
	void InitActorRow(AActor* InSnapshotActor, AActor* InWorldActor);
	void InitObjectRow(
		UObject* InSnapshotObject, UObject* InWorldObject,
		const TWeakPtr<FRowGeneratorInfo>& InSnapshotRowGenerator,
		const TWeakPtr<FRowGeneratorInfo>& InWorldRowGenerator);

	void InitPropertyRow(
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InContainingObjectGroup,
		const TSharedPtr<FPropertyHandleHierarchy>& InSnapshotHierarchy, const TSharedPtr<FPropertyHandleHierarchy>& InWorldHandleHierarchy,
		const bool bNewIsCounterpartValueSame);

	void InitPropertyRowWithCustomWidget(
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InContainingObjectGroup, FProperty* InProperty,
		const TSharedPtr<SWidget> InSnapshotCustomWidget, const TSharedPtr<SWidget> InWorldCustomWidget);

	void ApplyRowStateMemoryIfAvailable();
	const FString& GetOrGenerateRowPath();

	void GenerateModifiedActorGroupChildren(FPropertySelectionMap& PropertySelectionMap);

	bool DoesRowRepresentGroup() const;
	bool DoesRowRepresentObject() const;
	
	ELevelSnapshotsEditorResultsRowType GetRowType() const;

	/* Returns the ELevelSnapshotsEditorResultsRowType of a given property.
	 * Will never return ActorGroup, SubobjectGroup or TreeViewHeader. Returns None on error. */
	static ELevelSnapshotsEditorResultsRowType DetermineRowTypeFromProperty(
		FProperty* InProperty, const bool bIsCustomized, const bool bHasChildProperties);

	const TArray<FText>& GetHeaderColumns() const;

	FText GetDisplayName() const;
	FText GetTooltip() const { return Tooltip; }

	const FSlateBrush* GetIconBrush() const;
	
	/* bHasGeneratedChildren must be true to get actual children. */
	const TArray<FLevelSnapshotsEditorResultsRowPtr>& GetChildRows() const;
	void AddToChildRows(const FLevelSnapshotsEditorResultsRowPtr& InRow);
	void InsertChildRowAtIndex(const FLevelSnapshotsEditorResultsRowPtr& InRow, const int32 AtIndex = 0);

	bool GetIsTreeViewItemExpanded() const;
	void SetIsTreeViewItemExpanded(const bool bNewExpanded);

	bool GetShouldExpandAllChildren() const;
	void SetShouldExpandAllChildren(const bool bNewShouldExpandAllChildren);

	uint8 GetChildDepth() const;
	void SetChildDepth(const uint8 InDepth);

	TWeakPtr<FLevelSnapshotsEditorResultsRow> GetDirectParentRow() const;
	void SetDirectParentRow(const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InDirectParentRow);

	/* Walks up each direct parent's direct parents until it reaches the first FLevelSnapshotsEditorResultsRow without a direct parent, then returns it. */
	TWeakPtr<FLevelSnapshotsEditorResultsRow> GetParentRowAtTopOfHierarchy();

	TWeakPtr<FLevelSnapshotsEditorResultsRow> GetContainingObjectGroup() const;

	bool GetHasGeneratedChildren() const;
	void SetHasGeneratedChildren(const bool bNewGenerated);

	/* If bMatchAnyTokens is false, only nodes that match all terms will be returned. */
	bool MatchSearchTokensToSearchTerms(const TArray<FString> InTokens, const bool bMatchAnyTokens = false);

	/* This overload creates tokens from a string first, then calls ExecuteSearchOnChildNodes(const TArray<FString>& Tokens). */
	void ExecuteSearchOnChildNodes(const FString& SearchString) const;
	void ExecuteSearchOnChildNodes(const TArray<FString>& Tokens) const;

	void SetCachedSearchTerms(const FString& InTerms);

	UObject* GetSnapshotObject() const;
	UObject* GetWorldObject() const;

	UObject* GetFirstValidObject(ELevelSnapshotsObjectType& ReturnedType) const;

	FSoftObjectPath GetObjectPath() const;
	
	FProperty* GetProperty() const;
	FLevelSnapshotPropertyChain GetPropertyChain() const;

	TSharedPtr<IDetailTreeNode> GetSnapshotPropertyNode() const;
	TSharedPtr<IDetailTreeNode> GetWorldPropertyNode() const;
	ELevelSnapshotsObjectType GetFirstValidPropertyNode(TSharedPtr<IDetailTreeNode>& OutNode) const;

	TSharedPtr<IPropertyHandle> GetSnapshotPropertyHandle() const;
	TSharedPtr<IPropertyHandle> GetWorldPropertyHandle() const;
	ELevelSnapshotsObjectType GetFirstValidPropertyHandle(TSharedPtr<IPropertyHandle>& OutHandle) const;

	TSharedPtr<SWidget> GetSnapshotPropertyCustomWidget() const;
	TSharedPtr<SWidget> GetWorldPropertyCustomWidget() const;

	/* Returns true if a specified type of custom widget is valid for its row. Use ObjectType_None to return true only if both snapshot and world custom widgets are valid. */
	bool HasCustomWidget(ELevelSnapshotsObjectType InQueryType) const;

	bool GetIsCounterpartValueSame() const;
	void SetIsCounterpartValueSame(const bool bIsValueSame);

	ECheckBoxState GetWidgetCheckedState() const;
	void SetWidgetCheckedState(const ECheckBoxState NewState, const bool bShouldUpdateHierarchyCheckedStates = false);

	ECheckBoxState GenerateChildWidgetCheckedStateBasedOnParent() const;

	bool GetIsNodeChecked() const;
	void SetIsNodeChecked(const bool bNewChecked, const bool bShouldUpdateHierarchyCheckedStates = false);

	/* Hierarchy utilities */

	bool HasVisibleChildren() const;

	bool HasCheckedChildren()const;
	bool HasUncheckedChildren() const;

	/* Whether the group has any children with associated properties that have any difference between the chosen snapshot and the current level. */
	bool HasChangedChildren() const;

	void GetAllCheckedChildProperties(TArray<FLevelSnapshotsEditorResultsRowPtr>& CheckedSinglePropertyNodeArray) const;
	void GetAllUncheckedChildProperties(TArray<FLevelSnapshotsEditorResultsRowPtr>& UncheckedSinglePropertyNodeArray) const;
	
	bool GetShouldCheckboxBeHidden() const;
	void SetShouldCheckboxBeHidden(const bool bNewShouldCheckboxBeHidden);
	
	EVisibility GetDesiredVisibility() const;

private:

	/* Generic properties */
	ELevelSnapshotsEditorResultsRowType RowType = SingleProperty;
	FText DisplayName;
	FText Tooltip;
	TArray<FLevelSnapshotsEditorResultsRowPtr> ChildRows;
	bool bIsTreeViewItemExpanded = false;

	// Used to epand all children on shift+click.
	bool bShouldExpandAllChildren = false;

	/* For Header Rows */
	ELevelSnapshotsEditorResultsTreeViewHeaderType HeaderType = HeaderType_None;
	TArray<FText> HeaderColumns;

	/* Number of parents in row's ancestor chain */
	uint8 ChildDepth = 0;
	TWeakPtr<FLevelSnapshotsEditorResultsRow> DirectParentRow = nullptr;

	/* This is the component, subobject or actor group to which this row belongs. If nullptr, this row is a top-level actor group. */
	TWeakPtr<FLevelSnapshotsEditorResultsRow> ContainingObjectGroup = nullptr;

	// Only applies to object groups - all of the property groups are generated with the rest of the single properties
	bool bHasGeneratedChildren = false;

	/* When we generate Search Terms for a node, it's saved here so it does not need to be generated again until filters are changed */
	FString CachedSearchTerms;
	bool bDoesRowMatchSearchTerms = true;

	/* This is a breadcrumb trail of display names used to find or store the state of the row. */
	FString RowPath;

	/* For removed actor type rows */
	FSoftObjectPath RemovedActorPath;

	/* For object type rows */

	TWeakObjectPtr<UObject> SnapshotObject;
	TWeakObjectPtr<UObject> WorldObject;
	TWeakPtr<SLevelSnapshotsEditorResults> ResultsViewPtr;

	TWeakPtr<FRowGeneratorInfo> SnapshotRowGeneratorInfo;
	TWeakPtr<FRowGeneratorInfo> WorldRowGeneratorInfo;

	/* For property type rows */

	TSharedPtr<FPropertyHandleHierarchy> SnapshotPropertyHandleHierarchy;
	TSharedPtr<FPropertyHandleHierarchy> WorldPropertyHandleHierarchy;

	FProperty* RepresentedProperty;

	TSharedPtr<SWidget> SnapshotPropertyCustomWidget;
	TSharedPtr<SWidget> WorldPropertyCustomWidget;

	/* Whether the snapshot and world object properties have the same value */
	bool bIsCounterpartValueSame = false;

	bool bShouldCheckboxBeHidden;

	/* Checkbox in widget is bound to this property */
	ECheckBoxState WidgetCheckedState = ECheckBoxState::Checked;

	/* Returns a string of searchable keywords such as object names, property names, paths or anything else associated with the row that might be useful to search for. */
	const FString& GetOrCacheSearchTerms();

	/* Use MatchSearchTerms() first to match Search Terms against tokens, then call this method*/
	void SetDoesRowMatchSearchTerms(const bool bNewMatch);
	
	void EvaluateAndSetAllParentGroupCheckedStates() const;

	/* Evaluates all factors which should make a row visible or invisible but does not set visibility. */
	bool ShouldRowBeVisible() const;
	
	void InitTooltipWithObject(const FSoftObjectPath& RowObject);
};
