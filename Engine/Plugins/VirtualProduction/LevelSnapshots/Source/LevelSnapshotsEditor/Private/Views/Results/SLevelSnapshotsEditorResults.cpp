// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Results/SLevelSnapshotsEditorResults.h"

#include "Data/FilteredResults.h"
#include "LevelSnapshotsLog.h"
#include "LevelSnapshotsEditorResultsRow.h"
#include "LevelSnapshotsEditorStyle.h"
#include "SLevelSnapshotsEditorResultsRow.h"
#include "SnapshotRestorability.h"

#include "Algo/Find.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IPropertyRowGenerator.h"
#include "SPrimaryButton.h"
#include "UObject/UObjectHash.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SLevelSnapshotsEditorResults::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData)
{
	if (!ensure(InEditorData))
	{
		return;
	}
	EditorDataPtr = InEditorData;

	DefaultNameText = LOCTEXT("LevelSnapshots", "Level Snapshots");

	OnActiveSnapshotChangedHandle = InEditorData->OnActiveSnapshotChanged.AddSP(this, &SLevelSnapshotsEditorResults::OnSnapshotSelected);

	OnRefreshResultsHandle = InEditorData->OnRefreshResults.AddSP(this, &SLevelSnapshotsEditorResults::RefreshResults, false);
	
	FMenuBuilder ShowOptionsMenuBuilder = BuildShowOptionsMenu();
	
	FSlateFontInfo SnapshotNameTextFont = FAppStyle::Get().GetFontStyle("Bold");
	SnapshotNameTextFont.Size = 16;

	FSlateFontInfo SelectedActorCountTextFont = FAppStyle::Get().GetFontStyle("Bold");
	SelectedActorCountTextFont.Size = 14;
	
	FSlateFontInfo MiscActorCountTextFont = FAppStyle::Get().GetFontStyle("Regular");
	MiscActorCountTextFont.Size = 10;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f, 10.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
		
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FLevelSnapshotsEditorStyle::GetBrush(TEXT("LevelSnapshots.ToolbarButton")))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(SelectedSnapshotNamePtr, STextBlock)
				.Text(DefaultNameText)
				.Font(SnapshotNameTextFont)
			]
		]	
		
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				SAssignNew(ResultsSearchBoxPtr, SSearchBox)
				.HintText(LOCTEXT("LevelSnapshotsEditorResults_SearchHintText", "Search actors, components, properties..."))
				.OnTextChanged_Raw(this, &SLevelSnapshotsEditorResults::OnResultsViewSearchTextChanged)
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ContentPadding(0)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
				.MenuContent()
				[
					ShowOptionsMenuBuilder.MakeWidget()
				]
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
				]
			]
		]

		+ SVerticalBox::Slot()
		[
			SNew(SOverlay)
			
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			[
				SAssignNew(TreeViewPtr, STreeView<FLevelSnapshotsEditorResultsRowPtr>)
				.SelectionMode(ESelectionMode::None)
				.TreeItemsSource(&TreeViewRootHeaderObjects)
				.OnGenerateRow_Lambda([this](FLevelSnapshotsEditorResultsRowPtr Row, const TSharedRef<STableViewBase>& OwnerTable)
					{
						check(Row.IsValid());
					
						return SNew(STableRow<FLevelSnapshotsEditorResultsRowPtr>, OwnerTable)
							[
								SNew(SLevelSnapshotsEditorResultsRow, Row, SplitterManagerPtr)
							]
							.Visibility_Raw(Row.Get(), &FLevelSnapshotsEditorResultsRow::GetDesiredVisibility);
					})
				.OnGetChildren_Raw(this, &SLevelSnapshotsEditorResults::OnGetRowChildren)
				.OnExpansionChanged_Raw(this, &SLevelSnapshotsEditorResults::OnRowChildExpansionChange, false)
				.OnSetExpansionRecursive(this, &SLevelSnapshotsEditorResults::OnRowChildExpansionChange, true)
				.Visibility_Lambda([this]()
					{
						return this->DoesTreeViewHaveVisibleChildren() ? EVisibility::Visible : EVisibility::Collapsed;
					})
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.Padding(2.0f, 24.0f, 2.0f, 2.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LevelSnapshotsEditorResults_NoResults", "No results to show. Try selecting a snapshot, changing your active filters, or clearing any active search."))
				.Visibility_Lambda([this]()
					{
						return DoesTreeViewHaveVisibleChildren() ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
					})
			]
		]

		+SVerticalBox::Slot()
		.VAlign(VAlign_Bottom)
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.BorderBackgroundColor(FLinearColor::Black)
			[
				// Snapshot Information Text
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(FMargin(5.f, 5.f))
				[
					SAssignNew(InfoTextBox, SVerticalBox)
					.Visibility_Lambda([this]()
					{
						return EditorDataPtr.IsValid() && EditorDataPtr->GetEditorWorld() && EditorDataPtr->GetActiveSnapshot() && TreeViewRootHeaderObjects.Num() ? 
							EVisibility::HitTestInvisible : EVisibility::Hidden;
					})

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SelectedActorCountText, STextBlock)
						.Font(SelectedActorCountTextFont)
						.Justification(ETextJustify::Right)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(MiscActorCountText, STextBlock)
						.Font(MiscActorCountTextFont)
						.Justification(ETextJustify::Right)
					]
				]

				// Apply to World
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				.Padding(5.f, 2.f)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("RestoreLevelSnapshot", "Restore Level Snapshot"))
					.IsEnabled_Lambda([this]()
					{
						return EditorDataPtr.IsValid() && EditorDataPtr->GetEditorWorld() && EditorDataPtr->GetActiveSnapshot() && TreeViewRootHeaderObjects.Num();
					})
					.ToolTipText_Lambda([this]() 
					{
						return IsEnabled() && EditorDataPtr->IsFilterDirty() ? 
							FText(LOCTEXT("RestoreSnapshotTooltip_DirtyState", "Please refresh filters. Restore selected snapshot properties and actors to the world.")) :
							FText(LOCTEXT("RestoreSnapshotTooltip", "Restore selected snapshot properties and actors to the world."));
					})
					.OnClicked_Raw(this, &SLevelSnapshotsEditorResults::OnClickApplyToWorld)
				]
			]
		]
	];
}

SLevelSnapshotsEditorResults::~SLevelSnapshotsEditorResults()
{
	if (EditorDataPtr.IsValid())
	{
		EditorDataPtr->OnActiveSnapshotChanged.Remove(OnActiveSnapshotChangedHandle);
		EditorDataPtr->OnRefreshResults.Remove(OnRefreshResultsHandle);
		EditorDataPtr.Reset();
	}
	
	OnActiveSnapshotChangedHandle.Reset();
	OnRefreshResultsHandle.Reset();
	
	ResultsSearchBoxPtr.Reset();
	ResultsBoxContainerPtr.Reset();

	SplitterManagerPtr.Reset();

	SelectedSnapshotNamePtr.Reset();

	SelectedActorCountText.Reset();
	MiscActorCountText.Reset();

	DummyRow.Reset();

	FlushMemory(false);

	TreeViewPtr.Reset();
}

FMenuBuilder SLevelSnapshotsEditorResults::BuildShowOptionsMenu()
{
	FMenuBuilder ShowOptionsMenuBuilder = FMenuBuilder(true, nullptr);

	ShowOptionsMenuBuilder.AddMenuEntry(
		LOCTEXT("ShowUnselectedRows", "Show Unselected Rows"),
		LOCTEXT("ShowUnselectedRows_Tooltip", "If false, unselected rows will be hidden from view."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() {
				SetShowUnselectedRows(!GetShowUnselectedRows());
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SLevelSnapshotsEditorResults::GetShowUnselectedRows)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	ShowOptionsMenuBuilder.AddMenuEntry(
		LOCTEXT("CollapseAll", "Collapse All"),
		LOCTEXT("LevelSnapshotsResultsView_CollapseAll_Tooltip", "Collapse all expanded actor groups in the Modified Actors list."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() {
				SetAllActorGroupsCollapsed();
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	return ShowOptionsMenuBuilder;
}

void SLevelSnapshotsEditorResults::SetShowFilteredRows(const bool bNewSetting)
{
	bShowFilteredActors = bNewSetting;
}

void SLevelSnapshotsEditorResults::SetShowUnselectedRows(const bool bNewSetting)
{
	bShowUnselectedActors = bNewSetting;
}

bool SLevelSnapshotsEditorResults::GetShowFilteredRows() const
{
	return bShowFilteredActors;
}

bool SLevelSnapshotsEditorResults::GetShowUnselectedRows() const
{
	return bShowUnselectedActors;
}

void SLevelSnapshotsEditorResults::FlushMemory(const bool bShouldKeepMemoryAllocated)
{
	if (bShouldKeepMemoryAllocated)
	{
		TreeViewRootHeaderObjects.Reset();
		TreeViewModifiedActorGroupObjects.Reset();
		TreeViewAddedActorGroupObjects.Reset();
		TreeViewRemovedActorGroupObjects.Reset();
	}
	else
	{
		TreeViewRootHeaderObjects.Empty();
		TreeViewModifiedActorGroupObjects.Empty();
		TreeViewAddedActorGroupObjects.Empty();
		TreeViewRemovedActorGroupObjects.Empty();
	}

	CleanUpGenerators(bShouldKeepMemoryAllocated);
}

TOptional<ULevelSnapshot*> SLevelSnapshotsEditorResults::GetSelectedLevelSnapshot() const
{
	return ensure(EditorDataPtr.IsValid()) ? EditorDataPtr->GetActiveSnapshot() : TOptional<ULevelSnapshot*>();
}

void SLevelSnapshotsEditorResults::OnSnapshotSelected(ULevelSnapshot* InLevelSnapshot)
{	
	if (InLevelSnapshot)
	{
		UpdateSnapshotNameText(InLevelSnapshot);
		
		GenerateTreeView(true);

		if (EditorDataPtr.IsValid())
		{
			EditorDataPtr.Get()->SetIsFilterDirty(false);
		}
	}
	else
	{
		FlushMemory(false);
		UpdateSnapshotInformationText();
	}
}

void SLevelSnapshotsEditorResults::RefreshResults(const bool bSnapshotHasChanged)
{
	GenerateTreeView(bSnapshotHasChanged);

	if (EditorDataPtr.IsValid())
	{
		EditorDataPtr.Get()->SetIsFilterDirty(false);
	}
}

FReply SLevelSnapshotsEditorResults::OnClickApplyToWorld()
{
	if (!ensure(EditorDataPtr.IsValid()))
	{
		return FReply::Handled();
	}

	const TOptional<ULevelSnapshot*> ActiveLevelSnapshot = EditorDataPtr->GetActiveSnapshot();
	if (ActiveLevelSnapshot.IsSet())
	{
		SCOPED_SNAPSHOT_EDITOR_TRACE(ClickApplyToWorld);
		{
			SCOPED_SNAPSHOT_EDITOR_TRACE(GetSelectedPropertiesFromUI)
			BuildSelectionSetFromSelectedPropertiesInEachActorGroup();
		}

		UWorld* World = EditorDataPtr->GetEditorWorld();
		ActiveLevelSnapshot.GetValue()->ApplySnapshotToWorld(World, EditorDataPtr->GetFilterResults()->GetPropertiesToRollback());

		// Notify the user that a snapshot been applied
		FNotificationInfo Notification(
			FText::Format(LOCTEXT("NotificationFormatText_SnapshotApplied", "Snapshot '{0}' Restored"), 
				SelectedSnapshotNamePtr.IsValid() ? SelectedSnapshotNamePtr->GetText() : FText::GetEmpty()));
		Notification.Image = FLevelSnapshotsEditorStyle::GetBrush(TEXT("LevelSnapshots.ToolbarButton"));
		Notification.ExpireDuration = 2.f;
		Notification.bFireAndForget = true;

		TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Notification);

		SetAllActorGroupsCollapsed();

		// Set true to force clear expansion / checked state memory
		RefreshResults(true);
	}
	else
	{
		FNotificationInfo Info(LOCTEXT("SelectSnapshotFirst", "Select a snapshot first."));
		Info.ExpireDuration = 5.f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
	return FReply::Handled();
}

void SLevelSnapshotsEditorResults::UpdateSnapshotNameText(const TOptional<ULevelSnapshot*>& InLevelSnapshot) const
{
	if (SelectedSnapshotNamePtr.IsValid())
	{
		FText SnapshotName = DefaultNameText;
		
		if (InLevelSnapshot.IsSet())
		{
			ULevelSnapshot* Snapshot = InLevelSnapshot.GetValue();
			
			FName ReturnedName = Snapshot->GetSnapshotName();
			if (ReturnedName == "None")
			{
				// If a bespoke snapshot name is not provided then we'll use the asset name. Usually these names are the same.
				ReturnedName = Snapshot->GetFName();
			}

			SnapshotName = FText::FromName(ReturnedName);
		}
			
		SelectedSnapshotNamePtr->SetText(SnapshotName);
	}
}

void SLevelSnapshotsEditorResults::UpdateSnapshotInformationText()
{
	check(EditorDataPtr.IsValid());
	
	int32 SelectedModifiedActorCount = 0;
	const int32 TotalPassingModifiedActorCount = TreeViewModifiedActorGroupObjects.Num();

	int32 SelectedAddedActorCount = 0;
	int32 SelectedRemovedActorCount = 0;
	
	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ActorGroup : TreeViewModifiedActorGroupObjects)
	{
		if (ActorGroup->GetWidgetCheckedState() != ECheckBoxState::Unchecked)
		{
			SelectedModifiedActorCount++;
		}
	}

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ActorGroup : TreeViewAddedActorGroupObjects)
	{
		if (ActorGroup->GetWidgetCheckedState() != ECheckBoxState::Unchecked)
		{
			SelectedAddedActorCount++;
		}
	}

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ActorGroup : TreeViewRemovedActorGroupObjects)
	{
		if (ActorGroup->GetWidgetCheckedState() != ECheckBoxState::Unchecked)
		{
			SelectedRemovedActorCount++;
		}
	}

	if (SelectedActorCountText.IsValid())
	{
		SelectedActorCountText->SetText(FText::Format(LOCTEXT("ResultsRestoreInfoFormatSelectedModifiedActorCount", "{0} actor(s) (of {1} in snapshot) will be restored"),
			FText::AsNumber(SelectedModifiedActorCount), FText::AsNumber(TotalPassingModifiedActorCount)));
	}

	if (MiscActorCountText.IsValid())
	{
		MiscActorCountText->SetText(FText::Format(LOCTEXT("ResultsRestoreInfoFormatSelectedAddedRemovedActorCounts", "{0} actors will be recreated, {1} will be removed"),
			FText::AsNumber(SelectedRemovedActorCount), FText::AsNumber(SelectedAddedActorCount)));
	}
}

void SLevelSnapshotsEditorResults::RefreshScroll() const
{
	TreeViewPtr->RequestListRefresh();
}

void SLevelSnapshotsEditorResults::BuildSelectionSetFromSelectedPropertiesInEachActorGroup()
{
	if (!ensure(GetEditorDataPtr()))
	{
		return;
	}

	struct Local
	{
		static void RemoveUndesirablePropertiesFromSelectionMapRecursively(
			FPropertySelectionMap& SelectionMap, const FLevelSnapshotsEditorResultsRowPtr& Group)
		{
			if (!ensureMsgf(Group->DoesRowRepresentObject(),
				TEXT("%hs: Group does not represent an object. Group name: %s"), __FUNCTION__, *Group->GetDisplayName().ToString()))
			{
				return;
			}

			UObject* WorldObject = Group->GetWorldObject();
			if (!ensureMsgf(WorldObject,
				TEXT("%hs: WorldObject is not valid. Group name: %s"), __FUNCTION__, *Group->GetDisplayName().ToString()))
			{
				return;
			}
			
			if (Group->GetWidgetCheckedState() == ECheckBoxState::Unchecked)
			{
				SelectionMap.RemoveObjectPropertiesFromMap(WorldObject);
				SelectionMap.RemoveComponentSelection(Cast<AActor>(WorldObject));
				SelectionMap.RemoveCustomEditorSubobjectToRecreate(WorldObject, Group->GetSnapshotObject());
				return;
			}

			// We only want to check component or subobject groups or actor groups which have had their children generated already
			if (Group->GetRowType() == FLevelSnapshotsEditorResultsRow::ModifiedActorGroup && !Group->GetHasGeneratedChildren())
			{
				return;
			}

			// We only want to check groups which have unchecked children and remove properties which are unchecked
			if (!Group->HasUncheckedChildren())
			{
				return;
			}
			
			FPropertySelection CheckedNodeFieldPaths;
			if (const FPropertySelection* PropertySelection = SelectionMap.GetObjectSelection(WorldObject).GetPropertySelection())
			{
				// Make a copy of the property selection. If a node is unchecked, we'll remove it from the copy.
				CheckedNodeFieldPaths = *PropertySelection;
			}

			TArray<FLevelSnapshotsEditorResultsRowPtr> UncheckedChildPropertyNodes;

			UE::LevelSnapshots::FAddedAndRemovedComponentInfo NewComponentSelection;
			const UE::LevelSnapshots::FAddedAndRemovedComponentInfo* OldComponentSelection = SelectionMap.GetObjectSelection(WorldObject).GetComponentSelection();
			if (OldComponentSelection)
			{
				NewComponentSelection.SnapshotComponentsToAdd = OldComponentSelection->SnapshotComponentsToAdd;
				NewComponentSelection.EditorWorldComponentsToRemove = OldComponentSelection->EditorWorldComponentsToRemove;
			}
			
			for (const FLevelSnapshotsEditorResultsRowPtr& ChildRow : Group->GetChildRows())
			{
				if (!ChildRow.IsValid())
				{
					continue;
				}

				const FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType ChildRowType = ChildRow->GetRowType();

				if (ChildRow->DoesRowRepresentObject())
				{
					RemoveUndesirablePropertiesFromSelectionMapRecursively(SelectionMap, ChildRow);
				}
				else if ((ChildRowType == FLevelSnapshotsEditorResultsRow::SingleProperty || 
					ChildRowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInStruct || 
					ChildRowType == FLevelSnapshotsEditorResultsRow::CollectionGroup) && 
					!ChildRow->GetIsNodeChecked())
				{
					UncheckedChildPropertyNodes.Add(ChildRow);
				}
				else if (ChildRowType == FLevelSnapshotsEditorResultsRow::StructGroup && ChildRow->GetWidgetCheckedState() == ECheckBoxState::Unchecked)
				{
					UncheckedChildPropertyNodes.Add(ChildRow);
					ChildRow->GetAllUncheckedChildProperties(UncheckedChildPropertyNodes); 
				}
				else if (ChildRowType == FLevelSnapshotsEditorResultsRow::StructGroup || 
					ChildRowType == FLevelSnapshotsEditorResultsRow::StructInSetOrArray || 
					ChildRowType == FLevelSnapshotsEditorResultsRow::StructInMap)
				{					
					ChildRow->GetAllUncheckedChildProperties(UncheckedChildPropertyNodes);
				}
				else if (ChildRowType == FLevelSnapshotsEditorResultsRow::AddedComponentToRemove)
				{
					if (!ChildRow->GetIsNodeChecked())
					{
						if (UActorComponent* Component = Cast<UActorComponent>(ChildRow->GetWorldObject()))
						{
							NewComponentSelection.EditorWorldComponentsToRemove.Remove(Component);
						}
					}
				}
				else if (ChildRowType == FLevelSnapshotsEditorResultsRow::RemovedComponentToAdd)
				{
					if (!ChildRow->GetIsNodeChecked())
					{
						if (UActorComponent* Component = Cast<UActorComponent>(ChildRow->GetWorldObject()))
						{
							NewComponentSelection.SnapshotComponentsToAdd.Remove(Component);
						}
					}
				}
			}

			if (UncheckedChildPropertyNodes.Num())
			{
				for (const FLevelSnapshotsEditorResultsRowPtr& ChildRow : UncheckedChildPropertyNodes)
				{
					if (ChildRow.IsValid())
					{
						FLevelSnapshotPropertyChain Chain = ChildRow->GetPropertyChain();
						
						CheckedNodeFieldPaths.RemoveProperty(&Chain);
					}
				}

				if (CheckedNodeFieldPaths.IsEmpty())
				{
					SelectionMap.RemoveObjectPropertiesFromMap(WorldObject);
				}
				else
				{
					SelectionMap.AddObjectProperties(WorldObject, CheckedNodeFieldPaths);
				}
			}

			if (OldComponentSelection)
			{
				if (AActor* AsActor = Cast<AActor>(WorldObject))
				{
					SelectionMap.RemoveComponentSelection(AsActor);
					SelectionMap.AddComponentSelection(AsActor, NewComponentSelection);
				}
			}
		}
	};

	FPropertySelectionMap PropertySelectionMap = FilterListData.GetModifiedEditorObjectsSelectedProperties_AllowedByFilter();

	// Modified actors
	for (const FLevelSnapshotsEditorResultsRowPtr& Group : TreeViewModifiedActorGroupObjects)
	{
		if (Group.IsValid())
		{
			if (Group->GetWidgetCheckedState() == ECheckBoxState::Unchecked)
			{
				// Remove from PropertyMap
				if (AActor* WorldActor = Cast<AActor>(Group->GetWorldObject()))
				{
					PropertySelectionMap.RemoveObjectPropertiesFromMap(WorldActor);

					for (UActorComponent* Component : WorldActor->GetComponents())
					{
						PropertySelectionMap.RemoveObjectPropertiesFromMap(Component);
					}

					TArray<UObject*> Subobjects;
					GetObjectsWithOuter(WorldActor, Subobjects, EGetObjectsFlags::IncludeNestedObjects);

					for (UObject* Subobject : Subobjects)
					{
						PropertySelectionMap.RemoveObjectPropertiesFromMap(Subobject);
					}

					PropertySelectionMap.RemoveComponentSelection(WorldActor);
				}
			}
			else
			{
				Local::RemoveUndesirablePropertiesFromSelectionMapRecursively(PropertySelectionMap, Group);
			}
		}
	}

	// Added actors
	for (const FLevelSnapshotsEditorResultsRowPtr& Group : TreeViewAddedActorGroupObjects)
	{
		if (Group.IsValid() && Group->GetWidgetCheckedState() != ECheckBoxState::Checked)
		{
			AActor* WorldActor = Cast<AActor>(Group->GetWorldObject());
			PropertySelectionMap.RemoveNewActorToDespawn(WorldActor);
		}
	}

	// Removed actors
	for (const FLevelSnapshotsEditorResultsRowPtr& Group : TreeViewRemovedActorGroupObjects)
	{
		if (Group.IsValid() && Group->GetWidgetCheckedState() != ECheckBoxState::Checked)
		{
			PropertySelectionMap.RemoveDeletedActorToRespawn(Group->GetObjectPath());
		}
	}

	GetEditorDataPtr()->GetFilterResults()->SetPropertiesToRollback(PropertySelectionMap);
}

FString SLevelSnapshotsEditorResults::GetSearchStringFromSearchInputField() const
{
	return ensureAlwaysMsgf(ResultsSearchBoxPtr.IsValid(), TEXT("%hs: ResultsSearchBoxPtr is not valid. Check to make sure it was created."), __FUNCTION__)
	? ResultsSearchBoxPtr->GetText().ToString() : "";
}

ULevelSnapshotsEditorData* SLevelSnapshotsEditorResults::GetEditorDataPtr() const
{
	return EditorDataPtr.IsValid() ? EditorDataPtr.Get() : nullptr;
}

FPropertyRowGeneratorArgs SLevelSnapshotsEditorResults::GetLevelSnapshotsAppropriatePropertyRowGeneratorArgs()
{
	FPropertyRowGeneratorArgs Args;
	Args.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Show;
	Args.bAllowMultipleTopLevelObjects = false;

	return Args;
}

TWeakPtr<FRowGeneratorInfo> SLevelSnapshotsEditorResults::RegisterRowGenerator(
	const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InBoundObject,
	const ELevelSnapshotsObjectType InGeneratorType,
	FPropertyEditorModule& PropertyEditorModule)
{
	// Since we must keep many PRG objects alive in order to access the handle data, validating the nodes each tick is very taxing.
	// We can override the validation with a lambda since the validation function in PRG is not necessary for our implementation

	auto ValidationLambda = ([](const FRootPropertyNodeList& PropertyNodeList) { return true; });
	
	const TSharedRef<IPropertyRowGenerator>& RowGeneratorObject = 
		PropertyEditorModule.CreatePropertyRowGenerator(GetLevelSnapshotsAppropriatePropertyRowGeneratorArgs());

	RowGeneratorObject->SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes::CreateLambda(MoveTemp(ValidationLambda)));
	
	const TSharedRef<FRowGeneratorInfo> NewGeneratorInfo = MakeShared<FRowGeneratorInfo>(FRowGeneratorInfo(InBoundObject, InGeneratorType, RowGeneratorObject));

	RegisteredRowGenerators.Add(NewGeneratorInfo);

	return NewGeneratorInfo;
}

void SLevelSnapshotsEditorResults::CleanUpGenerators(const bool bShouldKeepMemoryAllocated)
{
	for (TSharedPtr<FRowGeneratorInfo>& Generator : RegisteredRowGenerators)
	{		
		if (Generator.IsValid())
		{
			Generator->FlushReferences();
			Generator.Reset();			
		}
	}

	if (bShouldKeepMemoryAllocated)
	{
		RegisteredRowGenerators.Reset();
	}
	else
	{
		RegisteredRowGenerators.Empty();
	}
}

bool SLevelSnapshotsEditorResults::FindRowStateMemoryByPath(const FString& InPath, FLevelSnapshotsEditorResultsRowStateMemory& OutRowStateMemory)
{
	if (RowStateMemory.Num())
	{
		const AlgoImpl::TRangePointerType<TSet<TSharedPtr<FLevelSnapshotsEditorResultsRowStateMemory>>>::Type& FindResult =
			Algo::FindByPredicate(RowStateMemory, [&InPath](const TSharedPtr<FLevelSnapshotsEditorResultsRowStateMemory>& InRowStateMemory)
			{
				return InRowStateMemory->PathToRow.Equals(InPath);
			});

		if (FindResult)
		{
			OutRowStateMemory = *FindResult->Get();
			return true;
		}
	}

	return false;
}

void SLevelSnapshotsEditorResults::AddRowStateToRowStateMemory(
	const TSharedPtr<FLevelSnapshotsEditorResultsRowStateMemory> InRowStateMemory)
{
	RowStateMemory.Add(InRowStateMemory);
}

void SLevelSnapshotsEditorResults::GenerateRowStateMemoryRecursively()
{
	struct Local
	{
		static void GenerateRowStateMemory(const TSharedRef<SLevelSnapshotsEditorResults>& InResultsView, const TSharedPtr<FLevelSnapshotsEditorResultsRow>& Row)
		{
			if (!Row.IsValid())
			{
				return;
			}
			
			InResultsView->AddRowStateToRowStateMemory(
				MakeShared<FLevelSnapshotsEditorResultsRowStateMemory>(Row->GetOrGenerateRowPath(), Row->GetIsTreeViewItemExpanded(), Row->GetWidgetCheckedState()));

			for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ChildRow : Row->GetChildRows())
			{
				GenerateRowStateMemory(InResultsView, ChildRow);
			}
		}
	};
	
	// Reset() rather than Empty() because it's likely we'll use a similar amount of memory
	RowStateMemory.Reset();

	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& Row : TreeViewRootHeaderObjects)
	{
		Local::GenerateRowStateMemory(SharedThis(this), Row);
	}
}

FLevelSnapshotsEditorResultsRowPtr& SLevelSnapshotsEditorResults::GetOrCreateDummyRow()
{
	if (!DummyRow.IsValid())
	{
		DummyRow = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FLevelSnapshotsEditorResultsRow(
				FText::FromString("Dummy"), FLevelSnapshotsEditorResultsRow::None, ECheckBoxState::Undetermined, SharedThis(this)));
	}
	
	return DummyRow;
}

void SLevelSnapshotsEditorResults::GenerateTreeView(const bool bSnapshotHasChanged)
{	
	if (!ensure(EditorDataPtr.IsValid()) || !ensure(TreeViewPtr.IsValid()))
	{
		return;
	}

	SCOPED_SNAPSHOT_EDITOR_TRACE(GenerateTreeView);

	if (bSnapshotHasChanged)
	{
		RowStateMemory.Empty();
	}
	else
	{
		GenerateRowStateMemoryRecursively();
	}
	
	FlushMemory(!bSnapshotHasChanged);
	
	UFilteredResults* FilteredResults = EditorDataPtr->GetFilterResults(); 
	FilteredResults->UpdateFilteredResults(EditorDataPtr->GetEditorWorld());

	FilterListData = FilteredResults->GetFilteredData();
	SplitterManagerPtr = MakeShared<FLevelSnapshotsEditorResultsSplitterManager>(FLevelSnapshotsEditorResultsSplitterManager());

	// Create root headers
	if (FilterListData.HasAnyModifiedActors())
	{
		FLevelSnapshotsEditorResultsRowPtr ModifiedActorsHeader = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FLevelSnapshotsEditorResultsRow(FText::GetEmpty(), FLevelSnapshotsEditorResultsRow::TreeViewHeader, ECheckBoxState::Checked, SharedThis(this)));
		ModifiedActorsHeader->InitHeaderRow(FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsTreeViewHeaderType::HeaderType_ModifiedActors, {
			LOCTEXT("ColumnName_ModifiedActors", "Modified Actors"),
			LOCTEXT("ColumnName_CurrentValue", "Current Value"),
			LOCTEXT("ColumnName_ValueToRestore", "Value to Restore")
			});
		
		if (GenerateTreeViewChildren_ModifiedActors(ModifiedActorsHeader))
		{
			TreeViewRootHeaderObjects.Add(ModifiedActorsHeader);
		}
		else
		{
			ModifiedActorsHeader.Reset();
		}
	}

	if (FilterListData.GetAddedWorldActors_AllowedByFilter().Num())
	{
		FLevelSnapshotsEditorResultsRowPtr AddedActorsHeader = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FLevelSnapshotsEditorResultsRow(FText::GetEmpty(), FLevelSnapshotsEditorResultsRow::TreeViewHeader, ECheckBoxState::Checked, SharedThis(this)));
		AddedActorsHeader->InitHeaderRow(FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsTreeViewHeaderType::HeaderType_AddedActors, 
			{ LOCTEXT("ColumnName_ActorsToRemove", "Actors to Remove") });

		if (GenerateTreeViewChildren_AddedActors(AddedActorsHeader))
		{
			TreeViewRootHeaderObjects.Add(AddedActorsHeader);
		}
		else
		{
			AddedActorsHeader.Reset();
		}
	}
	
	if (FilterListData.GetRemovedOriginalActorPaths_AllowedByFilter().Num())
	{
		FLevelSnapshotsEditorResultsRowPtr RemovedActorsHeader = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FLevelSnapshotsEditorResultsRow(FText::GetEmpty(), FLevelSnapshotsEditorResultsRow::TreeViewHeader, ECheckBoxState::Checked, SharedThis(this)));
		RemovedActorsHeader->InitHeaderRow(FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsTreeViewHeaderType::HeaderType_RemovedActors, 
			{ LOCTEXT("ColumnName_ActorsToAdd", "Actors to Add") });

		if (GenerateTreeViewChildren_RemovedActors(RemovedActorsHeader))
		{
			TreeViewRootHeaderObjects.Add(RemovedActorsHeader);
		}
		else
		{
			RemovedActorsHeader.Reset();
		}
	}

	TreeViewPtr->RequestListRefresh();
	UpdateSnapshotInformationText();

	// Apply last search
	ExecuteResultsViewSearchOnAllActors(GetSearchStringFromSearchInputField());
}

bool SLevelSnapshotsEditorResults::GenerateTreeViewChildren_ModifiedActors(FLevelSnapshotsEditorResultsRowPtr ModifiedActorsHeader)
{
	check(ModifiedActorsHeader);
	
	FilterListData.ForEachModifiedActor([this, &ModifiedActorsHeader](AActor* WorldActor)
	{
		if (!UE::LevelSnapshots::Restorability::IsActorDesirableForCapture(WorldActor))
		{
			return;
		}

		const FPropertySelectionMap& ModifiedSelectedActors = FilterListData.GetModifiedEditorObjectsSelectedProperties_AllowedByFilter();
		if (!ModifiedSelectedActors.HasChanges(WorldActor))
		{
			return;
		}


		// Create group
		const FString& ActorName = WorldActor->GetActorLabel();
		FLevelSnapshotsEditorResultsRowPtr NewActorGroup = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FLevelSnapshotsEditorResultsRow(
				FText::FromString(ActorName),
				FLevelSnapshotsEditorResultsRow::ModifiedActorGroup, ECheckBoxState::Checked,
				SharedThis(this),
				ModifiedActorsHeader)
				);
		TWeakObjectPtr<AActor> WeakSnapshotActor = FilterListData.GetSnapshotCounterpartFor(WorldActor);
		NewActorGroup->InitActorRow(WeakSnapshotActor.IsValid() ? WeakSnapshotActor.Get() : nullptr, WorldActor);
		ModifiedActorsHeader->AddToChildRows(NewActorGroup);
		
		TreeViewModifiedActorGroupObjects.Add(NewActorGroup);

		struct Local
		{
			static void RecursivelyIterateOverSubObjectsAndReturnSearchTerms(UObject* InObject, const FPropertySelectionMap& InSelectionMap, FString& NewCachedSearchTerms)
			{
				TArray<UObject*> SubObjects = InSelectionMap.GetDirectSubobjectsWithProperties(InObject);

				if (SubObjects.Num())
				{
					for (UObject* SubObject : SubObjects)
					{
						if (const FPropertySelection* SubObjectProperties = InSelectionMap.GetObjectSelection(SubObject).GetPropertySelection())
						{
							NewCachedSearchTerms += " " + SubObject->GetName();
							for (const TFieldPath<FProperty>& LeafProperty : SubObjectProperties->GetSelectedLeafProperties())
							{
								NewCachedSearchTerms += " " + LeafProperty.ToString();
							}
						}

						RecursivelyIterateOverSubObjectsAndReturnSearchTerms(SubObject, InSelectionMap, NewCachedSearchTerms);
					}
				}
			}
		};
		
		// Cache search terms using the desired leaf properties for each object newly added to ModifiedActorsSelectedProperties this loop
		FString NewCachedSearchTerms = WorldActor->GetHumanReadableName();
		if (const FPropertySelection* ActorProperties = ModifiedSelectedActors.GetObjectSelection(WorldActor).GetPropertySelection())
		{
			for (const TFieldPath<FProperty>& LeafProperty : ActorProperties->GetSelectedLeafProperties())
			{
				NewCachedSearchTerms += " " + LeafProperty.ToString();
			}
		}

		Local::RecursivelyIterateOverSubObjectsAndReturnSearchTerms(WorldActor, ModifiedSelectedActors, NewCachedSearchTerms);
		NewActorGroup->SetCachedSearchTerms(NewCachedSearchTerms);
	});

	return TreeViewModifiedActorGroupObjects.Num() > 0;
}

bool SLevelSnapshotsEditorResults::GenerateTreeViewChildren_AddedActors(FLevelSnapshotsEditorResultsRowPtr AddedActorsHeader)
{
	check(AddedActorsHeader);
	
	for (const TWeakObjectPtr<AActor>& Actor : FilterListData.GetAddedWorldActors_AllowedByFilter())
	{
		if (!Actor.IsValid())
		{
			continue;
		}
		
		// Create group
		FLevelSnapshotsEditorResultsRowPtr NewActorRow = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FLevelSnapshotsEditorResultsRow(
				FText::FromString(Actor.Get()->GetActorLabel()), FLevelSnapshotsEditorResultsRow::AddedActorToRemove, ECheckBoxState::Checked, SharedThis(this), AddedActorsHeader));
		NewActorRow->InitAddedActorRow(Actor.Get());

		AddedActorsHeader->AddToChildRows(NewActorRow);

		TreeViewAddedActorGroupObjects.Add(NewActorRow);
	}

	return TreeViewAddedActorGroupObjects.Num() > 0;
}

bool SLevelSnapshotsEditorResults::GenerateTreeViewChildren_RemovedActors(FLevelSnapshotsEditorResultsRowPtr RemovedActorsHeader)
{
	if (!ensure(RemovedActorsHeader && EditorDataPtr.IsValid() && EditorDataPtr->GetActiveSnapshot() != nullptr))
	{
		return false;
	}
	
	const TObjectPtr<ULevelSnapshot> ActiveSnapshot = EditorDataPtr->GetActiveSnapshot();
	for (const FSoftObjectPath& ActorPath : FilterListData.GetRemovedOriginalActorPaths_AllowedByFilter())
	{
		const FString ActorName = ActiveSnapshot->GetActorLabel(ActorPath);
		FLevelSnapshotsEditorResultsRowPtr NewActorRow = MakeShared<FLevelSnapshotsEditorResultsRow>(
			FText::FromString(ActorName),
			FLevelSnapshotsEditorResultsRow::RemovedActorToAdd,
			ECheckBoxState::Checked,
			SharedThis(this),
			RemovedActorsHeader
		);
		
		NewActorRow->InitRemovedActorRow(ActorPath);
		RemovedActorsHeader->AddToChildRows(NewActorRow);
		TreeViewRemovedActorGroupObjects.Add(NewActorRow);
	}

	return TreeViewRemovedActorGroupObjects.Num() > 0;
}

FReply SLevelSnapshotsEditorResults::SetAllActorGroupsCollapsed()
{
	if (TreeViewPtr.IsValid())
	{
		for (const FLevelSnapshotsEditorResultsRowPtr& RootRow : TreeViewModifiedActorGroupObjects)
		{
			if (!RootRow.IsValid())
			{
				continue;
			}
			
			TreeViewPtr->SetItemExpansion(RootRow, false);
			RootRow->SetIsTreeViewItemExpanded(false);
		}
	}

	return FReply::Handled();
}

void SLevelSnapshotsEditorResults::OnResultsViewSearchTextChanged(const FText& Text) const
{
	ExecuteResultsViewSearchOnAllActors(Text.ToString());
}

void SLevelSnapshotsEditorResults::ExecuteResultsViewSearchOnAllActors(const FString& SearchString) const
{
	// Consider all rows for search except the header rows
	ExecuteResultsViewSearchOnSpecifiedActors(SearchString, TreeViewModifiedActorGroupObjects);
	ExecuteResultsViewSearchOnSpecifiedActors(SearchString, TreeViewAddedActorGroupObjects);
	ExecuteResultsViewSearchOnSpecifiedActors(SearchString, TreeViewRemovedActorGroupObjects);
}

void SLevelSnapshotsEditorResults::ExecuteResultsViewSearchOnSpecifiedActors(
	const FString& SearchString, const TArray<TSharedPtr<FLevelSnapshotsEditorResultsRow>>& ActorRowsToConsider) const
{
	TArray<FString> Tokens;
	
	// unquoted search equivalent to a match-any-of search
	SearchString.ParseIntoArray(Tokens, TEXT(" "), true);
	
	for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& ChildRow : ActorRowsToConsider)
	{
		if (!ensure(ChildRow.IsValid()))
		{
			continue;
		}
		
		const bool bGroupMatch = ChildRow->MatchSearchTokensToSearchTerms(Tokens);
		
		// If the group name matches then we pass in an empty string so all child nodes are visible.
		// If the name doesn't match, then we need to evaluate each child.
		ChildRow->ExecuteSearchOnChildNodes(bGroupMatch ? "" : SearchString);
	}
}

bool SLevelSnapshotsEditorResults::DoesTreeViewHaveVisibleChildren() const
{
	if (TreeViewPtr.IsValid())
	{
		for (const TSharedPtr<FLevelSnapshotsEditorResultsRow>& Header : TreeViewRootHeaderObjects)
		{
			const EVisibility HeaderVisibility = Header->GetDesiredVisibility();
			
			if (HeaderVisibility != EVisibility::Hidden && HeaderVisibility != EVisibility::Collapsed)
			{
				return true;
			}
		}
	}
	
	return false;
}

void SLevelSnapshotsEditorResults::SetTreeViewItemExpanded(const TSharedPtr<FLevelSnapshotsEditorResultsRow>& RowToExpand, const bool bNewExpansion) const
{
	if (TreeViewPtr.IsValid())
	{
		TreeViewPtr->SetItemExpansion(RowToExpand, bNewExpansion);
	}
}

void SLevelSnapshotsEditorResults::OnGetRowChildren(FLevelSnapshotsEditorResultsRowPtr Row, TArray<FLevelSnapshotsEditorResultsRowPtr>& OutChildren)
{
	if (Row.IsValid())
	{
		if (Row->GetHasGeneratedChildren() || Row->GetRowType() != FLevelSnapshotsEditorResultsRow::ModifiedActorGroup)
		{
			OutChildren = Row->GetChildRows();
		}
		else
		{
			if (Row->GetIsTreeViewItemExpanded())
			{
				FPropertySelectionMap PropertySelectionMap = FilterListData.GetModifiedEditorObjectsSelectedProperties_AllowedByFilter();
				Row->GenerateModifiedActorGroupChildren(PropertySelectionMap);

				OutChildren = Row->GetChildRows();
			}
			else
			{
				OutChildren.Add(GetOrCreateDummyRow());
			}
		}

		if (Row->GetShouldExpandAllChildren())
		{
			SetChildExpansionRecursively(Row, true);
			Row->SetShouldExpandAllChildren(false);
		}
	}
}

void SLevelSnapshotsEditorResults::OnRowChildExpansionChange(FLevelSnapshotsEditorResultsRowPtr Row, const bool bIsExpanded, const bool bIsRecursive) const
{
	if (Row.IsValid())
	{
		if (bIsRecursive)
		{
			if (bIsExpanded)
			{
				if (Row->GetRowType() != FLevelSnapshotsEditorResultsRow::TreeViewHeader)
				{
					Row->SetShouldExpandAllChildren(true);
				}
			}
			else
			{
				SetChildExpansionRecursively(Row, bIsExpanded);
			}
		}
		
		TreeViewPtr->SetItemExpansion(Row, bIsExpanded);
		Row->SetIsTreeViewItemExpanded(bIsExpanded);
	}
}

void SLevelSnapshotsEditorResults::SetChildExpansionRecursively(const FLevelSnapshotsEditorResultsRowPtr& InRow, const bool bNewIsExpanded) const
{
	if (InRow.IsValid())
	{
		for (const FLevelSnapshotsEditorResultsRowPtr& Child : InRow->GetChildRows())
		{
			TreeViewPtr->SetItemExpansion(Child, bNewIsExpanded);
			Child->SetIsTreeViewItemExpanded(bNewIsExpanded);

			SetChildExpansionRecursively(Child, bNewIsExpanded);
		}
	}
};

#undef LOCTEXT_NAMESPACE

