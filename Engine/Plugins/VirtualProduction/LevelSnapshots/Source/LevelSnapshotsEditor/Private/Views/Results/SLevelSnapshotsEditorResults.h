// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyRowGenerator.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

#include "Data/FilterListData.h"
#include "Data/LevelSnapshotsEditorData.h"

enum ELevelSnapshotsObjectType : uint8;

struct FLevelSnapshotsEditorResultsRowStateMemory;
struct FRowGeneratorInfo;
struct FLevelSnapshotsEditorResultsRow;
struct FLevelSnapshotsEditorResultsSplitterManager;

class SBox;
class SSearchBox;
class STextBlock;

typedef TSharedPtr<FLevelSnapshotsEditorResultsRow> FLevelSnapshotsEditorResultsRowPtr;
typedef TSharedPtr<FLevelSnapshotsEditorResultsSplitterManager> FLevelSnapshotsEditorResultsSplitterManagerPtr;

class SLevelSnapshotsEditorResults final : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorResults)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData);

	virtual ~SLevelSnapshotsEditorResults() override;

	FMenuBuilder BuildShowOptionsMenu();

	void SetShowFilteredRows(const bool bNewSetting);
	void SetShowUnselectedRows(const bool bNewSetting);

	bool GetShowFilteredRows() const;
	bool GetShowUnselectedRows() const;
	
	void FlushMemory(const bool bShouldKeepMemoryAllocated);

	TOptional<ULevelSnapshot*> GetSelectedLevelSnapshot() const;

	void OnSnapshotSelected(ULevelSnapshot* InLevelSnapshot);
	void RefreshResults(const bool bSnapshotHasChanged = false);
	FReply OnClickApplyToWorld();

	void UpdateSnapshotNameText(const TOptional<ULevelSnapshot*>& InLevelSnapshot) const;
	void UpdateSnapshotInformationText();

	void RefreshScroll() const;

	/* This method builds a selection set of all visible and checked properties to pass back to apply to the world. */
	void BuildSelectionSetFromSelectedPropertiesInEachActorGroup();

	FString GetSearchStringFromSearchInputField() const;
	void ExecuteResultsViewSearchOnAllActors(const FString& SearchString) const;
	void ExecuteResultsViewSearchOnSpecifiedActors(const FString& SearchString, const TArray<TSharedPtr<FLevelSnapshotsEditorResultsRow>>& ActorRowsToConsider) const;

	bool DoesTreeViewHaveVisibleChildren() const;

	void SetTreeViewItemExpanded(const TSharedPtr<FLevelSnapshotsEditorResultsRow>& RowToExpand, const bool bNewExpansion) const;

	ULevelSnapshotsEditorData* GetEditorDataPtr() const;

	// Row Generator Management
	
	static FPropertyRowGeneratorArgs GetLevelSnapshotsAppropriatePropertyRowGeneratorArgs();

	/* This method should be used in place of the PropertyEditorModule's CreatePropertyRowGenerator
	 * because the returned generator is put into a struct managed by the Results View.*/
	TWeakPtr<FRowGeneratorInfo> RegisterRowGenerator(
		const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InBoundObject, const ELevelSnapshotsObjectType InGeneratorType,
		FPropertyEditorModule& PropertyEditorModule);

	void CleanUpGenerators(const bool bShouldKeepMemoryAllocated);

	bool FindRowStateMemoryByPath(const FString& InPath, FLevelSnapshotsEditorResultsRowStateMemory& OutRowStateMemory);
	void AddRowStateToRowStateMemory(const TSharedPtr<FLevelSnapshotsEditorResultsRowStateMemory> InRowStateMemory);
	void GenerateRowStateMemoryRecursively();

private:

	FLevelSnapshotsEditorResultsRowPtr& GetOrCreateDummyRow();

	// Header Slate Pointers
	TSharedPtr<STextBlock> SelectedSnapshotNamePtr;

	// Snapshot Information Text
	TSharedPtr<SVerticalBox> InfoTextBox;
	TSharedPtr<STextBlock> SelectedActorCountText;
	TSharedPtr<STextBlock> MiscActorCountText;

	FText DefaultNameText;
		
	FReply SetAllActorGroupsCollapsed();

	// Search
	
	void OnResultsViewSearchTextChanged(const FText& Text) const;

	FDelegateHandle OnActiveSnapshotChangedHandle;
	FDelegateHandle OnRefreshResultsHandle;

	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorDataPtr;

	TSharedPtr<SSearchBox> ResultsSearchBoxPtr;
	TSharedPtr<SBox> ResultsBoxContainerPtr;

	// 'Show' Options

	bool bShowFilteredActors = false;
	bool bShowUnselectedActors = true;

	/* For splitter sync */
	FLevelSnapshotsEditorResultsSplitterManagerPtr SplitterManagerPtr;

	//  Tree View Implementation

	void GenerateTreeView(const bool bSnapshotHasChanged);
	bool GenerateTreeViewChildren_ModifiedActors(FLevelSnapshotsEditorResultsRowPtr ModifiedActorsHeader);
	bool GenerateTreeViewChildren_AddedActors(FLevelSnapshotsEditorResultsRowPtr AddedActorsHeader);
	bool GenerateTreeViewChildren_RemovedActors(FLevelSnapshotsEditorResultsRowPtr RemovedActorsHeader);
	
	void OnGetRowChildren(FLevelSnapshotsEditorResultsRowPtr Row, TArray<FLevelSnapshotsEditorResultsRowPtr>& OutChildren);
	void OnRowChildExpansionChange(FLevelSnapshotsEditorResultsRowPtr Row, const bool bIsExpanded, const bool bIsRecursive = false) const;

	void SetChildExpansionRecursively(const FLevelSnapshotsEditorResultsRowPtr& InRow, const bool bNewIsExpanded) const;
	
	TSharedPtr<STreeView<FLevelSnapshotsEditorResultsRowPtr>> TreeViewPtr;
	
	/** Holds all the header groups */
	TArray<FLevelSnapshotsEditorResultsRowPtr> TreeViewRootHeaderObjects;
	TArray<FLevelSnapshotsEditorResultsRowPtr> TreeViewModifiedActorGroupObjects;
	TArray<FLevelSnapshotsEditorResultsRowPtr> TreeViewAddedActorGroupObjects;
	TArray<FLevelSnapshotsEditorResultsRowPtr> TreeViewRemovedActorGroupObjects;

	/* Used to show that a group has children before the children are actually generated */
	FLevelSnapshotsEditorResultsRowPtr DummyRow;
	FFilterListData FilterListData;

	/* The results view should be the sole manager of the RowGenerators' lifetimes */
	TSet<TSharedPtr<FRowGeneratorInfo>> RegisteredRowGenerators;

	/* This list remembers the checked and expansion states of previously created rows so these states can be recreated on refresh. */
	TSet<TSharedPtr<FLevelSnapshotsEditorResultsRowStateMemory>> RowStateMemory;
};
