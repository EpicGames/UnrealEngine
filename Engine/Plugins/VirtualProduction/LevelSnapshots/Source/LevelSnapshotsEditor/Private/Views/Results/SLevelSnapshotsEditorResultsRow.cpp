// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelSnapshotsEditorResultsRow.h"

#include "LevelSnapshotsEditorModule.h"
#include "LevelSnapshotsEditorSettings.h"
#include "LevelSnapshotsEditorStyle.h"

#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "IDetailTreeNode.h"
#include "PropertyHandle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SLevelSnapshotsEditorResultsRow::Construct(const FArguments& InArgs, const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InRow, const FLevelSnapshotsEditorResultsSplitterManagerPtr& InSplitterManagerPtr)
{	
	check(InRow.IsValid());

	Item = InRow;

	FLevelSnapshotsEditorResultsRowPtr PinnedItem = Item.Pin();
	
	SplitterManagerPtr = InSplitterManagerPtr;
	check(SplitterManagerPtr.IsValid());
	
	TSharedPtr<IPropertyHandle> ItemHandle;
	PinnedItem->GetFirstValidPropertyHandle(ItemHandle);
	const bool bHasValidHandle = ItemHandle.IsValid() && ItemHandle->IsValidHandle();
	
	const FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType RowType = PinnedItem->GetRowType();
	const FText DisplayText = RowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInMap ? FText::GetEmpty() : PinnedItem->GetDisplayName();

	const bool bIsHeaderRow = RowType == FLevelSnapshotsEditorResultsRow::TreeViewHeader;
	const bool bIsAddedOrRemovedActorRow = RowType == FLevelSnapshotsEditorResultsRow::AddedActorToRemove || RowType == FLevelSnapshotsEditorResultsRow::RemovedActorToAdd;
	const bool bIsAddedOrRemovedComponentRow = RowType == FLevelSnapshotsEditorResultsRow::AddedComponentToRemove || RowType == FLevelSnapshotsEditorResultsRow::RemovedComponentToAdd;
	const bool bIsSinglePropertyInCollection = 
		RowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInMap || RowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInSetOrArray;

	const bool bDoesRowNeedSplitter = 
		(RowType == FLevelSnapshotsEditorResultsRow::TreeViewHeader && PinnedItem->GetHeaderColumns().Num() > 1) || 
		RowType == FLevelSnapshotsEditorResultsRow::StructInMap || RowType == FLevelSnapshotsEditorResultsRow::StructInSetOrArray || RowType == FLevelSnapshotsEditorResultsRow::StructGroup ||
		(!bIsAddedOrRemovedActorRow && !bIsAddedOrRemovedComponentRow && !PinnedItem->DoesRowRepresentGroup());

	const bool bIsSingleProperty = RowType == FLevelSnapshotsEditorResultsRow::SingleProperty || RowType == FLevelSnapshotsEditorResultsRow::SinglePropertyInStruct;

	const bool bHasAtLeastOneValidNode = PinnedItem->GetWorldPropertyNode().IsValid() || PinnedItem->GetSnapshotPropertyNode().IsValid(); 

	const bool bHasAtLeastOneCustomWidget = PinnedItem->HasCustomWidget(ELevelSnapshotsObjectType::ObjectType_World) || PinnedItem->HasCustomWidget(ELevelSnapshotsObjectType::ObjectType_Snapshot);

	const bool bIsParentRowCheckboxHidden = PinnedItem->GetDirectParentRow().IsValid() && PinnedItem->GetDirectParentRow().Pin()->GetShouldCheckboxBeHidden();

	PinnedItem->SetShouldCheckboxBeHidden(
		bIsSinglePropertyInCollection || RowType == FLevelSnapshotsEditorResultsRow::StructInSetOrArray || RowType == FLevelSnapshotsEditorResultsRow::StructInMap || bIsParentRowCheckboxHidden ||
		(bIsSingleProperty && !bHasAtLeastOneValidNode && !bHasAtLeastOneCustomWidget));

	FText Tooltip = bHasValidHandle ? ItemHandle->GetToolTipText() : PinnedItem->GetTooltip();
	if (bIsSinglePropertyInCollection)
	{
		Tooltip = FText::Format(LOCTEXT("CollectionDisclaimer", "({0}) Individual members of collections cannot be selected. The whole collection will be restored."), Tooltip);
	}
	else if (RowType == FLevelSnapshotsEditorResultsRow::CollectionGroup)
	{
		Tooltip = FText::Format(LOCTEXT("Collection", "Collection ({0})"), Tooltip);
	}
	else if (RowType == FLevelSnapshotsEditorResultsRow::ModifiedComponentGroup)
	{
		Tooltip = FText::Format(LOCTEXT("ComponentOrderDisclaimer", "Component ({0}): Please note that component order reflects the order in the world, not the snapshot. LevelSnapshots does not alter component order."), Tooltip);
	}

	int32 IndentationDepth = 0;
	TWeakPtr<FLevelSnapshotsEditorResultsRow> ParentRow = PinnedItem->GetDirectParentRow();
	while (ParentRow.IsValid())
	{
		IndentationDepth++;
		ParentRow = ParentRow.Pin()->GetDirectParentRow();
	}
	PinnedItem->SetChildDepth(IndentationDepth);

	TSharedPtr<SBorder> BorderPtr;

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(5,2))
		[
			SAssignNew(BorderPtr, SBorder)
			.ToolTipText(Tooltip)
			.Padding(FMargin(0, 5))
			.BorderImage(GetBorderImage(RowType))
		]
	];

	// Create checkbox

	const TSharedRef<SHorizontalBox> BasicRowWidgets = SNew(SHorizontalBox);

	BasicRowWidgets->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	.AutoWidth()
	.Padding(5.f, 2.f)
	[
		SNew(SCheckBox)
		.Visibility(PinnedItem->GetShouldCheckboxBeHidden() ? EVisibility::Hidden : EVisibility::Visible)
		.IsChecked_Raw(PinnedItem.Get(), &FLevelSnapshotsEditorResultsRow::GetWidgetCheckedState)
		.OnCheckStateChanged_Raw(PinnedItem.Get(), &FLevelSnapshotsEditorResultsRow::SetWidgetCheckedState, true)
	];

	if (PinnedItem->DoesRowRepresentObject())
	{
		// Get icon for object
		if (const FSlateBrush* RowIcon = PinnedItem->GetIconBrush())
		{
			BasicRowWidgets->AddSlot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(0.f, 2.f, 5.f, 2.f)
			[
				SNew(SImage).Image(RowIcon)
			];
		}
	}

	if (bIsAddedOrRemovedComponentRow)
	{
		GenerateAddedAndRemovedRowComponents(BasicRowWidgets, PinnedItem, DisplayText);
	}
	else
	{
		// Create row display name text
		BasicRowWidgets->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(DisplayText)
		];
	}

	// Create value widgets
	
	if (bDoesRowNeedSplitter)
	{
		SAssignNew(OuterSplitterPtr, SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(5.0f)
		.HitDetectionSplitterHandleSize(5.0f);

		OuterSplitterPtr->AddSlot().SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.OnSlotResized(SSplitter::FOnSlotResized::CreateLambda([this] (float InWidth) {}))
		.Value(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &SLevelSnapshotsEditorResultsRow::GetNameColumnSize)))
		[
			BasicRowWidgets
		];

		OuterSplitterPtr->AddSlot()
		.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SLevelSnapshotsEditorResultsRow::SetNestedColumnSize))
		.Value(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &SLevelSnapshotsEditorResultsRow::CalculateAndReturnNestedColumnSize)))
		[
			SAssignNew(NestedSplitterPtr, SSplitter)
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			.PhysicalSplitterHandleSize(5.0f)
			.HitDetectionSplitterHandleSize(5.0f)
		];

		NestedSplitterPtr->AddSlot()
		.OnSlotResized(SSplitter::FOnSlotResized::CreateLambda([this] (float InWidth) {}))
		.Value(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &SLevelSnapshotsEditorResultsRow::GetWorldColumnSize)))
		[
			GenerateFinalValueWidget(
				ObjectType_World, PinnedItem, bIsHeaderRow, bIsAddedOrRemovedActorRow || bIsAddedOrRemovedComponentRow || PinnedItem->DoesRowRepresentGroup())
		];

		NestedSplitterPtr->AddSlot()
		.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SLevelSnapshotsEditorResultsRow::SetSnapshotColumnSize))
		.Value(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &SLevelSnapshotsEditorResultsRow::GetSnapshotColumnSize)))
		[
			GenerateFinalValueWidget(
				ObjectType_Snapshot, PinnedItem, bIsHeaderRow, bIsAddedOrRemovedActorRow || bIsAddedOrRemovedComponentRow || PinnedItem->DoesRowRepresentGroup())
		];
		
		BorderPtr->SetContent(OuterSplitterPtr.ToSharedRef());
	}
	else
	{
		BorderPtr->SetContent(BasicRowWidgets);
	}
}

SLevelSnapshotsEditorResultsRow::~SLevelSnapshotsEditorResultsRow()
{
	// Remove delegate bindings

	// Unbind event to the splitter being resized first
	if (NestedSplitterPtr.IsValid())
	{
		for (int32 SplitterSlotCount = 0; SplitterSlotCount < NestedSplitterPtr->GetChildren()->Num(); SplitterSlotCount++)
		{
			NestedSplitterPtr->SlotAt(SplitterSlotCount).OnSlotResized().Unbind();
		}
	}

	if (OuterSplitterPtr.IsValid())
	{
		for (int32 SplitterSlotCount = 0; SplitterSlotCount < OuterSplitterPtr->GetChildren()->Num(); SplitterSlotCount++)
		{
			OuterSplitterPtr->SlotAt(SplitterSlotCount).OnSlotResized().Unbind();
		}
	}

	OuterSplitterPtr.Reset();
	NestedSplitterPtr.Reset();
	SplitterManagerPtr.Reset();
}

void SLevelSnapshotsEditorResultsRow::GenerateAddedAndRemovedRowComponents(
	const TSharedRef<SHorizontalBox> BasicRowWidgets, const FLevelSnapshotsEditorResultsRowPtr PinnedItem, const FText& InDisplayText) const
{
	check(PinnedItem.IsValid());

	USceneComponent* ObjectAsSceneComponent = Cast<USceneComponent>(PinnedItem->GetWorldObject());

	BasicRowWidgets->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	.AutoWidth()
	.Padding(FMargin(5, 0))
	[
		SNew(SImage)
		.Image(FAppStyle::Get().GetBrush(
			PinnedItem->GetRowType() == FLevelSnapshotsEditorResultsRow::RemovedComponentToAdd ? "Icons.Plus" : "Icons.Minus"))
	];

	BasicRowWidgets->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("Italic"))
		.Text(InDisplayText)
		.ToolTipText(
			FText::Format
			( 
				PinnedItem->GetRowType() == FLevelSnapshotsEditorResultsRow::RemovedComponentToAdd ? AddTextFormat : RemoveTextFormat, 
				InDisplayText, 
				ObjectAsSceneComponent ? SceneComponentText : ActorComponentText, 
				ObjectAsSceneComponent ? 
					ObjectAsSceneComponent->GetAttachParent() ? 
						FText::FromString(ObjectAsSceneComponent->GetAttachParent()->GetName()) 
						: FText::FromString("Root") 
				: PinnedItem->GetDirectParentRow().IsValid() ? 
					PinnedItem->GetDirectParentRow().Pin()->GetDisplayName() 
					: FText::FromString("Actor")
			)
		)
	];
}

TSharedRef<SWidget> SLevelSnapshotsEditorResultsRow::GenerateFinalValueWidget(
	const ELevelSnapshotsObjectType InObjectType,
	FLevelSnapshotsEditorResultsRowPtr PinnedItem,
	const bool bIsHeaderRow,
	const bool bNeedsNullWidget
	) const
{
	// Nested Splitter Slot 0
	const int32 SlotIndex = InObjectType == ELevelSnapshotsObjectType::ObjectType_Snapshot ? 1 : 0;
	
	TSharedPtr<SWidget> ChildWidget;
	bool bIsChildWidgetCustomized = false;

	const TSharedPtr<IPropertyHandle> PropertyHandle = InObjectType == ELevelSnapshotsObjectType::ObjectType_Snapshot
		? PinnedItem->GetSnapshotPropertyHandle()
		: PinnedItem->GetWorldPropertyHandle();
	if (PropertyHandle && PropertyHandle->IsValidHandle())
	{
		if (const TSharedPtr<IPropertyHandle> KeyHandle = PropertyHandle->GetKeyHandle())
		{
			const TSharedRef<SSplitter> Splitter = SNew(SSplitter).ResizeMode(ESplitterResizeMode::FixedPosition);
			if (KeyHandle->IsValidHandle())
			{
				Splitter->AddSlot()[KeyHandle->CreatePropertyValueWidget(false)];
			}

			Splitter->AddSlot()[PropertyHandle->CreatePropertyValueWidget(false)];
			ChildWidget = Splitter;
		}
		else
		{
			const TSharedPtr<IDetailTreeNode> Node = InObjectType == ELevelSnapshotsObjectType::ObjectType_Snapshot
				? PinnedItem->GetSnapshotPropertyNode()
				: PinnedItem->GetWorldPropertyNode();
			if (Node)
			{
				bIsChildWidgetCustomized = true;
				const FNodeWidgets Widgets = Node->CreateNodeWidgets();
				ChildWidget = Widgets.WholeRowWidget.IsValid() ? Widgets.WholeRowWidget : Widgets.ValueWidget;
			}
			else
			{
				ChildWidget = PropertyHandle->CreatePropertyValueWidget(false);
			}
		}
	}
	else
	{
		if (bIsHeaderRow && PinnedItem->GetHeaderColumns().Num() > SlotIndex + 1)
		{
			ChildWidget = SNew(STextBlock).Text(PinnedItem->GetHeaderColumns()[SlotIndex + 1]);
		}
		else
		{
			if (bNeedsNullWidget)
			{
				ChildWidget = SNullWidget::NullWidget;
			}
			else if (PinnedItem->HasCustomWidget(ELevelSnapshotsObjectType::ObjectType_World))
			{
				ChildWidget = 
					InObjectType == ELevelSnapshotsObjectType::ObjectType_Snapshot ? PinnedItem->GetSnapshotPropertyCustomWidget() : PinnedItem->GetWorldPropertyCustomWidget();
			}
		}
	}

	if (!ChildWidget.IsValid())
	{
		ChildWidget = 
			SNew(STextBlock)
			.Text(InObjectType == ELevelSnapshotsObjectType::ObjectType_Snapshot ? 
				LOCTEXT("LevelSnapshotsEditorResults_NoSnapshotPropertyFound", "No snapshot property found") : 
				LOCTEXT("LevelSnapshotsEditorResults_NoWorldPropertyFound", "No World property found"));
	}

	ChildWidget->SetEnabled(bIsHeaderRow);
	ChildWidget->SetCanTick(bIsHeaderRow);

	TSharedPtr<SWidget> FinalValueWidget = SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2, 0))
		[
			ChildWidget.ToSharedRef()
		];

	if (!bIsChildWidgetCustomized)
	{
		FinalValueWidget = SNew(SInvalidationPanel)
			[
				FinalValueWidget.ToSharedRef()
			];
	}

	return FinalValueWidget.ToSharedRef();
}

const FSlateBrush* SLevelSnapshotsEditorResultsRow::GetBorderImage(const FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType InRowType)
{
	switch (InRowType)
	{							
		case FLevelSnapshotsEditorResultsRow::ModifiedActorGroup:
			return FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.ActorGroupBorder");

		case FLevelSnapshotsEditorResultsRow::AddedActorToRemove:
			return FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.ActorGroupBorder");

		case FLevelSnapshotsEditorResultsRow::RemovedActorToAdd:
			return FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.ActorGroupBorder");

		case FLevelSnapshotsEditorResultsRow::TreeViewHeader:
			return FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.HeaderRowBorder");

		default:
			return FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.DefaultBorder");
	}
}

float SLevelSnapshotsEditorResultsRow::GetNameColumnSize() const
{
	return 1.0f - CachedNestedColumnWidthAdjusted;
}

float SLevelSnapshotsEditorResultsRow::CalculateAndReturnNestedColumnSize()
{
	check(Item.IsValid());
	
	const TSharedPtr<FLevelSnapshotsEditorResultsRow>& PinnedItem = Item.Pin();
	const uint8 ChildDepth = PinnedItem->GetChildDepth();
	const float StartWidth = SplitterManagerPtr->NestedColumnWidth;

	if (ChildDepth > 0)
	{
		const float LocalPixelOffset = 10.0f;
		const float LocalPixelDifference = LocalPixelOffset * ChildDepth;
		const float LocalSizeX = GetTickSpaceGeometry().GetLocalSize().X;
		const float NestedItemCoefficient = (LocalSizeX + LocalPixelDifference) / LocalSizeX;
		
		CachedNestedColumnWidthAdjusted = StartWidth * NestedItemCoefficient;
		
		return CachedNestedColumnWidthAdjusted;
	}

	return CachedNestedColumnWidthAdjusted = StartWidth;
}

float SLevelSnapshotsEditorResultsRow::GetSnapshotColumnSize() const
{
	const float EndWidth = SplitterManagerPtr->SnapshotPropertyColumnWidth;
	return EndWidth;
}

float SLevelSnapshotsEditorResultsRow::GetWorldColumnSize() const
{
	return 1.0f - GetSnapshotColumnSize();
}

void SLevelSnapshotsEditorResultsRow::SetNestedColumnSize(const float InWidth) const
{
	SplitterManagerPtr->NestedColumnWidth = InWidth;
}

void SLevelSnapshotsEditorResultsRow::SetSnapshotColumnSize(const float InWidth) const
{
	SplitterManagerPtr->SnapshotPropertyColumnWidth = InWidth;
}

FReply SLevelSnapshotsEditorResultsRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (Item.IsValid() && Item.Pin()->GetRowType() == FLevelSnapshotsEditorResultsRow::ModifiedActorGroup)
	{		
		if (AActor* Actor = Cast<AActor>(Item.Pin()->GetWorldObject()))
		{
			const FLevelSnapshotsEditorModule& Module = FLevelSnapshotsEditorModule::Get();
				
			if (ULevelSnapshotsEditorSettings::Get()->bClickActorGroupToSelectActorInScene && GEditor)
			{
				GEditor->SelectNone(false, true, false);
				GEditor->SelectActor( Actor, true, true, true );
			}
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
