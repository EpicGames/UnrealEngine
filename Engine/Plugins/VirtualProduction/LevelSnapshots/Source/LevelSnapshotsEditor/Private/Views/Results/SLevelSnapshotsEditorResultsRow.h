// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSnapshotsEditorResultsRow.h"

#include "Widgets/SCompoundWidget.h"

struct FLevelSnapshotsEditorResultsSplitterManager;

class SSplitter;

typedef TSharedPtr<FLevelSnapshotsEditorResultsSplitterManager> FLevelSnapshotsEditorResultsSplitterManagerPtr;

class SLevelSnapshotsEditorResultsRow : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorResultsRow)
	{}

	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TWeakPtr<FLevelSnapshotsEditorResultsRow>& InRow, const FLevelSnapshotsEditorResultsSplitterManagerPtr& InSplitterManagerPtr);

	~SLevelSnapshotsEditorResultsRow();

	void GenerateAddedAndRemovedRowComponents(const TSharedRef<SHorizontalBox> BasicRowWidgets, const FLevelSnapshotsEditorResultsRowPtr PinnedItem, const FText& InDisplayText) const;

	TSharedRef<SWidget> GenerateFinalValueWidget(
	const ELevelSnapshotsObjectType InObjectType, FLevelSnapshotsEditorResultsRowPtr PinnedItem, const bool bIsHeaderRow, const bool bNeedsNullWidget) const;

	static const FSlateBrush* GetBorderImage(const FLevelSnapshotsEditorResultsRow::ELevelSnapshotsEditorResultsRowType InRowType);

	float GetNameColumnSize() const;

	float CalculateAndReturnNestedColumnSize();

	float GetSnapshotColumnSize() const;

	float GetWorldColumnSize() const;

	void SetNestedColumnSize(const float InWidth) const;

	void SetSnapshotColumnSize(const float InWidth) const;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	
	TSharedPtr<SSplitter> OuterSplitterPtr = nullptr;
	TSharedPtr<SSplitter> NestedSplitterPtr = nullptr;
	
	TWeakPtr<FLevelSnapshotsEditorResultsRow> Item = nullptr;

	/* For splitter sync */

	/* To sync up splitter location in tree view items, we need to account for the tree view's indentation.
	 * Instead of calculating the coefficient twice each frame (for left and right splitter slots), we do it once and cache it here. */
	float CachedNestedColumnWidthAdjusted = 0.f;
	
	FLevelSnapshotsEditorResultsSplitterManagerPtr SplitterManagerPtr;

	#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

	const FTextFormat AddTextFormat = FTextFormat(LOCTEXT("LevelSnapshots_AddFormat", "Restore {DisplayName} ({ObjectType}) to {NewParent}"));
	const FTextFormat RemoveTextFormat = FTextFormat(LOCTEXT("LevelSnapshots_RemoveFormat", "Remove {DisplayName} ({ObjectType}) from {CurrentParent}"));

	const FText ActorComponentText = LOCTEXT("LevelSnapshots_ActorComponent", "ActorComponent");
	const FText SceneComponentText = LOCTEXT("LevelSnapshots_SceneComponent", "SceneComponent");
	const FText SubobjectText = LOCTEXT("LevelSnapshots_Subobject", "Subobject");

	#undef LOCTEXT_NAMESPACE
};
