// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SLevelSnapshotsEditor.h"

#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshotsEditorModule.h"
#include "LevelSnapshotsEditorStyle.h"
#include "Util/TakeSnapshotUtil.h"
#include "Views/Input/SLevelSnapshotsEditorInput.h"
#include "Views/Filter/SLevelSnapshotsEditorFilters.h"
#include "Views/Results/SLevelSnapshotsEditorResults.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "Stats/StatsMisc.h"
#include "SPositiveActionButton.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshots"

const FName SLevelSnapshotsEditor::AppIdentifier(TEXT("LevelSnapshotsToolkit"));
const FName SLevelSnapshotsEditor::ToolbarTabId(TEXT("LevelsnapshotsToolkit_Toolbar"));
const FName SLevelSnapshotsEditor::InputTabID(TEXT("BaseAssetToolkit_Input"));
const FName SLevelSnapshotsEditor::FilterTabID(TEXT("BaseAssetToolkit_Filter"));
const FName SLevelSnapshotsEditor::ResultsTabID(TEXT("BaseAssetToolkit_Results"));

void SLevelSnapshotsEditor::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData, const TSharedRef<SDockTab>& ConstructUnderTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	check(InEditorData);
	
	EditorData = InEditorData;
	EditorData.Get()->ClearActiveSnapshot();

	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderTab);
	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("LevelSnapshotsMenuGroupName", "Level Snapshots"));
	TabManager->SetAllowWindowMenuBar(true);
	RegisterTabSpawners(TabManager.ToSharedRef(), AppMenuGroup);

	// Create our content
	TSharedRef<FTabManager::FLayout> Layout =
		FTabManager::NewLayout("Levelsnapshots_Layout_2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.1f)
				->AddTab(ToolbarTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
				)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->AddTab(InputTabID, ETabState::OpenedTab)
					->SetHideTabWell(true)
					)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->AddTab(FilterTabID, ETabState::OpenedTab)
					->SetHideTabWell(true)
					)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(ResultsTabID, ETabState::OpenedTab)
					->SetHideTabWell(true)
					)
				)
			);

	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);

	ChildSlot
	[
		TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
	];

	// on persist layout will handle saving layout if the editor is shut down:
	TabManager->SetOnPersistLayout(
		FTabManager::FOnPersistLayout::CreateLambda(
			[](const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				if (InLayout->GetPrimaryArea().Pin().IsValid())
				{
					FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
				}
			}
		)
	);
}

void SLevelSnapshotsEditor::OpenLevelSnapshotsDialogWithAssetSelected(const FAssetData& InAssetData) const
{
	EditorInputWidget->OpenLevelSnapshotsDialogWithAssetSelected(InAssetData);
}

void SLevelSnapshotsEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FWorkspaceItem>& AppMenuGroup)
{
	InTabManager->RegisterTabSpawner(ToolbarTabId, FOnSpawnTab::CreateSP(this, &SLevelSnapshotsEditor::SpawnTab_CustomToolbar))
        .SetDisplayName(LOCTEXT("ToolbarTab", "Toolbar"))
        .SetGroup(AppMenuGroup);
	
	InTabManager->RegisterTabSpawner(InputTabID, FOnSpawnTab::CreateSP(this, &SLevelSnapshotsEditor::SpawnTab_Input))
        .SetDisplayName(LOCTEXT("InputTab", "Input"))
        .SetGroup(AppMenuGroup);

	InTabManager->RegisterTabSpawner(FilterTabID, FOnSpawnTab::CreateSP(this, &SLevelSnapshotsEditor::SpawnTab_Filter))
        .SetDisplayName(LOCTEXT("Filter", "Filter"))
        .SetGroup(AppMenuGroup);

	InTabManager->RegisterTabSpawner(ResultsTabID, FOnSpawnTab::CreateSP(this, &SLevelSnapshotsEditor::SpawnTab_Results))
        .SetDisplayName(LOCTEXT("Result", "Result"))
        .SetGroup(AppMenuGroup);
}

TSharedRef<SDockTab> SLevelSnapshotsEditor::SpawnTab_CustomToolbar(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.CanEverClose(false)
        .Label(LOCTEXT("Levelsnapshots.Toolkit.ToolbarTitle", "Toolbar"))
		.ShouldAutosize(true)
        [
	        SNew(SBorder)
	        .Padding(0)
	        .BorderImage(FAppStyle::GetBrush("NoBorder"))
			.HAlign(HAlign_Fill)
	        [
				SNew(SHorizontalBox)

				// Show/Hide Input button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.f, 2.f)
				[
					SNew(SButton)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "ToggleButton")
					.ToolTipText(LOCTEXT("InputPanelToggleTooltip", "Show or hide the input panel"))
					.ContentPadding(FMargin(1, 0))
					.ForegroundColor(FSlateColor::UseForeground())
					.OnClicked(this, &SLevelSnapshotsEditor::OnClickInputPanelExpand)
					[
						SNew(SImage)
						.Image_Lambda([this] ()
						{
							return FAppStyle::GetBrush(bInputPanelExpanded ? "ContentBrowser.HideSourcesView" : "ContentBrowser.ShowSourcesView");
						})
					]
				]
				
				// Take snapshot
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(2.f, 2.f)
				[
					SNew(SPositiveActionButton)
					.OnClicked(this, &SLevelSnapshotsEditor::OnClickTakeSnapshot)
					.Text(LOCTEXT("TakeSnapshot", "Take Snapshot"))
				]

				// Open Settings
				+ SHorizontalBox::Slot()
                .HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
                [
					SNew(SBox)
					.WidthOverride(28)
					.HeightOverride(28)
					[
						SAssignNew(SettingsButtonPtr, SCheckBox)
						.Padding(FMargin(4.f))
						.ToolTipText(LOCTEXT("ShowSettings_Tip", "Show the general user/project settings for Level Snapshots"))
						.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
						.ForegroundColor(FSlateColor::UseForeground())
						.IsChecked(false)
						.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckState)
						{
							FLevelSnapshotsEditorModule::OpenLevelSnapshotsSettings();
							SettingsButtonPtr->SetIsChecked(false);
						})
		                [
			                SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
		                ]
					]
                ]
        	]
        ];
}

TSharedRef<SDockTab> SLevelSnapshotsEditor::SpawnTab_Input(const FSpawnTabArgs& Args)
{	
	check(EditorData.IsValid());
	
	InputTab = SNew(SDockTab)
		.CanEverClose(false)
		.Label(LOCTEXT("Levelsnapshots.Toolkit.InputTitle", "Input"))
		[
			SAssignNew(EditorInputWidget, SLevelSnapshotsEditorInput, EditorData.Get())
		];

	return InputTab.ToSharedRef();
}

TSharedRef<SDockTab> SLevelSnapshotsEditor::SpawnTab_Filter(const FSpawnTabArgs& Args)
{
	check(EditorData.IsValid());
	
	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.CanEverClose(false)
		.Label(LOCTEXT("Levelsnapshots.Toolkit.FilterTitle", "Filter"))
		[
			SNew(SLevelSnapshotsEditorFilters, EditorData.Get())
		];

	return DetailsTab.ToSharedRef();
}

TSharedRef<SDockTab> SLevelSnapshotsEditor::SpawnTab_Results(const FSpawnTabArgs& Args)
{
	check(EditorData.IsValid());
	
	TSharedPtr<SDockTab> ResultsTab = SNew(SDockTab)
		.CanEverClose(false)
		.Label(LOCTEXT("Levelsnapshots.Toolkit.ResultTitle", "Result"))
		[
			SNew(SLevelSnapshotsEditorResults, EditorData.Get())
		];

	return ResultsTab.ToSharedRef();
}

FReply SLevelSnapshotsEditor::OnClickTakeSnapshot()
{
	SnapshotEditor::TakeSnapshotWithOptionalForm();
	return FReply::Handled();
}

FReply SLevelSnapshotsEditor::OnClickInputPanelExpand()
{
	bInputPanelExpanded = !bInputPanelExpanded;

	if (InputTab.IsValid() && EditorInputWidget.IsValid())
	{
		InputTab->SetShouldAutosize(!bInputPanelExpanded);
		EditorInputWidget->SetVisibility(bInputPanelExpanded ? EVisibility::Visible : EVisibility::Collapsed);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
