// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Toolkits/AssetEditorToolkit.h"

class SLevelSnapshotsEditorInput;
class FLevelSnapshotsEditorInput;
class FLevelSnapshotsEditorFilters;
class FLevelSnapshotsEditorResults;
class FLevelSnapshotsEditorContext;
class SCheckBox;
class SDockTab;
class ULevelSnapshotsEditorData;

struct FSnapshotEditorViewData;

class ULevelSnapshot;

class SLevelSnapshotsEditor
	:
	public SCompoundWidget
{
	using Super = SCompoundWidget;

	static const FName AppIdentifier;
	static const FName ToolbarTabId;
	static const FName InputTabID;
	static const FName FilterTabID;
	static const FName ResultsTabID;
public:

	SLATE_BEGIN_ARGS(SLevelSnapshotsEditor) { }
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData, const TSharedRef<SDockTab>& ConstructUnderTab, const TSharedPtr<SWindow>& ConstructUnderWindow);
	
	void OpenLevelSnapshotsDialogWithAssetSelected(const FAssetData& InAssetData) const;

private:

	void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager, const TSharedRef<FWorkspaceItem>& AppMenuGroup);

	TSharedRef<SDockTab> SpawnTab_CustomToolbar(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Input(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Filter(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Results(const FSpawnTabArgs& Args);

	FReply OnClickTakeSnapshot();
	FReply OnClickInputPanelExpand();
	
	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData;

	TSharedPtr<SLevelSnapshotsEditorInput> EditorInputWidget;

	/** Holds the tab manager that manages the tabs */
	TSharedPtr<FTabManager> TabManager;

	TSharedPtr<SCheckBox> SettingsButtonPtr;

	TSharedPtr<SDockTab> InputTab;
	bool bInputPanelExpanded = true;
};
