// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Filter/SCreateNewFilterWidget.h"

#include "Data/Filters/ConjunctionFilter.h"
#include "Data/FavoriteFilterContainer.h"
#include "Widgets/Filter/SFilterSearchMenu.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SCreateNewFilterWidget::Construct(const FArguments& InArgs, UFavoriteFilterContainer* InAvailableFilters, UConjunctionFilter* InFilterToAddTo)
{
	if (!ensure(InAvailableFilters && InFilterToAddTo))
	{
		return;
	}
	
	AvailableFilters = InAvailableFilters;
	FilterToAddTo = InFilterToAddTo;

	ChildSlot
	[
		SAssignNew(ComboButton, SComboButton)
		.ContentPadding(0)
		.ToolTipText( LOCTEXT( "SelectFilterToUseToolTip", "Select filters you want to use." ) )
		.OnGetMenuContent(FOnGetContent::CreateLambda([this, InAvailableFilters, InFilterToAddTo]()
		{
			const TSharedRef<SFilterSearchMenu> Result = SNew(SFilterSearchMenu, InAvailableFilters)
				.OnSelectFilter_Lambda([this, InFilterToAddTo](const TSubclassOf<ULevelSnapshotFilter>& SelectedFilter)
				{
					InFilterToAddTo->CreateChild(SelectedFilter);
				});
			ComboButton->SetMenuContentWidgetToFocus(Result->GetSearchBox());
			return Result;
		}))
		.HasDownArrow( true )
		.ContentPadding( FMargin( 1, 0 ) )
		.Visibility(EVisibility::Visible)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AddFilter", "Add filter"))
		]
	];
}

#undef LOCTEXT_NAMESPACE