// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UConjunctionFilter;
class UFavoriteFilterContainer;
class SComboButton;

/* Put into rows and allows user to create to pick a filter to add to the row. */
class SCreateNewFilterWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SCreateNewFilterWidget)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UFavoriteFilterContainer* InAvailableFilters, UConjunctionFilter* InFilterToAddTo);

private:

	TWeakObjectPtr<UFavoriteFilterContainer> AvailableFilters;
	TWeakObjectPtr<UConjunctionFilter> FilterToAddTo;

	TSharedPtr<SComboButton> ComboButton;
};
