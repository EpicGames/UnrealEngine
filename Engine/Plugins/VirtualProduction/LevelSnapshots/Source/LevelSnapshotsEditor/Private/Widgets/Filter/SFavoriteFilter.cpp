// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Filter/SFavoriteFilter.h"

#include "Data/DragDrop/FavoriteFilterDragDrop.h"

#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

void SFavoriteFilter::Construct(const FArguments& InArgs, const TSubclassOf<ULevelSnapshotFilter>& InFilterClass, const TWeakObjectPtr<ULevelSnapshotsEditorData>& InEditorData)
{
	if (!ensure(InFilterClass.Get()))
	{
		return;
	}
	FilterClass = InFilterClass;
	DragDropActiveFilterSetterArgument = InEditorData;
	
	ChildSlot
	[
		SNew(SBorder)
		.Padding(2.f)
		.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 1))
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.FilterBackground"))
		.Padding(3.f)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Font(FAppStyle::GetFontStyle("ContentBrowser.FilterNameFont"))
				.ShadowOffset(FVector2D(1.f, 1.f))
				.Text(InArgs._FilterName)
		]
	];
}

FReply SFavoriteFilter::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SFavoriteFilter::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (ensure(DragDropActiveFilterSetterArgument.IsValid()))
	{
		return FReply::Handled().BeginDragDrop(MakeShared<FFavoriteFilterDragDrop>(FilterClass, DragDropActiveFilterSetterArgument.Get()));
	}
	return FReply::Handled();
}
