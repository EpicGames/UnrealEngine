// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Data/LevelSnapshotsEditorData.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Templates/SubclassOf.h"

class FLevelSnapshotsEditorFilters;

/**
 * Display in the favorite filter list.
 */
class SFavoriteFilter : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SFavoriteFilter)
	{}
		SLATE_ATTRIBUTE(FText, FilterName)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSubclassOf<ULevelSnapshotFilter>& InFilterClass, const TWeakObjectPtr<ULevelSnapshotsEditorData>& InEditorData);

	//~ Begin SWidget Interface
	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget Interface

private:

	TSubclassOf<ULevelSnapshotFilter> FilterClass;
	/* Passed to drag drop operation so it can set the filter being edited. */
	TWeakObjectPtr<ULevelSnapshotsEditorData> DragDropActiveFilterSetterArgument;
};
