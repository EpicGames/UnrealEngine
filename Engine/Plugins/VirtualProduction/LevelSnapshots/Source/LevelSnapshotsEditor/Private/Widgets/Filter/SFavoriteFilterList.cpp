// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFavoriteFilterList.h"


#include "Styling/AppStyle.h"
#include "LevelSnapshotFilters.h"
#include "LevelSnapshotsEditorStyle.h"
#include "SFavoriteFilter.h"
#include "SFilterSearchMenu.h"
#include "Widgets/Input/SComboButton.h"

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"


#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

namespace
{
	void OnSelectFilter(const TSubclassOf<ULevelSnapshotFilter>& SelectFilterClass, TWeakObjectPtr<UFavoriteFilterContainer> Filters)
	{
		if (ensure(Filters.IsValid()))
		{
			const bool bIsFavorite = Filters->GetFavorites().Contains(SelectFilterClass);
			if (bIsFavorite)
			{
				Filters->RemoveFromFavorites(SelectFilterClass);
			}
			else
			{
				Filters->AddToFavorites(SelectFilterClass);
			}
		}
	}
	
	bool IsFilterSelected(const TSubclassOf<ULevelSnapshotFilter>& FilterClass, TWeakObjectPtr<UFavoriteFilterContainer> Filters)
	{
		if (ensure(Filters.IsValid()))
		{
			return Filters->GetFavorites().Contains(FilterClass);
		}
		return false;
	};
	
	void SetIsCategorySelected(FName CategoryName, bool bNewIsCategorySelected, TWeakObjectPtr<UFavoriteFilterContainer> Filters)
	{
		if (ensure(Filters.IsValid()))
		{
			Filters->SetShouldIncludeAllClassesInCategory(CategoryName, bNewIsCategorySelected);
		}
	};
	
	bool IsCategorySelected(FName CategoryName, TWeakObjectPtr<UFavoriteFilterContainer> Filters) 
	{
		if (ensure(Filters.IsValid()))
		{
			return Filters->ShouldIncludeAllClassesInCategory(CategoryName);
		}
		return false;
	};
}

SFavoriteFilterList::~SFavoriteFilterList()
{
	if (FavoriteModel.IsValid())
	{
		FavoriteModel->OnFavoritesChanged.Remove(ChangedFavoritesDelegateHandle);
	}
}

void SFavoriteFilterList::Construct(const FArguments& InArgs, UFavoriteFilterContainer* InModel, ULevelSnapshotsEditorData* InEditorData)
{
	if (!ensure(InModel))
	{
		return;
	}
	
	FavoriteModel = InModel;
	ChangedFavoritesDelegateHandle = InModel->OnFavoritesChanged.AddSP(this, &SFavoriteFilterList::UpdateFilterList, InEditorData);
	
	ChildSlot
	[
		SNew(SBorder)
			.Padding(FMargin(5.0f, 8.0f))
			.BorderImage(FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.GroupBorder"))
		[
			SNew(SVerticalBox)

		    // Filter
		    + SVerticalBox::Slot() 
		    .Padding(0.f, 0.f)
		    .HAlign(HAlign_Left)
		    .AutoHeight()
		    [
		        SNew(SHorizontalBox)

		        + SHorizontalBox::Slot()
		        .AutoWidth()
		        [
		            SAssignNew(ComboButton, SComboButton)
		            .ComboButtonStyle( FAppStyle::Get(), "GenericFilters.ComboButtonStyle" )
		            .ForegroundColor(FLinearColor::White)
		            .ContentPadding(0)
		            .ToolTipText( LOCTEXT( "SelectFilterToUseToolTip", "Select filters you want to use." ) )
		            .OnGetMenuContent_Lambda([this, Filters = TWeakObjectPtr<UFavoriteFilterContainer>(InModel)]() -> TSharedRef<SWidget>
		            {
						if (ensure(Filters.IsValid()))
						{
							const TSharedRef<SFilterSearchMenu> Result = SNew(SFilterSearchMenu, Filters.Get())
	                            .OnSelectFilter_Static(OnSelectFilter, Filters)
	                            .OptionalIsFilterChecked_Static(IsFilterSelected, Filters)
	                            .OptionalSetIsFilterCategorySelected_Static(SetIsCategorySelected, Filters)
	                            .OptionalIsFilterCategorySelected_Static(IsCategorySelected, Filters);
							
	        				ComboButton->SetMenuContentWidgetToFocus(Result->GetSearchBox());
	        				return Result;
						}
		            	return SNullWidget::NullWidget;
		            })
		            .ContentPadding( FMargin( 1, 0 ) )
		            .Visibility(EVisibility::Visible)
		            .ButtonContent()
		            [
		                SNew(SHorizontalBox)

		                + SHorizontalBox::Slot()
		                .AutoWidth()
		                [
		                    SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
						]

		                + SHorizontalBox::Slot()
		                .AutoWidth()
						.Padding(2, 0, 0, 0)
		                [
		                    SNew(STextBlock)
		                    .TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
		                    .Text(LOCTEXT("FavoriteFilters", "Favorite filters"))
		                ]
		            ]
		        ]
		    ]

		    // Store favorites filters
		    + SVerticalBox::Slot()
		    .Padding(2.f, 2.f)
		    .AutoHeight()
		    [
		        SAssignNew(FilterList, SWrapBox)
		        .UseAllottedSize(true)
		    ]
		]
	];

	UpdateFilterList(InEditorData);
}

void SFavoriteFilterList::UpdateFilterList(ULevelSnapshotsEditorData* InEditorData)
{
	if (ensure(FavoriteModel.IsValid()) && ensure(FilterList.IsValid()))
	{
		FilterList->ClearChildren();
				
		const TArray<TSubclassOf<ULevelSnapshotFilter>>& FavoriteFilters = FavoriteModel->GetFavorites();
		for (const TSubclassOf<ULevelSnapshotFilter>& FavoriteFilter : FavoriteFilters)
		{
			FilterList->AddSlot()
				.Padding(3.f, 3.f)
				[
					SNew(SFavoriteFilter, FavoriteFilter, InEditorData)
						.FilterName(FavoriteFilter->GetDisplayNameText())
				];
		}
	}
}

#undef LOCTEXT_NAMESPACE
