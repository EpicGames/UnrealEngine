// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "Data/FavoriteFilterContainer.h"

class FLevelSnapshotsEditorFilters;
class UFavoriteFilterContainer;
class ULevelSnapshotsEditorData;
class SComboButton;
class SWrapBox;

class SFavoriteFilterList : public SCompoundWidget
{
public:

	~SFavoriteFilterList();
	
	SLATE_BEGIN_ARGS(SFavoriteFilterList)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UFavoriteFilterContainer* InModel, ULevelSnapshotsEditorData* InEditorData);

private:

	void UpdateFilterList(ULevelSnapshotsEditorData* InEditorData);

	TSharedPtr<SWrapBox> FilterList;
	TSharedPtr<SComboButton> ComboButton;

	TWeakObjectPtr<UFavoriteFilterContainer> FavoriteModel;
	FDelegateHandle ChangedFavoritesDelegateHandle;
};
