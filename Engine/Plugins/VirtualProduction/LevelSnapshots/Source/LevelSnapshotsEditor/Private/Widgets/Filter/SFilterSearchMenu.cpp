// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Filter/SFilterSearchMenu.h"

#include "Data/FavoriteFilterContainer.h"
#include "LevelSnapshotFilters.h"

#include "Algo/Transform.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

namespace
{
	TSharedRef<SWidget> MakeFilterMenu(
		UFavoriteFilterContainer* FilterModel,
		const SFilterSearchMenu::FFilterDelegate& OnSelectFilter,
		TOptional<SFilterSearchMenu::FIsFilterSelected> IsFilterSelected,
		TOptional<SFilterSearchMenu::FSetIsCategorySelected> SetIsCategorySelected,
		TOptional<SFilterSearchMenu::FIsCategorySelected> IsCategorySelected)
	{
		class FFilterMenuBuilder
		{
			FMenuBuilder MenuBuilder = FMenuBuilder(true, nullptr, nullptr, true, &FAppStyle::Get(), false);
			UFavoriteFilterContainer* FilterModel;
			
			const SFilterSearchMenu::FFilterDelegate OnSelectFilter;
			const TOptional<SFilterSearchMenu::FIsFilterSelected> IsFilterSelected;
			const TOptional<SFilterSearchMenu::FSetIsCategorySelected> SetIsCategorySelected;
			const TOptional<SFilterSearchMenu::FIsCategorySelected> IsCategorySelected;

			static void CreateMenuEntry(FMenuBuilder& MenuBuilder, const TSubclassOf<ULevelSnapshotFilter>& FilterClass, const SFilterSearchMenu::FFilterDelegate& OnSelectFilter, TOptional<SFilterSearchMenu::FIsFilterSelected> IsFilterSelected)
			{
				check(FilterClass.Get());
		
				FUIAction AddFilterAction;
				AddFilterAction.ExecuteAction.BindLambda([FilterClass, OnSelectFilter]()
                {
                    OnSelectFilter.Execute(FilterClass);
                });

				const bool bShouldShowCheckbox = IsFilterSelected.IsSet();
				if (bShouldShowCheckbox)
				{
					AddFilterAction.GetActionCheckState.BindLambda([FilterClass, IsFilterSelected = IsFilterSelected.GetValue()]() -> ECheckBoxState
                    {
                        return IsFilterSelected.Execute(FilterClass) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
                    });
				}
		
				MenuBuilder.AddMenuEntry(
                    FilterClass->GetDisplayNameText(),
                    FilterClass->GetToolTipText(),
                    FSlateIcon(),
                    AddFilterAction,
                    NAME_None,
                    bShouldShowCheckbox ? EUserInterfaceActionType::ToggleButton : EUserInterfaceActionType::Button
                );
			}

			static void BuildSubmenu(
				FMenuBuilder& SubmenuBuilder,
				FName CategoryName,
				TWeakObjectPtr<UFavoriteFilterContainer> FilterModel,
				SFilterSearchMenu::FFilterDelegate OnSelectFilter,
				TOptional<SFilterSearchMenu::FIsFilterSelected> IsFilterSelected)
			{
				if (!ensure(FilterModel.IsValid()))
				{
					return;
				}

				TArray<TSubclassOf<ULevelSnapshotFilter>> FiltersInCategory = FilterModel->GetFiltersInCategory(CategoryName);;
				for (const TSubclassOf<ULevelSnapshotFilter>& FilterClass : FiltersInCategory)
				{
					CreateMenuEntry(SubmenuBuilder, FilterClass, OnSelectFilter, IsFilterSelected);
				}
			}
			
			static void OnClickSubmenu(FName CategoryName, SFilterSearchMenu::FIsCategorySelected IsCategorySelected, SFilterSearchMenu::FSetIsCategorySelected SetIsCategorySelected)
			{
				const bool bIsSelected = IsCategorySelected.Execute(CategoryName);
				SetIsCategorySelected.Execute(
					CategoryName,
					!bIsSelected
					);
			}

			static ECheckBoxState GetSubmenuCheckState(FName CategoryName, SFilterSearchMenu::FIsCategorySelected IsCategorySelected)
			{
				return IsCategorySelected.Execute(CategoryName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		
		public:

			FFilterMenuBuilder(
				UFavoriteFilterContainer* InFilterModel, 
                SFilterSearchMenu::FFilterDelegate OnSelectFilter,
                TOptional<SFilterSearchMenu::FIsFilterSelected> IsFilterSelected,
                TOptional<SFilterSearchMenu::FSetIsCategorySelected> SetIsCategorySelected,
                TOptional<SFilterSearchMenu::FIsCategorySelected> IsCategorySelected)
				:
				FilterModel(InFilterModel),
				OnSelectFilter(OnSelectFilter),
				IsFilterSelected(IsFilterSelected),
				SetIsCategorySelected(SetIsCategorySelected),
				IsCategorySelected(IsCategorySelected)
			{}

			FFilterMenuBuilder& Build_CommonFilters()
			{
				MenuBuilder.BeginSection("CommonFilters", LOCTEXT("FilterMenu.CommonFilters", "Common Filters"));
				for (const TSubclassOf<ULevelSnapshotFilter>& CommonFilter : FilterModel->GetCommonFilters())
				{
					CreateMenuEntry(MenuBuilder, CommonFilter, OnSelectFilter, IsFilterSelected);
				}
				MenuBuilder.EndSection();
				
				return *this;
			}
			
			FFilterMenuBuilder& Build_FilterCategories()
			{
				MenuBuilder.BeginSection("FilterCategories", LOCTEXT("FilterMenu.FilterCategories", "Categories"));
				{
					const bool bDisplaySubmenuAsCheckbox = SetIsCategorySelected.IsSet() && IsCategorySelected.IsSet();
					
					const TArray<FName> FilterCategories = FilterModel->GetCategories();
					for (const FName CategoryName : FilterCategories)
					{
						const FUIAction SubmenuAction = [this, CategoryName, bDisplaySubmenuAsCheckbox]()
						{
							if (bDisplaySubmenuAsCheckbox)
							{
								return FUIAction(
                                    FExecuteAction::CreateStatic(OnClickSubmenu, CategoryName, *IsCategorySelected, *SetIsCategorySelected),
                                    FCanExecuteAction(),
                                    FGetActionCheckState::CreateStatic(GetSubmenuCheckState, CategoryName, *IsCategorySelected)
                                    );
							}
							return FUIAction();
						}();
						
						MenuBuilder.AddSubMenu(
							FilterModel->CategoryNameToText(CategoryName),
								FText(),
								FNewMenuDelegate::CreateStatic(BuildSubmenu, CategoryName, TWeakObjectPtr<UFavoriteFilterContainer>(FilterModel), OnSelectFilter, IsFilterSelected),
								SubmenuAction,
								NAME_None,
								bDisplaySubmenuAsCheckbox ? EUserInterfaceActionType::ToggleButton : EUserInterfaceActionType::Button
							);
					}
				}
				MenuBuilder.EndSection();
				return *this;
			}

			TSharedRef<SWidget> MakeWidget()
			{
				return MenuBuilder.MakeWidget();
			}
		};
		
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				FFilterMenuBuilder(FilterModel, OnSelectFilter, IsFilterSelected, SetIsCategorySelected, IsCategorySelected)
					.Build_CommonFilters()
					.Build_FilterCategories()
					.MakeWidget()
			];
	}
}

void SFilterSearchMenu::Construct(const FArguments& InArgs, UFavoriteFilterContainer* InFavoriteFilters)
{
	if (!ensure(InArgs._OnSelectFilter.IsBound()))
	{
		return;
	}
	ensureMsgf(InArgs._OptionalSetIsFilterCategorySelected.IsBound() == InArgs._OptionalIsFilterCategorySelected.IsBound(), TEXT("If you pass either SetIsFilterCategorySelected or IsFilterCategorySelected, the respective other must be passed in as well."));
	
	OnSelectFilter = InArgs._OnSelectFilter;

	SAssignNew(SearchBox, SSearchBox)
		.OnTextChanged_Raw(this, &SFilterSearchMenu::OnSearchChangedString)
		.OnTextCommitted_Raw(this, &SFilterSearchMenu::OnSearchCommited)
	;
	SAssignNew(SearchResultsWidget, SListView< TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> >)
        .ListItemsSource(&FilteredOptions)
		.OnMouseButtonClick(this, &SFilterSearchMenu::OnClickItem)
		.OnSelectionChanged(this, &SFilterSearchMenu::OnItemSelected)
		.OnKeyDownHandler(this, &SFilterSearchMenu::OnHandleListViewKeyDown)
        .OnGenerateRow(this, &SFilterSearchMenu::GenerateFilterClassWidget)
        .SelectionMode(ESelectionMode::Single);

	// Show unsearched menu by default
	UnsearchedMenu = MakeFilterMenu( 
		InFavoriteFilters, 
		OnSelectFilter,
		InArgs._OptionalIsFilterChecked.IsBound() ? InArgs._OptionalIsFilterChecked : TOptional<FIsFilterSelected>(),
		InArgs._OptionalSetIsFilterCategorySelected.IsBound() ? InArgs._OptionalSetIsFilterCategorySelected : TOptional<FSetIsCategorySelected>(),
		InArgs._OptionalIsFilterCategorySelected.IsBound() ? InArgs._OptionalIsFilterCategorySelected : TOptional<FIsCategorySelected>()
		);


	FMenuBuilder BaseMenuBuilder(true, nullptr, nullptr, false, &FAppStyle::Get(), false);
	BaseMenuBuilder.AddWidget(SearchBox.ToSharedRef(), FText(), true, false);
	BaseMenuBuilder.AddWidget(SAssignNew(MenuContainer, SHorizontalBox), FText(), true, false);
	ChildSlot
	[
		BaseMenuBuilder.MakeWidget()
	];
	
	UnfilteredOptions = InFavoriteFilters->GetAllAvailableFilters();
	ShowUnsearchedMenu();
	ResetOptions();
}

TSharedPtr<SSearchBox> SFilterSearchMenu::GetSearchBox() const
{
	return SearchBox;
}

void SFilterSearchMenu::ResetOptions()
{
	SelectedItem.Reset();
	FilteredOptions.Empty();
	Algo::Transform(UnfilteredOptions, FilteredOptions, [](const TSubclassOf<ULevelSnapshotFilter>& Item) { return MakeShared<TSubclassOf<ULevelSnapshotFilter>>(Item); });
}

void SFilterSearchMenu::OnSearchChangedString(const FText& SearchText)
{
	HighlightText = SearchText;
	ResetOptions();
	
	if (SearchText.IsEmpty())
	{
		ShowUnsearchedMenu();
		return;
	}
	ShowSearchResults();
	
	const FString SearchString = SearchText.ToString();
	for (int32 OptionIndex = 0; OptionIndex < FilteredOptions.Num(); ++OptionIndex)
	{
		const TSharedPtr<TSubclassOf<ULevelSnapshotFilter>>& FilterClass = FilteredOptions[OptionIndex];

		const bool bIsSearchTextContainedInClass = FilterClass->Get()->GetDisplayNameText().ToString().Contains(SearchString, ESearchCase::IgnoreCase);
		if (!bIsSearchTextContainedInClass)
		{
			FilteredOptions.RemoveAt(OptionIndex);
			--OptionIndex;
		}
	}

	if (FilteredOptions.Num() >= 1)
	{
		SelectedItem = FilteredOptions[0];
		SearchResultsWidget->SetItemSelection(SelectedItem, true);
	}
	else if (SelectedItem.IsValid())
	{
		SearchResultsWidget->SetItemSelection(SelectedItem, false);
		SelectedItem.Reset();
	}
	
	SearchResultsWidget->RequestListRefresh();
}

void SFilterSearchMenu::OnSearchCommited(const FText& SearchText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter && SelectedItem.IsValid())
	{
		OnUserConfirmedItemSelection(SelectedItem);
	}
}

void SFilterSearchMenu::ShowUnsearchedMenu()
{
	MenuContainer->ClearChildren();
	MenuContainer->AddSlot()
    [
        UnsearchedMenu.ToSharedRef()
    ];
}

void SFilterSearchMenu::ShowSearchResults()
{
	MenuContainer->ClearChildren();
	MenuContainer->AddSlot()
    [
        SearchResultsWidget.ToSharedRef()
    ];
}

void SFilterSearchMenu::OnClickItem(const TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> Item)
{
	OnUserConfirmedItemSelection(Item);
}

void SFilterSearchMenu::OnItemSelected(const TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> Item, ESelectInfo::Type SelectInfo)
{
	SelectedItem = Item;
	
	if (Item.IsValid() && SelectInfo == ESelectInfo::OnKeyPress)
	{
		OnUserConfirmedItemSelection(Item);
	}
}

FReply SFilterSearchMenu::OnHandleListViewKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
{
	if (KeyEvent.GetKey() == EKeys::Enter && SelectedItem.IsValid())
	{
		OnUserConfirmedItemSelection(SelectedItem);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

TSharedRef<ITableRow> SFilterSearchMenu::GenerateFilterClassWidget(TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow< TSharedPtr<FName> >, OwnerTable)
        .ShowSelection(true)
        .Content()
        [
            SNew(STextBlock)
				.Text(InItem->Get()->GetDisplayNameText())
				.HighlightText(HighlightText)
        ];
}

void SFilterSearchMenu::OnUserConfirmedItemSelection(TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> Item)
{
	OnSelectFilter.Execute(Item->Get());
	
	const TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow( AsShared() ).ToSharedRef();
	FSlateApplication::Get().RequestDestroyWindow( ParentContextMenuWindow );
}

#undef LOCTEXT_NAMESPACE
 
