// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"

class UFavoriteFilterContainer;
class ITableRow;
class ULevelSnapshotFilter;
class SHorizontalBox;
class SSearchBox;
class STableViewBase;

class SFilterSearchMenu : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FFilterDelegate, const TSubclassOf<ULevelSnapshotFilter>&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsFilterSelected, const TSubclassOf<ULevelSnapshotFilter>&);
	DECLARE_DELEGATE_TwoParams(FSetIsCategorySelected, FName /*CategoryName*/, bool /* bNewIsSelected*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsCategorySelected, FName /*CategoryName*/);
	
	
	SLATE_BEGIN_ARGS(SFilterSearchMenu)
	{}
	SLATE_EVENT(FFilterDelegate, OnSelectFilter)
	SLATE_EVENT(FIsFilterSelected, OptionalIsFilterChecked)
	SLATE_EVENT(FSetIsCategorySelected, OptionalSetIsFilterCategorySelected)
	SLATE_EVENT(FIsCategorySelected, OptionalIsFilterCategorySelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UFavoriteFilterContainer* InFavoriteFilters);

	TSharedPtr<SSearchBox> GetSearchBox() const;
	
private:

	void ResetOptions();
	
	void OnSearchChangedString(const FText& SearchText);
	void OnSearchCommited(const FText& SearchText, ETextCommit::Type CommitType);

	void ShowUnsearchedMenu();
	void ShowSearchResults();

	void OnClickItem(const TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> Item);
	void OnItemSelected(const TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> Item, ESelectInfo::Type SelectInfo);
	FReply OnHandleListViewKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent);
	TSharedRef<ITableRow> GenerateFilterClassWidget(TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/* Calls delegate and closes this menu */
	void OnUserConfirmedItemSelection(TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> Item);

	
	/******************** Passed in data ********************/

	
	/* Called when the user selects a filter */
	FFilterDelegate OnSelectFilter;

	/* Used to get available filters */
	TWeakObjectPtr<UFavoriteFilterContainer> FilterContainer;

	
	
	/******************** Widgets ********************/
	
	/* Searches filters */
	TSharedPtr<SSearchBox> SearchBox;
	/* Dummy widget which holds either UnsearchedMenu or the results of the search.  */
	TSharedPtr<SHorizontalBox> MenuContainer;
	
	/* Unfiltered menu with submenu of filters. */
	TSharedPtr<SWidget> UnsearchedMenu;
	/* Filtered menu with all filters matching the search. */
	TSharedPtr< SListView< TSharedPtr< TSubclassOf<ULevelSnapshotFilter> > > > SearchResultsWidget;

	

	/******************** Filtering data ********************/
	
	TArray< TSharedPtr< TSubclassOf<ULevelSnapshotFilter> > > FilteredOptions;
	TArray<TSubclassOf<ULevelSnapshotFilter>> UnfilteredOptions;
	
	TSharedPtr<TSubclassOf<ULevelSnapshotFilter>> SelectedItem;
	FText HighlightText;
};
