// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHoverableFilterActions.h"

#include "Styling/AppStyle.h"
#include "LevelSnapshotsEditorStyle.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SHoverableFilterActions::Construct(FArguments InArgs, TWeakPtr<SWidget> InHoverOwner)
{
	IsFilterIgnored = InArgs._IsFilterIgnored;
	OnChangeFilterIgnored = InArgs._OnChangeFilterIgnored;
	OnPressDelete = InArgs._OnPressDelete;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.IgnoreFilterBorder"))
		.BorderBackgroundColor(InArgs._BackgroundHoverColor)
		.Visibility_Lambda([InHoverOwner]() { return InHoverOwner.Pin()->IsHovered() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden; })
		[
			SNew(SHorizontalBox)
			.Visibility(EVisibility::SelfHitTestInvisible)

			// Shift it as far to the right as possible
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.FillWidth(1.f)

		
			// Checkbox
			+ SHorizontalBox::Slot()
			.Padding(4.f, 0.f)
			.AutoWidth()
			[
			SNew(SCheckBox)
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState){ OnChangeFilterIgnored.ExecuteIfBound(NewState != ECheckBoxState::Checked); })
				.IsChecked_Lambda([this](){ return IsFilterIgnored.Execute() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked; })
				.ToolTipText_Lambda([this](){ return IsFilterIgnored.Execute() ? LOCTEXT("FilterRow.Ignored", "Enable filter. Filter is ignored.") : LOCTEXT("FilterRow.NotIgnored", "Ignore filter. Filter is enabled."); })
			]
				
			// Remove Button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked_Lambda([this]()
				{
					OnPressDelete.ExecuteIfBound();
					return FReply::Handled();
				})
				.ButtonStyle(FLevelSnapshotsEditorStyle::Get(), "LevelSnapshotsEditor.RemoveFilterButton")
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
