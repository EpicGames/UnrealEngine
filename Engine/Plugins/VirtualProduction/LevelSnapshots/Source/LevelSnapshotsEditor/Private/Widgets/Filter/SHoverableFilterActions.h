// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/* Shows a checkbox for ignoring and a delete button */
class SHoverableFilterActions : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal(bool, FIsFilterIgnored);
	DECLARE_DELEGATE_OneParam(FOnChangeFilterIgnored, bool /* bNewIsIgnored */);
	DECLARE_DELEGATE(FOnPressDelete);
	
	SLATE_BEGIN_ARGS(SHoverableFilterActions)
		:
		_BackgroundHoverColor(FLinearColor(0.f, 0.f, 0.f, 0.f))
	{}
		SLATE_EVENT(FIsFilterIgnored, IsFilterIgnored)
		SLATE_EVENT(FOnChangeFilterIgnored, OnChangeFilterIgnored)
		SLATE_EVENT(FOnPressDelete, OnPressDelete)
		SLATE_ATTRIBUTE(FSlateColor, BackgroundHoverColor)
	SLATE_END_ARGS()

	void Construct(FArguments InArgs, TWeakPtr<SWidget> InHoverOwner);

private:

	FIsFilterIgnored IsFilterIgnored;
	FOnChangeFilterIgnored OnChangeFilterIgnored;
	FOnPressDelete OnPressDelete;
};
