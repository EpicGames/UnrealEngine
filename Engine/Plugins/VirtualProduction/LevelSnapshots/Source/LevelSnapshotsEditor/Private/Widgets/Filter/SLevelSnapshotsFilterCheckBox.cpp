// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Filter/SLevelSnapshotsFilterCheckBox.h"

#include "Styling/AppStyle.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"

void SLevelSnapshotsFilterCheckBox::Construct(const FArguments& Args)
{
	OnFilterCtrlClicked = Args._OnFilterCtrlClicked;
	OnFilterAltClicked = Args._OnFilterAltClicked;
	OnFilterMiddleButtonClicked = Args._OnFilterMiddleButtonClicked;
	OnFilterRightButtonClicked = Args._OnFilterRightButtonClicked;
	OnFilterClickedOnce = Args._OnFilterClickedOnce;

	TSharedPtr<SImage> Image;
	
	ChildSlot
    [
        SNew(SHorizontalBox)

        + SHorizontalBox::Slot()
        .AutoWidth()
        [
            SAssignNew(Image, SImage)
            .Image(FAppStyle::Get().GetBrush("ContentBrowser.FilterImage"))
            .ColorAndOpacity(Args._ForegroundColor)
        ]
        + SHorizontalBox::Slot()
        .Padding(Args._Padding)
        .VAlign(VAlign_Center)
        [
			Args._Content.Widget
        ]
    ];

	Image->SetOnMouseButtonUp(FPointerEventHandler::CreateLambda(
		[this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
        {
            if (OnFilterClickedOnce.IsBound() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
            {
                return OnFilterClickedOnce.Execute();
            }
            return FReply::Handled();
        })
    );
}

FReply SLevelSnapshotsFilterCheckBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}
FReply SLevelSnapshotsFilterCheckBox::OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) 
{
	if (InMouseEvent.IsControlDown() && OnFilterCtrlClicked.IsBound())
	{
		return OnFilterCtrlClicked.Execute().ReleaseMouseCapture();
	}
	if (InMouseEvent.IsAltDown() && OnFilterAltClicked.IsBound())
	{
		return OnFilterAltClicked.Execute().ReleaseMouseCapture();
	}
	if(InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnFilterMiddleButtonClicked.IsBound())
	{
		return OnFilterMiddleButtonClicked.Execute().ReleaseMouseCapture();
	}
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnFilterRightButtonClicked.IsBound())
	{
		return OnFilterRightButtonClicked.Execute().ReleaseMouseCapture();
	}
		
	return FReply::Handled().ReleaseMouseCapture();
}