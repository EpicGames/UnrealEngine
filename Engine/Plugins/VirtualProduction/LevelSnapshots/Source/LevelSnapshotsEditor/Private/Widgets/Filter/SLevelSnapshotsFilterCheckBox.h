// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SBorder;

/* Displays contents of filters. */
class SLevelSnapshotsFilterCheckBox : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLevelSnapshotsFilterCheckBox)
	{}
		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_EVENT(FOnClicked, OnFilterCtrlClicked)
		SLATE_EVENT(FOnClicked, OnFilterAltClicked)
		SLATE_EVENT(FOnClicked, OnFilterMiddleButtonClicked)
		SLATE_EVENT(FOnClicked, OnFilterRightButtonClicked)
		SLATE_EVENT(FOnClicked, OnFilterClickedOnce)
		SLATE_ATTRIBUTE( FMargin, Padding )
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )
	SLATE_END_ARGS();

	void Construct(const FArguments& Args);
	
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

private:
	
	FOnClicked OnFilterCtrlClicked;
	FOnClicked OnFilterAltClicked;
	FOnClicked OnFilterMiddleButtonClicked;
	FOnClicked OnFilterRightButtonClicked;
	FOnClicked OnFilterClickedOnce;

	TSharedPtr<SBorder> ContentContainer;
};
