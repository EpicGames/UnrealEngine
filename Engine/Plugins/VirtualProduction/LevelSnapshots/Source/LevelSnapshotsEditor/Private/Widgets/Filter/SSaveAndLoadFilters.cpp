// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Filter/SSaveAndLoadFilters.h"

#include "Data/Filters/LevelSnapshotsFilterPreset.h"
#include "Data/FilterLoader.h"
#include "Data/LevelSnapshotsEditorData.h"

#include "ContentBrowserModule.h"
#include "Engine/AssetManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "Misc/MessageDialog.h"
#include "SAssetDropTarget.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

namespace
{
	void SaveAs(TWeakObjectPtr<UFilterLoader> FilterLoader)
	{
		FilterLoader->SaveAs();
	}

	void SaveExisting(TWeakObjectPtr<UFilterLoader> FilterLoader)
	{
		const TOptional<FAssetData> CurrentlySelected = FilterLoader->GetAssetLastSavedOrLoaded();
		if (ensure(CurrentlySelected.IsSet()))
		{
			FFormatOrderedArguments Arguments;
			Arguments.Add(FText::FromString(CurrentlySelected->AssetName.ToString()));
			const FText Text = FText::Format(LOCTEXT("ConfirmOverwriteText", "This replaces the filters in asset {0} with ones set up in the editor. Are you sure?"), Arguments);
			const FText Title = LOCTEXT("ConfirmOverwriteTitle", "Confirm save");
			
			const EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::OkCancel, Text, Title);
			if (Answer == EAppReturnType::Cancel)
			{
				return;
			}
		}
		
		FilterLoader->OverwriteExisting();
	}

	void OnSelectPreset(const FAssetData& SelectedAssetData, TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData)
	{
		if (!ensure(EditorData.IsValid()))
		{
			return;
		}
		
		UObject* Loaded = SelectedAssetData.GetAsset();
		ULevelSnapshotsFilterPreset* AsFilter = Cast<ULevelSnapshotsFilterPreset>(Loaded);
		if (!ensure(Loaded) || !ensure(AsFilter))
		{
			return;
		}

		TWeakObjectPtr<UFilterLoader> FilterLoader = EditorData->GetFilterLoader();
		const TOptional<FAssetData> PreviousSelection = FilterLoader->GetAssetLastSavedOrLoaded();

		const EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("LoseChangesDialogText", "Are you sure you want to load another preset? Any changes you made will be lost."), LOCTEXT("LoseChangesDialogTitle", "Lose changes"));
		if (Answer == EAppReturnType::Cancel)
		{
			return;
		}

		FilterLoader->LoadAsset(AsFilter);
	}
}

void SSaveAndLoadFilters::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData)
{
	if (!ensure(InEditorData))
	{
		return;
	}

	EditorData = InEditorData;
	FilterLoader = InEditorData->GetFilterLoader();

	ChildSlot
    [
        SNew(SComboButton)
          .ToolTipText(LOCTEXT("SaveLoad_Tooltip", "Export the current filter to an asset, or load a previously saved filter."))
          .ContentPadding(4.f)
          .ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
          .OnGetMenuContent(this, &SSaveAndLoadFilters::GenerateSaveLoadMenu)
          .ForegroundColor(FSlateColor::UseForeground())
          .ButtonContent()
          [
            SNew(SHorizontalBox)
            + SHorizontalBox::Slot()
            .Padding(0, 1, 4, 0)
            .AutoWidth()
            [
              SNew(SImage)
              .Image(FSlateIconFinder::FindIconBrushForClass(ULevelSnapshotsFilterPreset::StaticClass()))
            ]

            + SHorizontalBox::Slot()
            .Padding(0, 1, 0, 0)
            [
              SNew(STextBlock)
              .Text(LOCTEXT("SavedToolbarButton", "Save / Load Preset"))
            ]
          ]
    ];
}

TSharedRef<SWidget> SSaveAndLoadFilters::GenerateSaveLoadMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	const TOptional<FAssetData> SelectedFilter = FilterLoader->GetAssetLastSavedOrLoaded();
	if (SelectedFilter.IsSet())
	{
		FFormatOrderedArguments Arguements;
		Arguements.Add(FText::FromString(SelectedFilter->AssetName.ToString()));
		const FText EntryName = FText::Format(LOCTEXT("LoadFilters.LoadedName", "Save \"{0}\""), Arguements);
						
		MenuBuilder.AddMenuEntry(
            EntryName,
            LOCTEXT("SaveExistingFiltersToolTip", "Overwrite the asset you last loaded"),
            FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
            FUIAction(
                FExecuteAction::CreateStatic(&SaveExisting, FilterLoader)
            )
            );
	}
	
	MenuBuilder.AddMenuEntry(
	    LOCTEXT("SaveFiltersAs", "Save as..."),
	    LOCTEXT("SaveFiltersAsToolTip", "Saves a new asset."),
	    FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAssetAs"),
	    FUIAction(
	        FExecuteAction::CreateStatic(&SaveAs, FilterLoader)
		)	
    );

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("LoadPreset_MenuSection", "Load preset"));
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.Filter.ClassPaths.Add(ULevelSnapshotsFilterPreset::StaticClass()->GetClassPathName());
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected.BindStatic(OnSelectPreset, EditorData);
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.bAllowDragging = false;
		
        TSharedRef<SWidget> PresetPicker = SNew(SBox)
            .HeightOverride(300)
            .WidthOverride(300)
            [
                SNew(SBorder)
                .BorderImage(FAppStyle::GetBrush("Menu.Background"))
                [
                    ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
                ]
            ];
		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


#undef LOCTEXT_NAMESPACE
