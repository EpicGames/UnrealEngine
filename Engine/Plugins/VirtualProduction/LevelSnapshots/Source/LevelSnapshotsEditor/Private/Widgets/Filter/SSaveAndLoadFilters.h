// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UFilterLoader;
class ULevelSnapshotsEditorData;

/* Creates buttons for save & load of filters. */
class SSaveAndLoadFilters : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSaveAndLoadFilters)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData);

private:
	
	TSharedRef<SWidget> GenerateSaveLoadMenu();
	
	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData;
	TWeakObjectPtr<UFilterLoader> FilterLoader;
};
