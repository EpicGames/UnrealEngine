// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSnapshotsEditorBrowser.h"

#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshot.h"
#include "LevelSnapshotsEditorFunctionLibrary.h"
#include "LevelSnapshotsEditorModule.h"
#include "LevelSnapshotsEditorStyle.h"
#include "LevelSnapshotsLog.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetThumbnail.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "EditorStyleSet.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SLevelSnapshotsEditorBrowser::Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData)
{
	OwningWorldPathAttribute = InArgs._OwningWorldPath;
	EditorData = InEditorData;

	check(OwningWorldPathAttribute.IsSet());

	const FContentBrowserModule& ContentBrowserModule =
		FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FARFilter ARFilter;
	ARFilter.ClassPaths.Add(ULevelSnapshot::StaticClass()->GetClassPathName());

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.CustomColumns = GetCustomColumns();
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = true;
	AssetPickerConfig.bAutohideSearchBar = false;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = false;
	AssetPickerConfig.bSortByPathInColumnView = false;
	AssetPickerConfig.SaveSettingsName = TEXT("LevelSnapshotPicker");
	AssetPickerConfig.ThumbnailScale = 0.8f;
	AssetPickerConfig.Filter = ARFilter;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.OnAssetDoubleClicked =
		FOnAssetSelected::CreateSP(this, &SLevelSnapshotsEditorBrowser::OnAssetDoubleClicked);
	AssetPickerConfig.OnShouldFilterAsset =
		FOnShouldFilterAsset::CreateSP(this, &SLevelSnapshotsEditorBrowser::OnShouldFilterAsset);
	AssetPickerConfig.OnGetAssetContextMenu =
		FOnGetAssetContextMenu::CreateSP(this, &SLevelSnapshotsEditorBrowser::OnGetAssetContextMenu);
	AssetPickerConfig.OnGetCustomAssetToolTip =
		FOnGetCustomAssetToolTip::CreateSP(this, &SLevelSnapshotsEditorBrowser::CreateCustomTooltip);
		

	ChildSlot
	[
		ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
	];	
}

SLevelSnapshotsEditorBrowser::~SLevelSnapshotsEditorBrowser()
{
	
}

void SLevelSnapshotsEditorBrowser::SelectAsset(const FAssetData& InAssetData) const
{
	SCOPED_SNAPSHOT_EDITOR_TRACE(SelectSnapshotAsset);
	
	FScopedSlowTask SelectSnapshot(100.f, LOCTEXT("SelectSnapshotKey", "Loading snapshot"));
	SelectSnapshot.EnterProgressFrame(60.f);
	SelectSnapshot.MakeDialog();
	
	ULevelSnapshot* Snapshot = Cast<ULevelSnapshot>(InAssetData.GetAsset());

	SelectSnapshot.EnterProgressFrame(40.f);
	if (ensure(Snapshot && EditorData.IsValid()))
	{
		EditorData->SetActiveSnapshot(Snapshot);
	}
}

TArray<FAssetViewCustomColumn> SLevelSnapshotsEditorBrowser::GetCustomColumns() const
{
	TArray<FAssetViewCustomColumn> ReturnValue;

	{
		auto ColumnStringReturningLambda = [](const FAssetData& AssetData, const FName& ColumnName)
		{
			FString AssetMetaData;
			const bool bHasMetaTag = AssetData.GetTagValue(ColumnName, AssetMetaData);
			AssetMetaData = bHasMetaTag ? FSoftObjectPath(AssetMetaData).GetAssetName() : "";
				
			return AssetMetaData;
		};
		
		FAssetViewCustomColumn Column;
		Column.ColumnName = FName("MapPath");
		Column.DataType = UObject::FAssetRegistryTag::TT_Alphabetical;
		Column.DisplayName = LOCTEXT("SnapshotMapNameColumnName", "Owning Map");
		Column.OnGetColumnData = FOnGetCustomAssetColumnData::CreateLambda(ColumnStringReturningLambda);
		Column.OnGetColumnDisplayText =
			FOnGetCustomAssetColumnDisplayText::CreateLambda(
				[ColumnStringReturningLambda](const FAssetData& AssetData, const FName& ColumnName)
			{
				return FText::FromString(ColumnStringReturningLambda(AssetData, ColumnName));
			});
		ReturnValue.Add(Column);
	}

	{
		auto ColumnStringReturningLambda = [](const FAssetData& AssetData, const FName& ColumnName)
		{
			FString AssetMetaData;			
			return AssetData.GetTagValue(ColumnName, AssetMetaData) ? AssetMetaData : "";
		};
		
		FAssetViewCustomColumn Column;
		Column.ColumnName = FName("SnapshotDescription");
		Column.DataType = UObject::FAssetRegistryTag::TT_Alphabetical;
		Column.DisplayName = LOCTEXT("SnapshotDescriptionColumnName", "Description");
		Column.OnGetColumnData = FOnGetCustomAssetColumnData::CreateLambda(ColumnStringReturningLambda);
		Column.OnGetColumnDisplayText = 
			FOnGetCustomAssetColumnDisplayText::CreateLambda(
				[ColumnStringReturningLambda](const FAssetData& AssetData, const FName& ColumnName)
			{
				return FText::FromString(ColumnStringReturningLambda(AssetData, ColumnName));
			});
		ReturnValue.Add(Column);
	}

	{
		auto ColumnStringReturningLambda = [](const FAssetData& AssetData, const FName& ColumnName)
		{
			FString AssetMetaData;			
			return AssetData.GetTagValue(ColumnName, AssetMetaData) ? AssetMetaData : "";
		};
		
		FAssetViewCustomColumn Column;
		Column.ColumnName = FName("CaptureTime");
		Column.DataType = UObject::FAssetRegistryTag::TT_Alphabetical;
		Column.DisplayName = LOCTEXT("SnapshotCaptureTimeColumnName", "Time Taken");
		Column.OnGetColumnData = FOnGetCustomAssetColumnData::CreateLambda(ColumnStringReturningLambda);
		Column.OnGetColumnDisplayText = 
			FOnGetCustomAssetColumnDisplayText::CreateLambda(
				[ColumnStringReturningLambda](const FAssetData& AssetData, const FName& ColumnName)
			{
				return FText::FromString(ColumnStringReturningLambda(AssetData, ColumnName));
			});
		ReturnValue.Add(Column);
	}

	return ReturnValue;
}

void SLevelSnapshotsEditorBrowser::OnAssetDoubleClicked(const FAssetData& InAssetData) const
{
	SelectAsset(InAssetData);
}

bool SLevelSnapshotsEditorBrowser::OnShouldFilterAsset(const FAssetData& InAssetData) const
{
	const FString SnapshotMapPath = InAssetData.GetTagValueRef<FString>("MapPath");

	const bool bShouldFilter = SnapshotMapPath != OwningWorldPathAttribute.Get().ToString();
	
	return bShouldFilter;
}

TSharedPtr<SWidget> SLevelSnapshotsEditorBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	if (SelectedAssets.Num() <= 0)
	{
		return nullptr;
	}

	UObject* SelectedAsset = SelectedAssets[0].GetAsset();
	if (SelectedAsset == nullptr)
	{
		return nullptr;
	}
	
	FMenuBuilder MenuBuilder(true, MakeShared<FUICommandList>());

	MenuBuilder.BeginSection(TEXT("Asset"), NSLOCTEXT("ReferenceViewerSchema", "AssetSectionLabel", "Asset"));
	{
		MenuBuilder.AddMenuEntry(
		LOCTEXT("AssetTypeActions_LevelSnapshot_UpdateSnapshotData", "Update Snapshot Data"),
		LOCTEXT("AssetTypeActions_LevelSnapshot_UpdateSnapshotDataToolTip", "Record a snapshot of the current map to this snapshot asset and update the thumbnail. Equivalent to 'Take Snapshot'. Select only one Level Snapshot asset at a time."),
		FSlateIcon(FLevelSnapshotsEditorStyle::GetStyleSetName(), "LevelSnapshots.ToolbarButton.Small"),
		FUIAction(
			FExecuteAction::CreateLambda([SelectedAsset] {
				if (const TObjectPtr<ULevelSnapshot> LevelSnapshotAsset = Cast<ULevelSnapshot>(SelectedAsset))
				{
					if (UWorld* World = ULevelSnapshotsEditorData::GetEditorWorld())
					{
						LevelSnapshotAsset->SnapshotWorld(World);
						ULevelSnapshotsEditorFunctionLibrary::GenerateThumbnailForSnapshotAsset(LevelSnapshotAsset);
						LevelSnapshotAsset->MarkPackageDirty();
					}
				}
			}),
			FCanExecuteAction::CreateLambda([] () { return true; })
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AssetTypeActions_LevelSnapshot_CaptureThumbnail", "Capture Thumbnail"),
			LOCTEXT("AssetTypeActions_LevelSnapshot_CaptureThumbnailsToolTip2", "Capture and update the thumbnail only for this asset."),
			FSlateIcon(FLevelSnapshotsEditorStyle::GetStyleSetName(), "LevelSnapshots.ToolbarButton.Small"),
			FUIAction(
			FExecuteAction::CreateLambda([SelectedAsset] {
				if (const TObjectPtr<ULevelSnapshot> LevelSnapshotAsset = Cast<ULevelSnapshot>(SelectedAsset))
				{
					ULevelSnapshotsEditorFunctionLibrary::GenerateThumbnailForSnapshotAsset(LevelSnapshotAsset);
					LevelSnapshotAsset->MarkPackageDirty();
				}
			}),
			FCanExecuteAction::CreateLambda([] () { return true; })
			)
		);
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Browse", "Browse to Asset"),
			LOCTEXT("BrowseTooltip", "Browses to the associated asset and selects it in the most recently used Content Browser (summoning one if necessary)"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small"),
			FUIAction(
				FExecuteAction::CreateLambda([SelectedAsset] ()
				{
					if (SelectedAsset)
					{
						const TArray<FAssetData>& Assets = { SelectedAsset };
						FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
						ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
					}
				}),
				FCanExecuteAction::CreateLambda([] () { return true; })
			)
		);

		MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenSnapshot", "Open Snapshot in Editor"),
		LOCTEXT("OpenSnapshotToolTip", "Open this snapshot in the Level Snapshots Editor."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.SummonOpenAssetDialog"),
			FUIAction(
				FExecuteAction::CreateLambda([this, SelectedAsset] ()
				{
					if (SelectedAsset)
					{
						SelectAsset(SelectedAsset);
					}
				}),
				FCanExecuteAction::CreateLambda([] () { return true; })
			)
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SToolTip> SLevelSnapshotsEditorBrowser::CreateCustomTooltip(FAssetData& AssetData)
{
	constexpr uint32 MaxTooltipWidth = 1024;
	constexpr uint32 ThumbnailSize = 256;
	constexpr uint32 MaxDescriptionHeight = ThumbnailSize / 3 * 2;
	
	const TSharedRef<FAssetThumbnail> AssetThumbnail =
		MakeShared<FAssetThumbnail>(
			AssetData,
			ThumbnailSize,
			ThumbnailSize,
			UThumbnailManager::Get().GetSharedThumbnailPool()
		);
	
	FAssetThumbnailConfig AssetThumbnailConfig;
	AssetThumbnailConfig.bAllowFadeIn = false;
	AssetThumbnailConfig.bAllowRealTimeOnHovered = false;
	AssetThumbnailConfig.bForceGenericThumbnail = false;
	AssetThumbnailConfig.ColorStripOrientation = EThumbnailColorStripOrientation::VerticalRightEdge;

	FString OutDescription;
	AssetData.GetTagValue("SnapshotDescription",OutDescription);

	FString OutCaptureTime;
	AssetData.GetTagValue("CaptureTime",OutCaptureTime);
	
	FSlateFontInfo AssetNameTextFont = FAppStyle::Get().GetFontStyle("Bold");
	AssetNameTextFont.Size = 10;
	
	return SNew(SToolTip)
	[
		SNew(SBox)
		.MaxDesiredWidth(MaxTooltipWidth)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.HeightOverride(ThumbnailSize + 5) // Offset for asset color strip
				.WidthOverride(ThumbnailSize)
				[
					AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig)
				]
			]

			+SHorizontalBox::Slot()
			.MaxWidth(MaxTooltipWidth - ThumbnailSize)
			.VAlign(VAlign_Center)
			.Padding(FMargin(5.f))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(FText::FromName(AssetData.AssetName))
					.Font(AssetNameTextFont)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(FText::FromString(OutCaptureTime))
				]

				+SVerticalBox::Slot()
				.MaxHeight(MaxDescriptionHeight)
				[
					SNew(STextBlock)
					.Text(FText::FromString(OutDescription))
					.AutoWrapText(true)
					.WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)				
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
