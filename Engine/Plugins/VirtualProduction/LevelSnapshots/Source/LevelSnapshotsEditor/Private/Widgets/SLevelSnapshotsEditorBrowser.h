// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ULevelSnapshotsEditorData;

struct FAssetData;
struct FAssetViewCustomColumn;

class SLevelSnapshotsEditorBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorBrowser)
	{}

	/** Attribute for retrieving the current context */
	SLATE_ATTRIBUTE(FSoftObjectPath, OwningWorldPath)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULevelSnapshotsEditorData* InEditorData);
	virtual ~SLevelSnapshotsEditorBrowser() override;

	void SelectAsset(const FAssetData& InAssetData) const;

private:
	
	TArray<FAssetViewCustomColumn> GetCustomColumns() const;
	void OnAssetDoubleClicked(const FAssetData& InAssetData) const;
	bool OnShouldFilterAsset(const FAssetData& InAssetData) const;
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets);
	TSharedRef<SToolTip> CreateCustomTooltip(FAssetData& AssetData);
	
	TAttribute<FSoftObjectPath> OwningWorldPathAttribute;

	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData;
};
