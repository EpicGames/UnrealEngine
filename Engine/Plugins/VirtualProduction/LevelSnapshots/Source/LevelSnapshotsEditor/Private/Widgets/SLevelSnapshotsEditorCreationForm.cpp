// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSnapshotsEditorCreationForm.h"

#include "DetailLayoutBuilder.h"
#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshotsEditorStyle.h"
#include "LevelSnapshotsEditorSettings.h"

#include "Styling/AppStyle.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "LevelSnapshotsSettings.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "SPrimaryButton.h"

namespace LevelSnapshotsEditor
{
	class FShowOnlyDataManagementsDetailsCustomization : public IDetailCustomization
	{
	public:
		
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
		{
			// Iterate through all categories in case somebody adds a new category and forgets to update this
			const FString DataCategory("Data");
			for (TFieldIterator<FProperty> PropertyIt(ULevelSnapshotsEditorSettings::StaticClass()); PropertyIt; ++PropertyIt)
			{
				const FString Category = PropertyIt->HasMetaData(TEXT("Category")) ? PropertyIt->GetMetaData(TEXT("Category")) : FString();
				if (!Category.Equals(DataCategory))
				{
					DetailBuilder.HideCategory(*Category);
				}
			}
		}
	};
}


TSharedRef<SWindow> SLevelSnapshotsEditorCreationForm::MakeAndShowCreationWindow(const FCloseCreationFormDelegate& CallOnClose)
{
	// Compute centered window position based on max window size, which include when all categories are expanded
	const FVector2D LastSize = ULevelSnapshotsEditorSettings::Get()->GetLastCreationWindowSize();
	const FVector2D BaseWindowSize = FVector2D(LastSize.X, LastSize.Y); // Max window size it can get based on current slate

	const FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
	const FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
	const FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

	const FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - BaseWindowSize) / 2.0f);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("LevelSnapshots", "LevelSnapshots_CreationForm_Title", "Create Level Snapshot"))
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.ClientSize(BaseWindowSize)
		.AdjustInitialSizeAndPositionForDPIScale(false)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ScreenPosition(WindowPosition);

	const TSharedRef<SLevelSnapshotsEditorCreationForm> CreationForm = SNew(SLevelSnapshotsEditorCreationForm, Window, CallOnClose);
	Window->SetContent
	(
		CreationForm
	);
	Window->SetOnWindowClosed(FOnWindowClosed::CreateSP(CreationForm, &SLevelSnapshotsEditorCreationForm::OnWindowClosed));

	FSlateApplication::Get().AddWindow(Window);
	return Window;
}

void SLevelSnapshotsEditorCreationForm::Construct(
	const FArguments& InArgs,
	TWeakPtr<SWindow> InWidgetWindow,
	const FCloseCreationFormDelegate& CallOnClose)
{
	WidgetWindow = InWidgetWindow;
	CallOnCloseDelegate = CallOnClose;
	
	FSlateFontInfo NameTextFont = FAppStyle::Get().GetFontStyle("Regular");
	NameTextFont.Size = 9;
	
	FSlateFontInfo OverrideNameTextFont = FAppStyle::Get().GetFontStyle("Bold");
	OverrideNameTextFont.Size = 16;

	FSlateFontInfo DescriptionTextFont = FAppStyle::Get().GetFontStyle("Bold");
	DescriptionTextFont.Size = 10;
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(new FSlateColorBrush(FColor(10, 10, 10)))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f, 2.f, 2.f, 0.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(1.f, 1.f, 0.f, 0.f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
						.Font(NameTextFont)
						.ColorAndOpacity(FColor(200, 200, 200))
						.Text(NSLOCTEXT("LevelSnapshots", "CreationForm_SnapshotNameLabel", "Name"))
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.f, 1.f, 8.f, 10.f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Fill)
						[
							SNew(SEditableTextBox)
							.Font(OverrideNameTextFont)
							.BackgroundColor(FLinearColor::Transparent)
							.ForegroundColor(FSlateColor::UseForeground())
							.Justification(ETextJustify::Center)
							.SelectAllTextWhenFocused(true)
							.HintText(NSLOCTEXT("LevelSnapshots", "CreationForm_SnapshotNameOverrideHintText", "Override Snapshot Name..."))
							.Text(this, &SLevelSnapshotsEditorCreationForm::GetNameOverrideText)
							.OnTextCommitted(this, &SLevelSnapshotsEditorCreationForm::SetNameOverrideText)
							.ToolTipText(
							NSLOCTEXT("LevelSnapshots", "CreationForm_NameOverrideFieldTooltipText", "Override the name defined in Project Settings while using the Creation Form."))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.IsFocusable(false)
							.ToolTipText(
								NSLOCTEXT("LevelSnapshots", "CreationForm_ResetNameTooltipText", "Reset the overridden name to the one defined in Project Settings."))
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.ContentPadding(0)
							.Visibility(this, &SLevelSnapshotsEditorCreationForm::GetNameDiffersFromDefaultAsVisibility)
							.OnClicked(this, &SLevelSnapshotsEditorCreationForm::OnResetNameClicked)
							.Content()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
							]
						]
					]
				]
			]

			+SVerticalBox::Slot()
			.Padding(2.f, 10.f, 2.f, 0.f)
			.VAlign(VAlign_Fill)
			[
				SNew(SMultiLineEditableTextBox)
				.Font(DescriptionTextFont)
				.BackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f))
				.ForegroundColor(FSlateColor::UseForeground())
				.SelectAllTextWhenFocused(true)
				.HintText(NSLOCTEXT("LevelSnapshots", "CreationForm_DescriptionHintText", "Description"))
				.Text(DescriptionText)
				.OnTextCommitted(this, &SLevelSnapshotsEditorCreationForm::SetDescriptionText)
				.AllowMultiLine(true)
				.AutoWrapText(true)
			]
			
			+SVerticalBox::Slot()
			.Padding(2.f, 10.f, 2.f, 0.f)
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "NormalText.Important")
				.Text(NSLOCTEXT("LevelSnapshots", "CreationForm_SaveDirLabel", "Save Directory"))
			]

			+SVerticalBox::Slot()
			.Padding(2.f, 2.f, 2.f, 0.f)
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				MakeDataManagementSettingsDetailsWidget()
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Fill)
			.Padding(2.f, 5.f)
			[
				SNew(SHorizontalBox)
				
				// Create snapshot button
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(SPrimaryButton)
					.OnClicked(this, &SLevelSnapshotsEditorCreationForm::OnCreateButtonPressed)
					.Text(NSLOCTEXT("LevelSnapshots", "NotificationFormatText_CreationForm_CreateSnapshotButton", "Create Level Snapshot"))
				]
			]
		]
	];
}

TSharedRef<SWidget> SLevelSnapshotsEditorCreationForm::MakeDataManagementSettingsDetailsWidget() const
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowScrollBar = false;

	TSharedRef<IDetailsView> Details = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	const TArray<UObject*> ProjectSettingsObjects = { ULevelSnapshotsEditorSettings::Get() };
	// By requirement, we're only supposed to show the data management settings
	Details->RegisterInstancedCustomPropertyLayout(
		ULevelSnapshotsEditorSettings::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateLambda([](){ return MakeShared<LevelSnapshotsEditor::FShowOnlyDataManagementsDetailsCustomization>(); })
		);
	Details->SetObjects(ProjectSettingsObjects);
	Details->SetEnabled(true);

	return Details;
}

FText SLevelSnapshotsEditorCreationForm::GetNameOverrideText() const
{
	const ULevelSnapshotsEditorSettings* Settings = ULevelSnapshotsEditorSettings::Get();
	check(Settings);

	const FString SnapshotName = NameOverride.Get(Settings->DefaultLevelSnapshotName);
	UWorld* World = ULevelSnapshotsEditorData::GetEditorWorld();
	if (!ensure(World))
	{
		return FText::FromString(SnapshotName);
	}

	return ULevelSnapshotsEditorSettings::ParseLevelSnapshotsTokensInText(
		FText::FromString(SnapshotName),
		World->GetName()
		);
}

void SLevelSnapshotsEditorCreationForm::SetNameOverrideText(const FText& InNewText, ETextCommit::Type InCommitType)
{
	FString NameAsString = InNewText.ToString();
	ULevelSnapshotsEditorSettings::SanitizePathInline(NameAsString, true);
	NameOverride = NameAsString;
}

void SLevelSnapshotsEditorCreationForm::SetDescriptionText(const FText& InNewText, ETextCommit::Type InCommitType)
{
	DescriptionText = InNewText;
}

EVisibility SLevelSnapshotsEditorCreationForm::GetNameDiffersFromDefaultAsVisibility() const
{
	return NameOverride.IsSet() ? EVisibility::Visible : EVisibility::Hidden;
}

FReply SLevelSnapshotsEditorCreationForm::OnResetNameClicked()
{
	SetNameOverrideText(FText::FromString(ULevelSnapshotsEditorSettings::Get()->DefaultLevelSnapshotName), ETextCommit::OnEnter);
	return FReply::Handled();
}

FReply SLevelSnapshotsEditorCreationForm::OnCreateButtonPressed()
{
	CallOnCloseDelegate.ExecuteIfBound(DescriptionText, GetNameOverrideText().ToString());

	check(WidgetWindow.IsValid());
	WidgetWindow.Pin()->RequestDestroyWindow();
	
	return FReply::Handled();
}

void SLevelSnapshotsEditorCreationForm::OnWindowClosed(const TSharedRef<SWindow>& ParentWindow) const
{
	const FVector2D WindowSize = ParentWindow->GetClientSizeInScreen();
	ULevelSnapshotsEditorSettings* Settings = ULevelSnapshotsEditorSettings::Get();
	Settings->SetLastCreationWindowSize(WindowSize);
	Settings->SaveConfig();
}
