// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dialog/SCustomDialog.h"

DECLARE_DELEGATE_TwoParams(FCloseCreationFormDelegate, const FText& /*Description*/, const FString& /*SnapshotName*/);

class ULevelSnapshotsSettings;
class ULevelSnapshotsEditorSettings;
class SWindow;

class SLevelSnapshotsEditorCreationForm : public SCompoundWidget
{
public:

	static TSharedRef<SWindow> MakeAndShowCreationWindow(const FCloseCreationFormDelegate& CallOnClose);
	
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorCreationForm)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<SWindow> InWidgetWindow, const FCloseCreationFormDelegate& CallOnClose);
	
	FText GetNameOverrideText() const;
	void SetNameOverrideText(const FText& InNewText, ETextCommit::Type InCommitType);
	void SetDescriptionText(const FText& InNewText, ETextCommit::Type InCommitType);

	EVisibility GetNameDiffersFromDefaultAsVisibility() const;

	FReply OnResetNameClicked();
	FReply OnCreateButtonPressed();

private:
	
	TWeakPtr<SWindow> WidgetWindow;

	FText DescriptionText;
	TOptional<FString> NameOverride;
	FCloseCreationFormDelegate CallOnCloseDelegate;
	
	TSharedRef<SWidget> MakeDataManagementSettingsDetailsWidget() const;
	void OnWindowClosed(const TSharedRef<SWindow>& ParentWindow) const;
};
