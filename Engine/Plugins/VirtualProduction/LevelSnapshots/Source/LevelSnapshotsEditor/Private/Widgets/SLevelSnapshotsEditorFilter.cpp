// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSnapshotsEditorFilter.h"

#include "Data/Filters/NegatableFilter.h"
#include "Data/LevelSnapshotsEditorData.h"
#include "LevelSnapshotsEditorStyle.h"
#include "Widgets/Filter/SLevelSnapshotsFilterCheckBox.h"
#include "Widgets/Filter/SHoverableFilterActions.h"

#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

/* Child of SFilterCheckBox. Like text but allows click events. */
class SClickableText : public STextBlock
{
public:
	
	void SetOnClicked(const FOnClicked& Callback)
	{
		OnClicked = Callback;
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return OnClicked.Execute();
		}
		// Allow SFilterCheckBox to respond to right-clicks, etc.
		return FReply::Unhandled();
	}

private:
	FOnClicked OnClicked;;
};

SLevelSnapshotsEditorFilter::~SLevelSnapshotsEditorFilter()
{
	if (ensure(EditorData.IsValid()))
	{
		EditorData->OnEditedFilterChanged.Remove(ActiveFilterChangedDelegateHandle);
	}

	if (SnapshotFilter.IsValid())
	{
		SnapshotFilter->OnFilterDestroyed.Remove(OnFilterDestroyedDelegateHandle);
	}
}

void SLevelSnapshotsEditorFilter::Construct(const FArguments& InArgs, const TWeakObjectPtr<UNegatableFilter>& InFilter, ULevelSnapshotsEditorData* InEditorData)
{
	if (!ensure(InFilter.IsValid() && InArgs._OnClickRemoveFilter.IsBound() && InArgs._IsParentFilterIgnored.IsBound()))
	{
		return;
	}
	
	SnapshotFilter = InFilter;
	EditorData = InEditorData;
	OnClickRemoveFilter = InArgs._OnClickRemoveFilter;
	IsParentFilterIgnored = InArgs._IsParentFilterIgnored;

	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::Visible)

		+SOverlay::Slot()
		[
			SNew(SBorder)
			.Padding(0)
			.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 1))
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.FilterBackground"))
			.ColorAndOpacity_Lambda([this](){ return SnapshotFilter.IsValid() && SnapshotFilter->IsIgnored() ? FLinearColor(0.175f, 0.175f, 0.175f, 1.f) : FLinearColor(1,1,1,1); })
			[
				SAssignNew(ToggleButtonPtr, SLevelSnapshotsFilterCheckBox) 
				.ToolTipText(this, &SLevelSnapshotsEditorFilter::GetFilterTooltip)
				.OnFilterClickedOnce(this, &SLevelSnapshotsEditorFilter::OnNegateFilter)
				.ForegroundColor(this, &SLevelSnapshotsEditorFilter::GetFilterColor)
				.Padding(FMargin(5.f, 1.f))
				[
					SAssignNew(FilterNamePtr, SClickableText)
					.MinDesiredWidth(65.f)	//  SHoverableFilterActions (see below) makes clicking filters with short names difficult
					.ColorAndOpacity(FLinearColor::White)
					.Font(FAppStyle::GetFontStyle("ContentBrowser.FilterNameFont"))
					.ShadowOffset(FVector2D(1.f, 1.f))
					.Text_Lambda([InFilter]()
					{
						if (InFilter.IsValid())
						{
							return InFilter->GetDisplayName();
						}
						return FText();
					})	
				]
			]
		]

		// Ignore check box
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SHoverableFilterActions, SharedThis(this))
			.Visibility(EVisibility::SelfHitTestInvisible)
			.IsFilterIgnored_Lambda([this](){ return SnapshotFilter.IsValid() ? SnapshotFilter->IsIgnored() : true; })
			.OnChangeFilterIgnored_Lambda([this](bool bNewValue)
			{
				if (SnapshotFilter.IsValid())
				{
					SnapshotFilter->SetIsIgnored(bNewValue);
				}
			})
			.OnPressDelete_Lambda([this](){ OnRemoveFilter(); })
			.BackgroundHoverColor(FSlateColor(FLinearColor(0.f, 0.f, 0.f, 0.4f)))
		]
	];

	// Hightlight & unhighlight filter when being edited
	FilterNamePtr->SetOnClicked(FOnClicked::CreateRaw(this, &SLevelSnapshotsEditorFilter::OnSelectFilterForEdit));
	
	ActiveFilterChangedDelegateHandle = InEditorData->OnEditedFilterChanged.AddSP(this, &SLevelSnapshotsEditorFilter::OnActiveFilterChanged);
	OnFilterDestroyedDelegateHandle = InFilter->OnFilterDestroyed.AddLambda([this](UNegatableFilter* Filter)
	{
		if (EditorData.IsValid() && EditorData->IsEditingFilter(Filter))
		{
			return EditorData->SetEditedFilter({});
		}
	});
}

const TWeakObjectPtr<UNegatableFilter>& SLevelSnapshotsEditorFilter::GetSnapshotFilter() const
{
	return SnapshotFilter;
}

int32 SLevelSnapshotsEditorFilter::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId,
	const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	
	if (bShouldHighlightFilter)
	{
		const FVector2D ShadowSize(14, 14);
		const FSlateBrush* ShadowBrush = FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.FilterSelected");
	
		// Draw a shadow	
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToInflatedPaintGeometry(ShadowSize),
			ShadowBrush
		);
	}
	
	return LayerId;
}

FText SLevelSnapshotsEditorFilter::GetFilterTooltip() const
{
	if (SnapshotFilter.IsValid())
	{
		switch (SnapshotFilter->GetFilterBehavior())
		{
		case EFilterBehavior::DoNotNegate:
			return LOCTEXT("DoNotNegate", "Filter result is neither negated nor ignored. Click to toggle.");
		case EFilterBehavior::Negate:
			return LOCTEXT("Negate", "Filter result is negated. Click to toggle.");
		}
	}
	return FText();
}

FSlateColor SLevelSnapshotsEditorFilter::GetFilterColor() const
{
	if (!SnapshotFilter.IsValid())
	{
		return FLinearColor::Black;
	}

	FLinearColor FinalColor;
	switch (SnapshotFilter->GetFilterBehavior())
	{
	case EFilterBehavior::DoNotNegate:
		FinalColor = FLinearColor::Green;
		break;
	case EFilterBehavior::Negate:
		FinalColor = FLinearColor::Red;
		break;
	default: 
		FinalColor = FLinearColor::Black;
	}

	return SnapshotFilter->IsIgnored() || IsParentFilterIgnored.Execute() ? FinalColor * FLinearColor(0.15f, 0.15f, 0.15f, 1) : FinalColor;
}

FReply SLevelSnapshotsEditorFilter::OnSelectFilterForEdit()
{
	if (!ensure(EditorData.IsValid()))
	{
		return FReply::Handled();
	}

	if (EditorData->IsEditingFilter(SnapshotFilter.Get()))
	{
		bShouldHighlightFilter = false;
		EditorData->SetEditedFilter(nullptr);
	}
	else
	{
		bShouldHighlightFilter = true;
		EditorData->SetEditedFilter(SnapshotFilter.Get());
	}
	
	return FReply::Handled();
}

FReply SLevelSnapshotsEditorFilter::OnNegateFilter()
{
	if (ensure(SnapshotFilter.IsValid()))
	{
		SnapshotFilter->SetFilterBehaviour(
			SnapshotFilter->GetFilterBehavior() == EFilterBehavior::DoNotNegate ? EFilterBehavior::Negate : EFilterBehavior::DoNotNegate 
		);
	}
	
	return FReply::Handled();
}

FReply SLevelSnapshotsEditorFilter::OnRemoveFilter()
{
	OnClickRemoveFilter.Execute(SharedThis(this));
	
	return FReply::Handled();
}

void SLevelSnapshotsEditorFilter::OnActiveFilterChanged(UNegatableFilter* NewFilter)
{
	if (SnapshotFilter.IsValid()) // This can actually become stale after a save: UI rebuilds next tick but object was already destroyed.
	{
		bShouldHighlightFilter = NewFilter && NewFilter == SnapshotFilter.Get();
	}
	
}


#undef LOCTEXT_NAMESPACE
