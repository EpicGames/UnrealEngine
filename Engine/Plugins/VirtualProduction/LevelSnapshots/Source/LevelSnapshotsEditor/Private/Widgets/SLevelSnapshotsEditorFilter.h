// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Data/LevelSnapshotsEditorData.h"
#include "Data/Filters/NegatableFilter.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SLevelSnapshotsFilterCheckBox;
class SClickableText;
class FLevelSnapshotsEditorFilters;
class ULevelSnapshotsEditorData;

enum class ECheckBoxState : uint8;

struct FSlateColor;

/* Displays a filter in the editor. */
class SLevelSnapshotsEditorFilter : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnClickRemoveFilter, TSharedRef<SLevelSnapshotsEditorFilter>);
	DECLARE_DELEGATE_RetVal(bool, FIsParentFilterIgnored);

	~SLevelSnapshotsEditorFilter();
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorFilter)
	{}
		SLATE_EVENT(FOnClickRemoveFilter, OnClickRemoveFilter)
		SLATE_EVENT(FIsParentFilterIgnored, IsParentFilterIgnored)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakObjectPtr<UNegatableFilter>& InFilter, ULevelSnapshotsEditorData* InEditorData);

	const TWeakObjectPtr<UNegatableFilter>& GetSnapshotFilter() const;

	//~ Begin SWidget Interface
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	//~ End SWidget Interface

	
private:

	FText GetFilterTooltip() const;
	FSlateColor GetFilterColor() const;
	
	FReply OnSelectFilterForEdit();
	FReply OnNegateFilter();
	FReply OnRemoveFilter();
	
	void OnActiveFilterChanged(UNegatableFilter* NewFilter);

	/* Used to remove this filter */
	FOnClickRemoveFilter OnClickRemoveFilter;
	/* Used to darken filter colour when the parent is ignored. */
	FIsParentFilterIgnored IsParentFilterIgnored;

	FDelegateHandle OnFilterDestroyedDelegateHandle;
	FDelegateHandle ActiveFilterChangedDelegateHandle;
	bool bShouldHighlightFilter = false;;
	
	/** The button to toggle the filter on or off */
	TSharedPtr<SLevelSnapshotsFilterCheckBox> ToggleButtonPtr;
	/* Displays filter name and shows details panel when clicked. */
	TSharedPtr<SClickableText> FilterNamePtr;

	/* Filter managed by this widget */
	TWeakObjectPtr<UNegatableFilter> SnapshotFilter;
	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData;
};
