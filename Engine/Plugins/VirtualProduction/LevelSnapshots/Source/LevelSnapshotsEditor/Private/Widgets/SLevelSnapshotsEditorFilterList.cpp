// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSnapshotsEditorFilterList.h"

#include "Data/Filters/ConjunctionFilter.h"
#include "LevelSnapshotsEditorStyle.h"
#include "Widgets/Filter/SCreateNewFilterWidget.h"
#include "Widgets/SLevelSnapshotsEditorFilter.h"

#include "Components/HorizontalBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SLevelSnapshotsEditorFilterList::Construct(const FArguments& InArgs, UConjunctionFilter* InManagedAndCondition, ULevelSnapshotsEditorData* InEditorData)
{
	EditorData = InEditorData;
	ManagedAndCondition = InManagedAndCondition;

	ChildSlot
	[
		SAssignNew(FilterBox, SWrapBox)
			.UseAllottedSize(true)
	];

	SAssignNew(AddFilterWidget, SCreateNewFilterWidget, InEditorData->GetFavoriteFilters(), InManagedAndCondition);

	Rebuild();
}

void SLevelSnapshotsEditorFilterList::Rebuild()
{
	FilterBox->ClearChildren();
	
	if (!AddTutorialTextAndCreateFilterWidgetIfEmpty())
	{
		bool bSkipAnd = true;
		for (UNegatableFilter* Filter : ManagedAndCondition->GetChildren())
		{
			AddChild(Filter, bSkipAnd);
			bSkipAnd = false;
		}
	}
}

void SLevelSnapshotsEditorFilterList::OnClickRemoveFilter(TSharedRef<SLevelSnapshotsEditorFilter> RemovedFilterWidget) const
{
	ManagedAndCondition->RemoveChild(RemovedFilterWidget->GetSnapshotFilter().Get());
}

bool SLevelSnapshotsEditorFilterList::AddTutorialTextAndCreateFilterWidgetIfEmpty() const
{
	if (!ManagedAndCondition.IsValid())
	{
		return false;
	}
	
	const bool bHasNoFilters = ManagedAndCondition->GetChildren().Num() == 0; 
	if (bHasNoFilters)
	{
		FilterBox->AddSlot()
            .Padding(3, 3)
        [
            SNew(SHorizontalBox)

            + SHorizontalBox::Slot()
                .Padding(2.f, 0.f)
                .VAlign(VAlign_Center)
            [
                SNew(STextBlock)
                .Text(LOCTEXT("FilterListDragTutorial", "Drag a favorite filter here"))
                .Justification(ETextJustify::Center)
            ]
        ];
		
		FilterBox->AddSlot()
		.Padding(3,5)
		[
			AddFilterWidget.ToSharedRef()
		];
	}
	
	return bHasNoFilters;
}

void SLevelSnapshotsEditorFilterList::AddChild(UNegatableFilter* AddedFilter, bool bSkipAnd) const
{
	if (!ManagedAndCondition.IsValid())
	{
		return;
	}
	
	const bool bWasEmptyBefore = ManagedAndCondition->GetChildren().Num() == 1;
	if (bWasEmptyBefore)
	{
		FilterBox->ClearChildren();
	}
	else if (!bSkipAnd)
	{	
		FilterBox->AddSlot()
			.Padding(1)
			.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
                .TextStyle( FLevelSnapshotsEditorStyle::Get(), "LevelSnapshotsEditor.FilterRow.And")
                .Text(LOCTEXT("FilterRow.And", "&"))
                .WrapTextAt(128.0f)
        ];
	}

	const TSharedRef<SCreateNewFilterWidget> AddFilterWidgetAsRef = AddFilterWidget.ToSharedRef();
	FilterBox->RemoveSlot(AddFilterWidgetAsRef);
	FilterBox->AddSlot()
		.Padding(3, 1)
		[
			SNew(SLevelSnapshotsEditorFilter, AddedFilter, EditorData.Get())
				.OnClickRemoveFilter(SLevelSnapshotsEditorFilter::FOnClickRemoveFilter::CreateSP(this, &SLevelSnapshotsEditorFilterList::OnClickRemoveFilter))
				.IsParentFilterIgnored_Lambda([this]() { return ManagedAndCondition->IsIgnored(); })
		];
	
	// AddFilterWidget should be last widget of row
	FilterBox->AddSlot()
		.Padding(3,3)
	[
		AddFilterWidgetAsRef
	];
}

#undef LOCTEXT_NAMESPACE