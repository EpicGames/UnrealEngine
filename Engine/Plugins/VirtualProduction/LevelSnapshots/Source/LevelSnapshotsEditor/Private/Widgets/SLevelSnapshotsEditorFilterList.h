// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Data/LevelSnapshotsEditorData.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FLevelSnapshotsEditorFilters;
class SCreateNewFilterWidget;
class ULevelSnapshotsEditorData;
class SLevelSnapshotsEditorFilter;
class SWrapBox;
class UConjunctionFilter;
class UNegatableFilter;

/* Lists a bunch of filters. */
class SLevelSnapshotsEditorFilterList : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorFilterList)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UConjunctionFilter* InManagedAndCondition, ULevelSnapshotsEditorData* InEditorData);

private:

	void Rebuild();
	
	void OnClickRemoveFilter(TSharedRef<SLevelSnapshotsEditorFilter> RemovedFilterWidget) const;
	bool AddTutorialTextAndCreateFilterWidgetIfEmpty() const;
	void AddChild(UNegatableFilter* AddedFilter, bool bSkipAnd = false) const;
	
	/** The horizontal box which contains all the filters */
	TSharedPtr<SWrapBox> FilterBox;
	/* Wiget with which user can add filters */
	TSharedPtr<SCreateNewFilterWidget> AddFilterWidget;

	TWeakObjectPtr<ULevelSnapshotsEditorData> EditorData;
	TWeakObjectPtr<UConjunctionFilter> ManagedAndCondition;
};
