// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSnapshotsEditorFilterRow.h"

#include "Data/Filters/ConjunctionFilter.h"
#include "Data/DragDrop/FavoriteFilterDragDrop.h"
#include "LevelSnapshotsEditorStyle.h"
#include "Widgets/SLevelSnapshotsEditorFilterList.h"
#include "Widgets/Filter/SHoverableFilterActions.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "LevelSnapshotsEditor"

void SLevelSnapshotsEditorFilterRow::Construct(
	const FArguments& InArgs, 
	ULevelSnapshotsEditorData* InEditorData,
	UConjunctionFilter* InManagedFilter,
	const bool bShouldShowOrInFront
)
{
	OnClickRemoveRow = InArgs._OnClickRemoveRow;
	ManagedFilterWeakPtr = InManagedFilter;

	TSharedRef<SWidget> FrontOfRow = [bShouldShowOrInFront]()
	{
		if (bShouldShowOrInFront)
		{
			return
				SNew(SBox)
					.WidthOverride(30.f)
				[
					SNew(STextBlock)
	                    .TextStyle( FLevelSnapshotsEditorStyle::Get(), "LevelSnapshotsEditor.FilterRow.Or")
	                    .Text(LOCTEXT("FilterRow.Or", "OR"))
	                    .WrapTextAt(128.0f)
				];
		}
		return
			SNew(SBox)
				.WidthOverride(30.f);
	}();

	ChildSlot
	[
		SNew(SHorizontalBox)

		// OR in front of row
		+ SHorizontalBox::Slot()
		.Padding(7.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			FrontOfRow
		]

		// Row
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillWidth(1.f)
		[
			SNew(SBorder)
			.Padding(FMargin(5.0f, 5.f))
			.BorderImage(FLevelSnapshotsEditorStyle::GetBrush("LevelSnapshotsEditor.GroupBorder"))
			.BorderBackgroundColor_Lambda([this](){ return ManagedFilterWeakPtr.IsValid() && ManagedFilterWeakPtr->IsIgnored() ? FSlateColor(FLinearColor(0.4f, 0.4f, 0.4f, 1.f)) : FSlateColor(FLinearColor(1,1,1,1)); })
			.ColorAndOpacity_Lambda([this](){ return ManagedFilterWeakPtr.IsValid() && ManagedFilterWeakPtr->IsIgnored() ? FLinearColor(0.4f, 0.4f, 0.4f, 1.f) : FLinearColor(1,1,1,1); })
			[
				SNew(SHorizontalBox)

				// Filters
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.Padding(5.f, 5.f)
				.FillWidth(1.f)
				[
					SAssignNew(FilterList, SLevelSnapshotsEditorFilterList, InManagedFilter, InEditorData)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					SNew(SHoverableFilterActions, SharedThis(this))
					.IsFilterIgnored_Lambda([this](){ return ManagedFilterWeakPtr.IsValid() ? ManagedFilterWeakPtr->IsIgnored() : true; })
					.OnChangeFilterIgnored_Lambda([this](bool bNewValue)
					{
						if (ManagedFilterWeakPtr.IsValid())
						{
							ManagedFilterWeakPtr->SetIsIgnored(bNewValue);
						}
					})
					.OnPressDelete_Lambda([this](){ OnClickRemoveRow.ExecuteIfBound(SharedThis(this)); })
				]
			]
		]			
	];
}

const TWeakObjectPtr<UConjunctionFilter>& SLevelSnapshotsEditorFilterRow::GetManagedFilter()
{
	return ManagedFilterWeakPtr;
}

void SLevelSnapshotsEditorFilterRow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FFavoriteFilterDragDrop> DragDrop = DragDropEvent.GetOperationAs<FFavoriteFilterDragDrop>())
	{
		DragDrop->OnEnterRow(SharedThis(this));
	}
}

void SLevelSnapshotsEditorFilterRow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FFavoriteFilterDragDrop> DragDrop = DragDropEvent.GetOperationAs<FFavoriteFilterDragDrop>())
	{
		DragDrop->OnLeaveRow(SharedThis(this));
	}
}

FReply SLevelSnapshotsEditorFilterRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FFavoriteFilterDragDrop> DragDrop = DragDropEvent.GetOperationAs<FFavoriteFilterDragDrop>())
	{
		const bool bDropResult = DragDrop->OnDropOnRow(SharedThis(this));
		return bDropResult ? FReply::Handled().EndDragDrop() : FReply::Unhandled();
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
