// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Data/Filters/ConjunctionFilter.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SLevelSnapshotsEditorFilters;
class SLevelSnapshotsEditorFilterList;
class ULevelSnapshotsEditorData;

/* Creates all widgets needed to show an AND-condition of filters. */
class SLevelSnapshotsEditorFilterRow : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnClickRemoveRow, TSharedRef<SLevelSnapshotsEditorFilterRow>);
	
	SLATE_BEGIN_ARGS(SLevelSnapshotsEditorFilterRow)
	{}
		SLATE_EVENT(FOnClickRemoveRow, OnClickRemoveRow)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, 
		ULevelSnapshotsEditorData* InEditorData,
		UConjunctionFilter* InManagedFilter,
		const bool bShouldShowOrInFront
	);

	const TWeakObjectPtr<UConjunctionFilter>& GetManagedFilter();

	//~ Begin SWidget Interface
	void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End SWidget Interface

private:
	
	FOnClickRemoveRow OnClickRemoveRow;

	TWeakObjectPtr<UConjunctionFilter> ManagedFilterWeakPtr;
	/* Stores all filters */
	TSharedPtr<SLevelSnapshotsEditorFilterList> FilterList;
};

