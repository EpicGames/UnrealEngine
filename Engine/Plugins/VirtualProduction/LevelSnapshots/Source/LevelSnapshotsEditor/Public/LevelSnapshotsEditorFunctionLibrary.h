// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LevelSnapshotsEditorFunctionLibrary.generated.h"

class ULevelSnapshot;
class ULevelSnapshotFilter;

UCLASS()
class LEVELSNAPSHOTSEDITOR_API ULevelSnapshotsEditorFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * @brief Creates a new Level Snapshot asset in the content browser and then captures the target world
	 * @param WorldContextObject Context object to determine which world to take the snapshot in
	 * @param FileName The desired asset file name
	 * @param FolderPath The desired asset location
	 * @param bShouldCreateUniqueFileName If true, the asset name will have a number incrementally added to the file name if an asset with a similar name already exists. If false, the existing asset will be overwritten.
	 */
	UFUNCTION(BlueprintCallable, Category = "LevelSnapshots", meta = (DevelopmentOnly, WorldContext = "WorldContextObject"))
	static ULevelSnapshot* TakeLevelSnapshotAndSaveToDisk(
		const UObject* WorldContextObject, const FString& FileName, const FString& FolderPath, const FString& Description, const bool bShouldCreateUniqueFileName = true);

	/**
	 * Uses TakeLevelSnapshotAndSaveToDisk() and assumes Editor World
	 */
	UFUNCTION(BlueprintCallable, Category = "LevelSnapshots")
	static void TakeAndSaveLevelSnapshotEditorWorld(const FString& FileName, const FString& FolderPath, const FString& Description);

	/* If the snapshot is saved in the registry, takes a screenshot of the editor scene and sets it as thumnail for the snapshot. */
	UFUNCTION(BlueprintCallable, Category = "LevelSnapshots")
	static void GenerateThumbnailForSnapshotAsset(ULevelSnapshot* SnapshotPackage);

};