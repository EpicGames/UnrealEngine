// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSnapshotsLog.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "Params/ObjectSnapshotSerializationData.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"

class FScriptMapHelper;

namespace UE::LevelSnapshots
{
	class ILevelSnapshotsModule;
}

namespace UE::LevelSnapshots::nDisplay::Private
{
	/**
	 * Helper class for a subobject which has a TMap<FString, UObject>
	 * Super class must implement this function: const FMapProperty* GetMapProperty().
	 */
	template<typename TDerived>
	class TMapSubobjectSerializer
		:
		public ICustomObjectSnapshotSerializer,
		public TSharedFromThis<TDerived>
	{
		TDerived* This() { return static_cast<TDerived*>(this); }
		const TDerived* This() const { return static_cast<const TDerived*>(this); }

		UObject* FindSubobject(UObject* Owner, const ISnapshotSubobjectMetaData& ObjectData)
		{
			FString ViewportKey;
			ObjectData.ReadObjectAnnotation(FObjectAnnotator::CreateLambda([&ViewportKey](FArchive& Reader)
			{
				Reader << ViewportKey;
			}));
			
			FScriptMapHelper ViewportsMap = GetSubobjectMap(Owner);
			return GetValue(ViewportsMap, ViewportKey);
		}

		FScriptMapHelper GetSubobjectMap(UObject* Owner) const
		{
			const FMapProperty* MapProp = This()->GetMapProperty();
			return FScriptMapHelper(MapProp, MapProp->ContainerPtrToValuePtr<void>(Owner));
		}
		
		const FString& GetKey(FScriptMapHelper& Map, int32 Index) const
		{
			const FStrProperty* KeyProp = CastField<FStrProperty>(This()->GetMapProperty()->KeyProp);
			check(KeyProp);
		
			const void* KeyValuePtr = Map.GetKeyPtr(Index);
			return KeyProp->GetPropertyValue_InContainer(KeyValuePtr);
		}

		UObject* GetValue(FScriptMapHelper& Map, int32 Index) const
		{
			const FObjectProperty* ValueProp = CastField<FObjectProperty>(This()->GetMapProperty()->ValueProp);
			check(ValueProp);
		
			const void* ValueValuePtr = Map.GetValuePtr(Index);
			return ValueProp->GetObjectPropertyValue(ValueValuePtr);
		}

		UObject* GetValue(FScriptMapHelper& Map, const FString& Key) const
		{
			const FObjectProperty* ValueProp = CastField<FObjectProperty>(This()->GetMapProperty()->ValueProp);
			check(ValueProp);
		
			void* ValuePtr = Map.FindValueFromHash(&Key);
			return ValuePtr ? ValueProp->GetObjectPropertyValue(ValuePtr) : nullptr;
		}

	public:
		
		//~ Begin ICustomObjectSnapshotSerializer Interface
		virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
		{
			FScriptMapHelper ViewportsMap = GetSubobjectMap(EditorObject);
			for (int32 i = 0; i < ViewportsMap.Num(); ++i)
			{
				const FString& Key = GetKey(ViewportsMap, i);
				UObject* Value = GetValue(ViewportsMap, i);
				if (ensure(IsValid(Value)))
				{
					const int32 ViewportIndex = DataStorage.AddSubobjectSnapshot(Value);
					TSharedPtr<ISnapshotSubobjectMetaData> MetaData = DataStorage.GetSubobjectMetaData(ViewportIndex);
					if (ensureMsgf(MetaData, TEXT("AddSubobjectSnapshot failed unexpectingly")))
					{
						MetaData->WriteObjectAnnotation(FObjectAnnotator::CreateLambda([&Key](FArchive& Writer)
						{
							FString ViewportKey = Key;
							Writer << ViewportKey;
						}));
						
						continue;
					}
				}

				UE_LOGF(LogLevelSnapshots, Error, "Failed to save %ls that is supposed to be a subobject of %ls", *Value->GetPathName(), *EditorObject->GetPathName());
			}
		}
		virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
		{
			return FindSubobject(SnapshotObject, ObjectData);
		}
		virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
		{
			return FindSubobject(EditorObject, ObjectData);
		}
		virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
		{
			return FindSubobject(EditorObject, ObjectData);
		}
		//~ Begin ICustomObjectSnapshotSerializer Interface
	};	
}


