// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/ICustomObjectSnapshotSerializer.h"
#include "Params/ObjectSnapshotSerializationData.h"

namespace UE::LevelSnapshots
{
	class ILevelSnapshotsModule;
}

namespace UE::LevelSnapshots::nDisplay::Private
{
	template<typename TDerived>
	class TReferenceSubobjectSerializer
		:
		public ICustomObjectSnapshotSerializer,
		public TSharedFromThis<TDerived>
	{
		TDerived* This() { return static_cast<TDerived*>(this); }
	public:
	
		static void Register(ILevelSnapshotsModule& Module);
	
		//~ Begin ICustomObjectSnapshotSerializer Interface
		virtual void OnTakeSnapshot(UObject* EditorObject, ICustomSnapshotSerializationData& DataStorage) override
		{
			UObject* Subobject = This()->FindSubobject(EditorObject);
			if (IsValid(Subobject))
			{
				DataStorage.AddSubobjectSnapshot(Subobject);
			}
		}
		virtual UObject* FindOrRecreateSubobjectInSnapshotWorld(UObject* SnapshotObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
		{
			return This()->FindSubobject(SnapshotObject);
		}
		virtual UObject* FindOrRecreateSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
		{
			return This()->FindSubobject(EditorObject);
		}
		virtual UObject* FindSubobjectInEditorWorld(UObject* EditorObject, const ISnapshotSubobjectMetaData& ObjectData, const ICustomSnapshotSerializationData& DataStorage) override
		{
			return This()->FindSubobject(EditorObject);
		}
		//~ Begin ICustomObjectSnapshotSerializer Interface
	};
}