// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationClusterNodeSerializer.h"

#include "ILevelSnapshotsModule.h"
#include "UObject/UnrealType.h"

UClass* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterNodeSerializer::GetSupportedClass()
{
	static const FSoftClassPath ClassPath("/Script/DisplayClusterConfiguration.DisplayClusterConfigurationClusterNode");
	return ClassPath.ResolveClass();
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterNodeSerializer::MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module)
{
	const FProperty* ViewportsProperty = GetMapProperty();
	if (ensure(ViewportsProperty))
	{
		Module.AddExplicitlyUnsupportedProperties( { ViewportsProperty } );
	}
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterNodeSerializer::Register(ILevelSnapshotsModule& Module)
{
	MarkPropertiesAsExplicitlyUnsupported(Module);
	Module.RegisterCustomObjectSerializer(GetSupportedClass(), MakeShared<FDisplayClusterConfigurationClusterNodeSerializer>());
}

const FMapProperty* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterNodeSerializer::GetMapProperty()
{
	const FName ViewportsPropertyName("Viewports");
	const FMapProperty* MapProp = CastField<FMapProperty>(GetSupportedClass()->FindPropertyByName(ViewportsPropertyName));
	
	checkf(MapProp, TEXT("Expected class %s to have map property %s"), *GetSupportedClass()->GetName(), *ViewportsPropertyName.ToString());
	return MapProp;
}
