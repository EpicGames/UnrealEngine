// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationClusterSerializer.h"

#include "ILevelSnapshotsModule.h"
#include "Params/ObjectSnapshotSerializationData.h"

#include "UObject/UnrealType.h"

UClass* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterSerializer::GetSupportedClass()
{
	static const FSoftClassPath ClassPath("/Script/DisplayClusterConfiguration.DisplayClusterConfigurationCluster");
	return ClassPath.ResolveClass();
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterSerializer::MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module)
{
	const FProperty* NodesProperty = GetMapProperty();
	if (ensure(NodesProperty))
	{
		Module.AddExplicitlyUnsupportedProperties( { NodesProperty } );
	}
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterSerializer::Register(ILevelSnapshotsModule& Module)
{
	MarkPropertiesAsExplicitlyUnsupported(Module);
	Module.RegisterCustomObjectSerializer(GetSupportedClass(), MakeShared<FDisplayClusterConfigurationClusterSerializer>());
}

const FMapProperty* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationClusterSerializer::GetMapProperty()
{
	const FName NodesPropertyName("Nodes");
	const FMapProperty* Result = CastField<FMapProperty>(GetSupportedClass()->FindPropertyByName(NodesPropertyName));
	check(Result);
	return Result;
}
