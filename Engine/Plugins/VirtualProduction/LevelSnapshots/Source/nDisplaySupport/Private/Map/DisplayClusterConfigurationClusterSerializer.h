// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Helpers/MapSubobjectSerializer.h"

namespace UE::LevelSnapshots
{
	class ILevelSnapshotsModule;
}

namespace UE::LevelSnapshots::nDisplay::Private
{
	class FDisplayClusterConfigurationClusterSerializer : public TMapSubobjectSerializer<FDisplayClusterConfigurationClusterSerializer>
	{
		static UClass* GetSupportedClass(); 
		static void MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module);
	public:
	
		static void Register(ILevelSnapshotsModule& Module);

		//~ Begin TMapSubobjectSerializer Interface
		static const FMapProperty* GetMapProperty();
		//~ End TMapSubobjectSerializer Interface
	};
}

