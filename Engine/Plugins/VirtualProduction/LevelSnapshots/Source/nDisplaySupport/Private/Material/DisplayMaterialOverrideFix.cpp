// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayMaterialOverrideFix.h"

#include "ILevelSnapshotsModule.h"
#include "Params/PropertyComparisonParams.h"

#include "UObject/UnrealType.h"
#include "Components/MeshComponent.h"

namespace UE::LevelSnapshots::nDisplay::Private::Internal
{
	static UClass* GetDisplayClusterPreviewComponentClass()
	{
		static const FSoftClassPath ClassPath("/Script/DisplayCluster.DisplayClusterScreenComponent");
		return ClassPath.ResolveClass();
	}
}


void UE::LevelSnapshots::nDisplay::Private::FDisplayMaterialOverrideFix::Register(ILevelSnapshotsModule& Module)
{
	TSharedRef<FDisplayMaterialOverrideFix> MaterialFix = MakeShared<FDisplayMaterialOverrideFix>();
	MaterialFix->OverrideMaterials = UMeshComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials));
	check(MaterialFix->OverrideMaterials);
	
	Module.RegisterPropertyComparer(Internal::GetDisplayClusterPreviewComponentClass(), MaterialFix);
}

UE::LevelSnapshots::IPropertyComparer::EPropertyComparison UE::LevelSnapshots::nDisplay::Private::FDisplayMaterialOverrideFix::ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const
{
	if (Params.LeafProperty == OverrideMaterials && Params.InspectedClass->IsChildOf(Internal::GetDisplayClusterPreviewComponentClass()))
	{
		// The material is set dynamically every tick, it will always diff and we do not want to restore it
		return EPropertyComparison::TreatEqual;
	}
	
	return EPropertyComparison::CheckNormally;
}
