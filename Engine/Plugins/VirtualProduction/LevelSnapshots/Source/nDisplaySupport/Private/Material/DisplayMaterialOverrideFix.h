// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IPropertyComparer.h"

namespace UE::LevelSnapshots
{
	class ILevelSnapshotsModule;
}

namespace UE::LevelSnapshots::nDisplay::Private
{
	/** Fixes material instances in UDisplayClusterPreviewComponent showing up as changed */
	class FDisplayMaterialOverrideFix : public IPropertyComparer
	{
		FProperty* OverrideMaterials{};

	public:

		static void Register(ILevelSnapshotsModule& Module);
	
		//~ Begin IPropertyComparer Interface
		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override;
		//~ End IPropertyComparer Interface
	};
}

