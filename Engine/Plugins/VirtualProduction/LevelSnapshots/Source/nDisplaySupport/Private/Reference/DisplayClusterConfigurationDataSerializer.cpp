// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationDataSerializer.h"

#include "ILevelSnapshotsModule.h"
#include "UObject/UnrealType.h"

namespace UE::LevelSnapshots::nDisplay::Private::Internal
{
	const FName ClusterPropertyName("Cluster");
}

UClass* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationDataSerializer::GetSupportedClass()
{
	static const FSoftClassPath ClassPath("/Script/DisplayClusterConfiguration.DisplayClusterConfigurationData");
	return ClassPath.ResolveClass();
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationDataSerializer::MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module)
{
	const FName ExportedObjectsPropertyName("ExportedObjects");
	
	const FSoftClassPath ClassPath("/Script/DisplayClusterConfiguration.DisplayClusterConfigurationData_Base");
	UClass* ConfigDataBaseClass = ClassPath.ResolveClass();
	check(ConfigDataBaseClass);
	const FProperty* ExportedObjectsProperty = ConfigDataBaseClass->FindPropertyByName(ExportedObjectsPropertyName);
	
	const FProperty* ClusterProperty = GetSupportedClass()->FindPropertyByName(Internal::ClusterPropertyName);
	if (ensure(ClusterProperty && ExportedObjectsProperty))
	{
		Module.AddExplicitlyUnsupportedProperties( { ClusterProperty, ExportedObjectsProperty} );
	}
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationDataSerializer::Register(ILevelSnapshotsModule& Module)
{
	MarkPropertiesAsExplicitlyUnsupported(Module);
	Module.RegisterCustomObjectSerializer(GetSupportedClass(), MakeShared<FDisplayClusterConfigurationDataSerializer>());
}

UObject* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterConfigurationDataSerializer::FindSubobject(UObject* Owner) const
{
	const FObjectProperty* ObjectProp = CastField<FObjectProperty>(GetSupportedClass()->FindPropertyByName(Internal::ClusterPropertyName));
	checkf(ObjectProp, TEXT("Expected class %s to have object property %s"), *GetSupportedClass()->GetName(), *Internal::ClusterPropertyName.ToString());
	check(Owner->GetClass()->IsChildOf(GetSupportedClass()));

	return ObjectProp->GetObjectPropertyValue(ObjectProp->ContainerPtrToValuePtr<void>(Owner));
}