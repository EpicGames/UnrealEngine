// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Helpers/ReferenceSubobjectSerializer.h"

namespace UE::LevelSnapshots
{
	class ILevelSnapshotsModule;
}

namespace UE::LevelSnapshots::nDisplay::Private
{
	class FDisplayClusterConfigurationDataSerializer : public TReferenceSubobjectSerializer<FDisplayClusterConfigurationDataSerializer>
	{
		static UClass* GetSupportedClass();
		static void MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module);

	public:

		static void Register(ILevelSnapshotsModule& Module);
	
		//~ Begin FDisplayClusterRootActorSerializer Interface
		UObject* FindSubobject(UObject* Owner) const;
		//~ End FDisplayClusterRootActorSerializer Interface
	};
}

