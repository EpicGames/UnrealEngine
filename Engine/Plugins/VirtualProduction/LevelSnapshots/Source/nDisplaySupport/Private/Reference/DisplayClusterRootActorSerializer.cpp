// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRootActorSerializer.h"

#include "ILevelSnapshotsModule.h"
#include "Params/ObjectSnapshotSerializationData.h"
#include "Params/PropertyComparisonParams.h"

#include "Components/MeshComponent.h"
#include "UObject/UnrealType.h"

namespace UE::LevelSnapshots::nDisplay::Private::Internal
{
	const FName CurrentConfigDataPropertyName("CurrentConfigData");
}

UClass* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterRootActorSerializer::GetSupportedClass()
{
	static const FSoftClassPath ClassPath("/Script/DisplayCluster.DisplayClusterRootActor");
	return ClassPath.ResolveClass();
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterRootActorSerializer::MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module)
{
	const FProperty* CurrentConfigDataProperty = GetSupportedClass()->FindPropertyByName(Internal::CurrentConfigDataPropertyName);
	if (ensure(CurrentConfigDataProperty))
	{
		Module.AddExplicitlyUnsupportedProperties( { CurrentConfigDataProperty } );
	}
}

void UE::LevelSnapshots::nDisplay::Private::FDisplayClusterRootActorSerializer::Register(ILevelSnapshotsModule& Module)
{
	MarkPropertiesAsExplicitlyUnsupported(Module);

	const TSharedRef<FDisplayClusterRootActorSerializer> ActorSerializer = MakeShared<FDisplayClusterRootActorSerializer>();
	Module.RegisterCustomObjectSerializer(GetSupportedClass(), ActorSerializer);
	Module.RegisterPropertyComparer(UMeshComponent::StaticClass(), ActorSerializer);
}

UE::LevelSnapshots::nDisplay::Private::FDisplayClusterRootActorSerializer::FDisplayClusterRootActorSerializer()
	: OverrideMaterials(UMeshComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials)))
{
	ensure(OverrideMaterials);
}

UE::LevelSnapshots::IPropertyComparer::EPropertyComparison UE::LevelSnapshots::nDisplay::Private::FDisplayClusterRootActorSerializer::ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const
{
	if (Params.LeafProperty == OverrideMaterials && Params.SnapshotActor->IsA(GetSupportedClass()))
	{
		// Materials are managed by UDisplayClusterPreviewComponent - they cannot be restored
		// 
		// The snapshot version of the material can be null when:
		//		1. ProjectionPolicy is Mesh and
		//		2. There is a mesh component on the nDisplay config
		//		
		// UDisplayClusterPreviewComponent is not created in the snapshot world
		// UDisplayClusterPreviewComponent is the outer for materials assigned to override materials
		// Since the UDisplayClusterPreviewComponent is null, the materials will also be null since they're missing an outer
		return EPropertyComparison::TreatEqual;
	}

	return EPropertyComparison::CheckNormally;
}

UObject* UE::LevelSnapshots::nDisplay::Private::FDisplayClusterRootActorSerializer::FindSubobject(UObject* Owner) const
{
	const FObjectProperty* ObjectProp = CastField<FObjectProperty>(GetSupportedClass()->FindPropertyByName(Internal::CurrentConfigDataPropertyName));
	checkf(ObjectProp, TEXT("Expected class %s to have object property %s"), *GetSupportedClass()->GetName(), *Internal::CurrentConfigDataPropertyName.ToString());
	check(Owner->GetClass()->IsChildOf(GetSupportedClass()));

	return ObjectProp->GetObjectPropertyValue(ObjectProp->ContainerPtrToValuePtr<void>(Owner));
}
