// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Helpers/ReferenceSubobjectSerializer.h"
#include "Interfaces/IPropertyComparer.h"

namespace UE::LevelSnapshots
{
	class ILevelSnapshotsModule;
}

namespace UE::LevelSnapshots::nDisplay::Private
{
	class FDisplayClusterRootActorSerializer
		:
		public TReferenceSubobjectSerializer<FDisplayClusterRootActorSerializer>,
		public IPropertyComparer
	{
		static UClass* GetSupportedClass();
		static void MarkPropertiesAsExplicitlyUnsupported(ILevelSnapshotsModule& Module);

		const FProperty* OverrideMaterials;

	public:

		static void Register(ILevelSnapshotsModule& Module);

		FDisplayClusterRootActorSerializer();

		//~ Begin IPropertyComparer Interface
		virtual EPropertyComparison ShouldConsiderPropertyEqual(const FPropertyComparisonParams& Params) const override;
		//~ End IPropertyComparer Interface

		//~ Begin FDisplayClusterRootActorSerializer Interface
		UObject* FindSubobject(UObject* Owner) const;
		//~ End FDisplayClusterRootActorSerializer Interface
	};
}


