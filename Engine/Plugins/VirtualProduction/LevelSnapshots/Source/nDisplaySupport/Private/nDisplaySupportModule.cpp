// Copyright Epic Games, Inc. All Rights Reserved.

#include "nDisplaySupportModule.h"

#include "ILevelSnapshotsModule.h"
#include "Map/DisplayClusterConfigurationClusterNodeSerializer.h"
#include "Map/DisplayClusterConfigurationClusterSerializer.h"
#include "Material/DisplayMaterialOverrideFix.h"
#include "Reference/DisplayClusterConfigurationDataSerializer.h"
#include "Reference/DisplayClusterRootActorSerializer.h"

#define LOCTEXT_NAMESPACE "nDisplaySupport"

void UE::LevelSnapshots::nDisplay::Private::FnDisplaySupportModule::StartupModule()
{
	FModuleManager& ModuleManager = FModuleManager::Get();
	const bool bIsNDisplayLoaded = ModuleManager.IsModuleLoaded("DisplayCluster");
	if (bIsNDisplayLoaded)
	{
		ILevelSnapshotsModule& LevelSnapshotsModule = ModuleManager.LoadModuleChecked<ILevelSnapshotsModule>("LevelSnapshots");
		
		FDisplayClusterRootActorSerializer::Register(LevelSnapshotsModule);
		FDisplayClusterConfigurationDataSerializer::Register(LevelSnapshotsModule);
		FDisplayClusterConfigurationClusterSerializer::Register(LevelSnapshotsModule);
		FDisplayClusterConfigurationClusterNodeSerializer::Register(LevelSnapshotsModule);

		FDisplayMaterialOverrideFix::Register(LevelSnapshotsModule);
	}
}
void UE::LevelSnapshots::nDisplay::Private::FnDisplaySupportModule::ShutdownModule()
{}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(UE::LevelSnapshots::nDisplay::Private::FnDisplaySupportModule, nDisplaySupport)